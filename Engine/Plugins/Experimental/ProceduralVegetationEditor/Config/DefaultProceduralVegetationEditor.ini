[CoreRedirects]
+ClassRedirects=(OldName="/Script/ProceduralVegetation.PVOutputSettings",NewName="/Script/ProceduralVegetation.PVExportSettings")
+ClassRedirects=(OldName="/Script/ProceduralVegetation.PVSimplySettings",NewName="/Script/ProceduralVegetation.PVSimplifySettings")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVFoliageDistributionParametricSettings",NewName="/Script/ProceduralVegetation.PVDistributionParametricParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVFoliageDistributionHormoneBasedSettings",NewName="/Script/ProceduralVegetation.PVDistributionHormoneBasedParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVFoliageDistributionVectorSettings",NewName="/Script/ProceduralVegetation.PVDistributionVectorParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVFoliageDistributionConditions",NewName="/Script/ProceduralVegetation.PVDistributionConditionParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVFoliageDistributionConditionSettings",NewName="/Script/ProceduralVegetation.PVDistributionConditionInfluence")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVGrafterDistributionParametricSettings",NewName="/Script/ProceduralVegetation.PVDistributionParametricParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVGrafterDistributionHormoneBasedSettings",NewName="/Script/ProceduralVegetation.PVDistributionHormoneBasedParams")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVFoliageDistributionSettingsMode",NewName="/Script/ProceduralVegetation.EPVDistributionSettingsMode")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVFoliageDistributionBasis",NewName="/Script/ProceduralVegetation.EPVDistributionBasis")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVGrafterDistributionSettingsMode",NewName="/Script/ProceduralVegetation.EPVDistributionSettingsMode")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVGrafterDistributionBasis",NewName="/Script/ProceduralVegetation.EPVDistributionBasis")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.PVGrafterDistributionParametricSettings.RandomizeScaleMinimum",NewName="RandomizeScaleAngleMinimum")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.PVGrafterDistributionParametricSettings.RandomizeScaleMaximum",NewName="RandomizeScaleAngleMaximum")
+ClassRedirects=(OldName="/Script/ProceduralVegetation.PVTextureSetupSettings",NewName="/Script/ProceduralVegetation.PVTrunkTextureSetupSettings")
+ClassRedirects=(OldName="/Script/ProceduralVegetation.PVTextureSetupData",NewName="/Script/ProceduralVegetation.PVTrunkTextureSetupData")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVTextureSetupParams",NewName="/Script/ProceduralVegetation.PVTrunkTextureSetupParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVTextureSetupInfo",NewName="/Script/ProceduralVegetation.PVTrunkTextureSetupInfo")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVTextureChannelParams",NewName="/Script/ProceduralVegetation.PVTrunkTextureChannelParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVTextureGenerationParams",NewName="/Script/ProceduralVegetation.PVTrunkTextureGenerationParams")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVGenerationUVRange",NewName="/Script/ProceduralVegetation.PVTrunkTextureGenerationUVRange")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVDataTypeInfoTextureSetup",NewName="/Script/ProceduralVegetation.PVDataTypeInfoTrunkTextureSetup")
+StructRedirects=(OldName="/Script/ProceduralVegetation.PVFoliageAttributes",NewName="/Script/ProceduralVegetation.PVDistributionConditions")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVTextureSetupMode",NewName="/Script/ProceduralVegetation.EPVTrunkTextureSetupMode")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVTextureResolution",NewName="/Script/ProceduralVegetation.EPVTrunkTextureResolution")
+EnumRedirects=(OldName="/Script/ProceduralVegetation.EPVTextureChannel",NewName="/Script/ProceduralVegetation.EPVTrunkTextureChannel")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.PVTrunkTextureSetupSettings.TextureSetupParams",NewName="TrunkTextureSetupParams")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.PVTrunkTextureSetupSettings.TextureSetupInfo",NewName="TrunkTextureSetupInfo")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.PVTrunkTextureSetupData.TextureSetupInfo",NewName="TrunkTextureSetupInfo")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.TrunkGenerationMaterialSetup.bDeriveFromTextureSetup",NewName="bDeriveFromTrunkTextureSetup")
+PropertyRedirects=(OldName="/Script/ProceduralVegetation.PVAuxinParams.MinGravitationalDot",NewName="MinGravitationalDot_DEPRECATED")