{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Procedural Vegetation Editor",
	"Description": "Node Graph based Editor that allows users to create Nanite Foliage ready vegetation directly in the engine. Users can load Procedural Vegetation Presets that contain prebuilt data for a species, and customize/create variations using the node graph.",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "ProceduralVegetation",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "ProceduralVegetationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "GeometryScripting",
			"Enabled": true
		},
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "DynamicWind",
			"Enabled": true
		}
	]
}