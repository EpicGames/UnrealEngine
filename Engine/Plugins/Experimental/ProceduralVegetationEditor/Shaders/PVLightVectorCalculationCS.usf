// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

struct PVLightDetectionData
{
	int PointNumber;
	int RayNumber;
	float3 Direction;
	float LightValue;
	int Hits;
	int FaliureType;
};

struct PVPointLightVectorData
{
	int PointNumber;
	float LightAvailable;
	float3 LightOptimalDirection;
	float3 LightSubOptimalDirection;
};

uint MaxBounce;
uint NumRays;
uint NumPoints;
StructuredBuffer<PVLightDetectionData>    LightDetectionData;
StructuredBuffer<float3>                  Rays;
RWStructuredBuffer<PVPointLightVectorData> PointLightVectorData;

#define PV_EPSILON 1e-6

// Vendor-stable fp32 sqrt — see PVLightDetectionCS.usf. Hardware `sqrt`/`rsqrt` are
// only +/-1 ULP (not correctly rounded) and differ across GPU vendors; one Heron
// correction from the hardware seed removes that dependence. This direction feeds the
// grower's phototropism, so a ULP here propagates into visibly different growth.
float PVStableSqrt(float X)
{
	if (X <= 0.0f)
	{
		return 0.0f;
	}
	float Sh       = INVARIANT(sqrt(X));
	float Residual = INVARIANT_FMA(-Sh, Sh, X);
	float Sl       = INVARIANT_DIV(Residual, INVARIANT_MUL(2.0f, Sh));
	return INVARIANT_ADD(Sh, Sl);
}

float PVLenSq(float3 V)
{
	float Px = INVARIANT_MUL(V.x, V.x);
	float Py = INVARIANT_MUL(V.y, V.y);
	float Pz = INVARIANT_MUL(V.z, V.z);
	return INVARIANT_ADD(INVARIANT_ADD(Px, Py), Pz);
}

float3 InvariantNormalize(float3 V)
{
	float LenSq = PVLenSq(V);
	if (LenSq < PV_EPSILON * PV_EPSILON)
	{
		return float3(0, 0, 0);
	}
	float InvLen = INVARIANT_DIV(1.0f, PVStableSqrt(LenSq));
	return float3(INVARIANT_MUL(V.x, InvLen), INVARIANT_MUL(V.y, InvLen), INVARIANT_MUL(V.z, InvLen));
}

// 2D dispatch: (min(N,65535), ceil(N/65535), 1) groups x [1,1,1] threads.
// One thread = one skeleton point; reads all 64 ray results for that point.
[numthreads(1,1,1)]
void MainCS(uint3 DTid : SV_DispatchThreadID, uint3 GTid : SV_GroupThreadID, uint3 Gid : SV_GroupID)
{
	// Reconstruct flat point index from 2D dispatch ID
	uint PointIndex = DTid.y * 65535u + DTid.x;
	if (PointIndex >= NumPoints)
	{
		return;
	}

	float MaxLight              = -1.0f;
	float TotalLight            = 0.0f;
	float3 LightOptimalDirection    = float3(0,0,0);
	float3 LightSubOptimalDirection = float3(0,0,0);
	int NumSubOptimalDirections = 0;
	float OptimalDot            = 0.0f;
	bool isInside               = false;
	float AverageLight          = 0.674108;

	for (int r = 0; r < NumRays; r++)
	{
		int LightDetectionDataIndex = (int)(PointIndex * NumRays) + r;
		float LightValue   = LightDetectionData[LightDetectionDataIndex].LightValue;
		float3 RayDirection = LightDetectionData[LightDetectionDataIndex].Direction;
		float Dot = RayDirection.z; // dot((0,0,1),RayDirection) is exactly z — avoid the vendor-fused dot
		TotalLight += clamp(LightValue, 0.0f, 1.0f);
		if (LightValue >= MaxLight && Dot > OptimalDot)
		{
			OptimalDot = Dot;
			MaxLight = LightValue;
			LightOptimalDirection = RayDirection;
			if (LightDetectionData[LightDetectionDataIndex].Hits == MaxBounce)
			{
				isInside = true;
				LightOptimalDirection = -LightOptimalDirection;
			}
			else
			{
				isInside = false;
			}
		}

		if (LightDetectionData[LightDetectionDataIndex].Hits > 0)
		{
			LightSubOptimalDirection += -Rays[r];
			NumSubOptimalDirections++;
		}
	}

	if (NumSubOptimalDirections > 0)
	{
		LightSubOptimalDirection /= NumSubOptimalDirections;
	}
	else
	{
		LightSubOptimalDirection = LightOptimalDirection;
	}

	// Guard: hit-ray vectors may cancel to zero (symmetric occlusion) or
	// LightOptimalDirection may be zero (no ray escaped with positive Z, e.g.
	// fully occluded bud). Both cases produce NaN from normalize — fall back
	// to straight up as the safest assumption when light direction is unknown.
	// Compare squared length against the squared threshold — avoids `length()`'s
	// vendor-divergent sqrt in a comparison that could otherwise flip the fallback.
	if (PVLenSq(LightSubOptimalDirection) < (0.0001 * 0.0001))
	{
		LightSubOptimalDirection = float3(0, 0, 1);
	}
	LightSubOptimalDirection = InvariantNormalize(LightSubOptimalDirection);

	// Same guard for the optimal direction: zero here propagates into the
	// phototropism lerp in PVGrower and produces a degenerate apical direction.
	if (PVLenSq(LightOptimalDirection) < (0.0001 * 0.0001))
	{
		LightOptimalDirection = float3(0, 0, 1);
	}

	// lerp(0, 1, x) == x, so the available-light remap is just a division; pin both
	// divides so the per-point LightAvailable agrees across machines.
	float AvailableLight = INVARIANT_DIV(TotalLight, (float)NumRays);
	AvailableLight = isInside ? 0.0 : INVARIANT_DIV(AvailableLight, AverageLight);

	PointLightVectorData[PointIndex].LightAvailable          = AvailableLight;
	PointLightVectorData[PointIndex].LightOptimalDirection   = LightOptimalDirection;
	PointLightVectorData[PointIndex].LightSubOptimalDirection = LightSubOptimalDirection;
	PointLightVectorData[PointIndex].PointNumber             = LightDetectionData[PointIndex * NumRays].PointNumber;
}
