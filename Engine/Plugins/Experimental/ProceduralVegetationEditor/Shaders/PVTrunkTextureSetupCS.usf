// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

float OffsetX;
float TileX;
float TileY;
float ScaleX;

Texture2D<float4> Source;
SamplerState SourceSampler;
Texture2D<float4> SourceB;
SamplerState SourceSamplerB;
RWTexture2D<float4> Target;
uint2 TargetSize;

[numthreads(8,8,1)]
void DilationCS(uint3 DTid : SV_DispatchThreadID, uint3 GTid : SV_GroupThreadID, uint3 Gid : SV_GroupID)
{
	if (DTid.x >= TargetSize.x || DTid.y >= TargetSize.y) return;
	
	float2 UV = (float2(DTid.xy)) / float2(TargetSize.xy);
	UV.x = UV.x - OffsetX;
	UV.x = UV.x * TileX; 
	UV.y = UV.y * TileY;
	
	if (UV.x < 0) UV.x = 0;
	else if (UV.x > ScaleX * TileX) UV.x = 1.0;
	else UV.x = UV.x / ScaleX;
	float4 Color = Source.SampleLevel(SourceSampler, UV, 0);
	
	Target[DTid.xy] = Color;
}

[numthreads(8,8,1)]
void ScaleCS(uint3 DTid : SV_DispatchThreadID, uint3 GTid : SV_GroupThreadID, uint3 Gid : SV_GroupID)
{
	if (DTid.x >= TargetSize.x || DTid.y >= TargetSize.y) return;
	
	float2 UV = (float2(DTid.xy)) / float2(TargetSize.xy);
	//UV.x = UV.x + OffsetX;
	float2 OutTid = float2(DTid.x + OffsetX * TargetSize.x, DTid.y);
		
	if (UV.x <= (ScaleX))
	{
		UV.x = ( UV.x / ScaleX );
		float4 Color = Source.SampleLevel(SourceSampler, UV, 0);
		Target[OutTid] = Color;
	}
}
