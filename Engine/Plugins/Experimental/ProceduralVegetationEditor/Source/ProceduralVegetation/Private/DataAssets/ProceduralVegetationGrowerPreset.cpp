// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataAssets/ProceduralVegetationGrowerPreset.h"

void UProceduralVegetationGrowerPreset::PostLoad()
{
	Super::PostLoad();

	if (GrowthParams.Auxin.MinGravitationalDot != 0.0f)
	{
		GrowthParams.GravityParams.MinGravitationalDot = GrowthParams.Auxin.MinGravitationalDot;
		GrowthParams.Auxin.MinGravitationalDot = 0.0f;
	}
}
