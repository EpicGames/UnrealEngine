// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "Implementations/PVGrower.h"
#include "ProceduralVegetationGrowerPreset.generated.h"

UCLASS(BlueprintType)
class PROCEDURALVEGETATION_API UProceduralVegetationGrowerPreset : public UDataAsset
{
	GENERATED_BODY()
public:
	virtual void PostLoad() override;

	UPROPERTY(EditAnywhere, Category = Settings, meta=( ShowOnlyInnerProperties))
	FPVGrowerParams GrowthParams;
};
