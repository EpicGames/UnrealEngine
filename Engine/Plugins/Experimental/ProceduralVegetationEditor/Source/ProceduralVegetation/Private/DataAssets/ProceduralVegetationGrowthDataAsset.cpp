// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralVegetationGrowthDataAsset.h"
#include "ProceduralVegetationModule.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "HAL/FileManager.h"
#include "Helpers/PVJSONHelper.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"


void UProceduralVegetationGrowthDataAsset::UpdateDataAsset()
{
	UE_LOGF(LogProceduralVegetation, Log, "UpdateDataAsset Clicked");
	
	if (JsonDirectoryPath.Path.IsEmpty())
	{
		UE_LOGF(LogProceduralVegetation, Warning, "Please specify a JSON path");
		return;
	}
	
	FString OutError;
	IFileManager& FileManager = IFileManager::Get();
	
	TArray<FString> FileNames;
	FileManager.FindFiles(FileNames, *JsonDirectoryPath.Path, TEXT("*.json"));
 
	Variants.Empty();
	GrowthVariations.Empty();
	
	FString JSONExtension = ".json";

	for (const auto& FileName : FileNames)
	{
		FString FullPath = JsonDirectoryPath.Path / FileName;

		FManagedArrayCollection Collection;
		FString MetaJsonError;

		if (PV::JSON::LoadGrowthDataJsonToCollection(Collection, FullPath, OutError))
		{
			FString Name = FPaths::GetBaseFilename(FullPath);
			UE_LOGF(LogProceduralVegetation, Log, "Variant %ls loaded from file", *Name);

			Variants.Add(Name, Collection);

			GrowthVariations.Add({FName(Name)});
		}
		
		if (!OutError.IsEmpty())
		{
			UE_LOGF(LogProceduralVegetation, Warning, "Invalid growth data json File %ls %ls",*FullPath, *OutError);
		}
	}
	
	GetPackage()->SetDirtyFlag(true);
}
