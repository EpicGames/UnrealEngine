// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ProceduralVegetationGrowthDataAsset.generated.h"

USTRUCT()
struct FPVGrowthVariationInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Variations", meta=(Tooltip="Name of the variation. Matches the output pin label."))
	FName Name;
};

UCLASS(BlueprintType)
class PROCEDURALVEGETATION_API UProceduralVegetationGrowthDataAsset : public UDataAsset
{
	GENERATED_BODY()
	
public:
	UPROPERTY()
	TMap<FString, FManagedArrayCollection> Variants;

	UPROPERTY(VisibleAnywhere, Category = "Growth Data")
	TArray<FPVGrowthVariationInfo> GrowthVariations;

private:
	UPROPERTY(EditAnywhere, Category="Internal", meta = (DevelopmentOnly))
	FDirectoryPath JsonDirectoryPath;

	UFUNCTION(CallInEditor, Category="Internal", meta = (DevelopmentOnly))
	void UpdateDataAsset();
};
