// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "ProceduralVegetationPlantProfileDataAsset.h"
#include "ProceduralVegetationModule.h"
#include "Helpers/PVJSONHelper.h"

void UProceduralVegetationPlantProfileDataAsset::Load()
{
	FString OutErrorMessage;
	TArray<FPVPlantProfile> OutProfileData;
	
	if (PV::JSON::LoadProfileData(ProfileFilePath.FilePath, OutProfileData, OutErrorMessage))
	{
		if (OutProfileData.Num() > 0)
		{
			Profiles = OutProfileData;
		}
	}
	else if (!OutErrorMessage.IsEmpty())
	{
		UE_LOGF(LogProceduralVegetation, Error, ("PlantProfileDataAsset : Error loading profile data : %ls"), *OutErrorMessage);
	}
}
