// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "ProceduralVegetationPlantProfileDataAsset.generated.h"

USTRUCT(BlueprintType)
struct PROCEDURALVEGETATION_API FPVPlantProfile
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="PlantProfile")
	FString Name;
	UPROPERTY(VisibleAnywhere, Category="PlantProfile")
	TArray<float> Points;
};


UCLASS(BlueprintType)
class PROCEDURALVEGETATION_API UProceduralVegetationPlantProfileDataAsset : public UDataAsset
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category="PlantProfiles")
	TArray<FPVPlantProfile> Profiles;

	UPROPERTY(EditAnywhere, Category="Internal", meta = (DevelopmentOnly))
	FFilePath ProfileFilePath;

	UFUNCTION(CallInEditor, Category="Internal", meta = (DevelopmentOnly))
	void Load();
};