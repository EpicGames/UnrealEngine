// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVData.h"
#include "PCGContext.h"
#include "Data/PCGPointArrayData.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGHelpers.h"
#include "Serialization/ArchiveCrc32.h"
#include "UObject/Package.h"

UPVData::UPVData(const FObjectInitializer& ObjectInitializer)
	: UPCGSpatialData(ObjectInitializer)
{
	Collection = MakeShared<FManagedArrayCollection>();
}

void UPVData::Initialize(FManagedArrayCollection&& InCollection)
{
	Collection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
}

void UPVData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (bFullDataCrc)
	{
		// Implementation note: metadata not supported at this point.

		FString ClassName = StaticClass()->GetPathName();

		Ar << ClassName;
		Ar << *Collection;
#if WITH_EDITOR
		Ar << DebugSettings;
#endif
	}
	else
	{
		AddUIDToCrc(Ar);
	}
}

FBox UPVData::GetBounds() const
{
	return FBox(EForceInit::ForceInit);
}

bool UPVData::SamplePoint(const FTransform& InTransform, const FBox& InBounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	return false;
}

const UPCGBasePointData* UPVData::ToBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGPrimitiveData::CreatePointData);

	UPCGBasePointData* Data = FPCGContext::NewObject_AnyThread<UPCGBasePointData>(Context, GetTransientPackage(), PointDataClass);
	return Data;
}

UPCGSpatialData* UPVData::CopyInternal(FPCGContext* Context) const
{
	UPVData* NewProceduralVegetationData = FPCGContext::NewObject_AnyThread<UPVData>(Context);

	FManagedArrayCollection NewCollection;
	Collection->CopyTo(&NewCollection);
	
	NewProceduralVegetationData->Initialize(MoveTemp(NewCollection));

#if WITH_EDITOR
	NewProceduralVegetationData->DebugSettings = DebugSettings;
#endif
	
	return NewProceduralVegetationData;
}

#if WITH_EDITORONLY_DATA
void UPVData::AddVisualizationCollection(FManagedArrayCollection InCollection, TArray<FPVVisualizationSettings> InVisualizationSettings)
{
	VisualizationCollections.Emplace(MoveTemp(InCollection), MoveTemp(InVisualizationSettings));
}

void UPVData::SetDebugSettings(const FPVDebugSettings& InSettings) const
{
	const FScopeLock Lock(&DebugSettingsMutex);
	DebugSettings = InSettings;
}

void UPVData::SetDebugSettings(FPVDebugSettings&& InSettings) const
{ 
	const FScopeLock Lock(&DebugSettingsMutex);
	DebugSettings = MoveTemp(InSettings);
}

FPVDebugSettings UPVData::GetDebugSettings() const 
{ 
	const FScopeLock Lock(&DebugSettingsMutex);
	return DebugSettings;
}
#endif

const UPCGPointArrayData* UPVData::ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointArrayData>(ToBasePointData(Context, InBounds, UPCGPointArrayData::StaticClass()));
}
const UPCGPointData* UPVData::ToPointData(FPCGContext* Context, const FBox& InBounds) const
{
	return CastChecked<UPCGPointData>(ToBasePointData(Context, InBounds, UPCGPointData::StaticClass()));
}

