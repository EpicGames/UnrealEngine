// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDistributionParams.h"

bool FPVDistributionConditionParams::HasActiveCondition() const
{
	for (EPVDistributionCondition Condition : TEnumRange<EPVDistributionCondition>())
	{
		if (IsActiveCondition(Condition))
		{
			return true;
		}
	}
	return false;
}

bool FPVDistributionConditionParams::IsActiveCondition(const EPVDistributionCondition InCondition) const
{
	switch (InCondition)
	{
	case EPVDistributionCondition::Light:
		return bActivateLight && Light.Weight > 0.0f;

	case EPVDistributionCondition::Scale:
		return bActivateScale && Scale.Weight > 0.0f;

	case EPVDistributionCondition::UpAlignment:
		return bActivateUpAlignment && UpAlignment.Weight > 0.0f;

	case EPVDistributionCondition::Tip:
		return bActivateTip && Tip.Weight > 0.0f;

	case EPVDistributionCondition::Health:
		return bActivateHealth && Health.Weight > 0.0f;

	case EPVDistributionCondition::Height:
		return bActivateHeight && Height.Weight > 0.0f;

	case EPVDistributionCondition::Generation:
		return bActivateGeneration && Generation.Weight > 0.0f;

	default:
		return false;
	}
}

void FPVDistributionConditionParams::SetConditionState(const EPVDistributionCondition InCondition, const bool InNewState)
{
	switch (InCondition)
	{
	case EPVDistributionCondition::Light:
		bActivateLight = InNewState;
		break;

	case EPVDistributionCondition::Scale:
		bActivateScale = InNewState;
		break;

	case EPVDistributionCondition::UpAlignment:
		bActivateUpAlignment = InNewState;
		break;

	case EPVDistributionCondition::Tip:
		bActivateTip = InNewState;
		break;

	case EPVDistributionCondition::Health:
		bActivateHealth = InNewState;
		break;

	case EPVDistributionCondition::Height:
		bActivateHeight = InNewState;
		break;

	case EPVDistributionCondition::Generation:
		bActivateGeneration = InNewState;
		break;

	default:
		check(false);
		break;
	}
}

FPVDistributionConditionInfluence* FPVDistributionConditionParams::GetInfluence(const EPVDistributionCondition InCondition)
{
	switch (InCondition)
	{
	case EPVDistributionCondition::Light:
		return &Light;

	case EPVDistributionCondition::Scale:
		return &Scale;

	case EPVDistributionCondition::UpAlignment:
		return &UpAlignment;

	case EPVDistributionCondition::Tip:
		return &Tip;

	case EPVDistributionCondition::Health:
		return &Health;

	case EPVDistributionCondition::Height:
		return &Height;

	case EPVDistributionCondition::Generation:
		return &Generation;

	default:
		return nullptr;
	}
}

bool FPVDistributionConditionParams::GetInfluence(const EPVDistributionCondition InCondition,
	FPVDistributionConditionInfluence& OutSettings) const
{
	switch (InCondition)
	{
	case EPVDistributionCondition::Light:
		{
			OutSettings =  Light;
			return true;
		}
	case EPVDistributionCondition::Scale:
		{
			OutSettings = Scale;
			return true;
		}
	case EPVDistributionCondition::UpAlignment:
		{
			OutSettings = UpAlignment;
			return true;
		}
	case EPVDistributionCondition::Tip:
		{
			OutSettings = Tip;
			return true;
		}
	case EPVDistributionCondition::Health:
		{
			OutSettings = Health;
			return true;
		}
	case EPVDistributionCondition::Height:
		{
			OutSettings = Height;
			return true;
		}
	case EPVDistributionCondition::Generation:
		{
			OutSettings = Generation;
			return true;
		}
	default:
		return false;
	}
}
