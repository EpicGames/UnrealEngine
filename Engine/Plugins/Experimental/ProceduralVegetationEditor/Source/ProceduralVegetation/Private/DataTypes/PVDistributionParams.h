// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Utils/PVFloatRamp.h"
#include "Helpers/PVPhyllotaxyHelper.h"

#include "Misc/EnumRange.h"

#include "PVDistributionParams.generated.h"

UENUM()
enum class EPVDistributionCondition : uint8
{
	Light,
	Scale,
	UpAlignment,
	Tip,
	Health,
	Height,
	Generation,
	Count UMETA(Hidden),
	None UMETA(Hidden),
};

ENUM_RANGE_BY_COUNT(EPVDistributionCondition, EPVDistributionCondition::Count)

USTRUCT()
struct FPVDistributionConditionInfluence
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (ClampMin = 0.0f, ClampMax = 1.0f, UIMin = 0.0f, UIMax = 1.0f, Tooltip = "How heavily this condition counts in the combined score.\n\nMultiplied with this condition's measured value. Use to weight one condition more than others."))
	float Weight = 1.0f;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (ClampMin = -1.0f, ClampMax = 1.0f, UIMin = -1.0f, UIMax = 1.0f, Tooltip = "Constant added to this condition's score.\n\nShifts the condition's contribution up or down. Useful to favor or penalize a condition."))
	float Offset = 0.0f;

	FString ToString() const
	{
		return FString::Printf(TEXT("Weight = %f, Offset = %f"), Weight, Offset);
	}
};

USTRUCT()
struct FPVDistributionConditionParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (ClampMin = 0.0f, ClampMax = 1.0f, UIMin = 0.0f, UIMax = 1.0f, Tooltip = "Minimum combined condition score required for placement.\n\nEach active condition contributes a 0-1 score; scores are combined (weighted) and compared to this threshold. Instances below the threshold are skipped. Higher = stricter filtering; lower = more permissive."))
	float CutoffThreshold = 0.3f;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (ClampMin = 1, ClampMax = 10, UIMin = 0, UIMax = 10, Tooltip = "Minimum instances to place per branch regardless of conditions.\n\nSafety net: even if conditions reject everything, this many instances are considered viable candidates on each branch. Prevents bare branches when filtering is too aggressive."))
	int32 MinimumCandidates = 1;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable Light as a placement filter.\n\nWhen on, exposes the Light influence with `Weight` and `Offset` fields."))
	bool bActivateLight = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateLight", Tooltip = "Light influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence Light;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable Scale as a placement filter.\n\nWhen on, exposes the Scale influence with `Weight` and `Offset` fields."))
	bool bActivateScale = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateScale", Tooltip = "Scale influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence Scale;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable UpAlignment as a placement filter.\n\nWhen on, exposes the UpAlignment influence with `Weight` and `Offset` fields."))
	bool bActivateUpAlignment = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateUpAlignment", Tooltip = "UpAlignment influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence UpAlignment;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable Tip as a placement filter.\n\nWhen on, exposes the Tip influence with `Weight` and `Offset` fields."))
	bool bActivateTip = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateTip", Tooltip = "Tip influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence Tip;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable Health as a placement filter.\n\nWhen on, exposes the Health influence with `Weight` and `Offset` fields."))
	bool bActivateHealth = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateHealth", Tooltip = "Health influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence Health;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable Height as a placement filter.\n\nWhen on, exposes the Height influence with `Weight` and `Offset` fields."))
	bool bActivateHeight = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateHeight", Tooltip = "Height influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence Height;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (PinHiddenByDefault, InlineEditConditionToggle, Tooltip = "Enable Generation as a placement filter.\n\nWhen on, exposes the Generation influence with `Weight` and `Offset` fields."))
	bool bActivateGeneration = false;

	UPROPERTY(EditAnywhere, Category = "ConditionSettings", meta = (EditCondition = "bActivateGeneration", Tooltip = "Generation influence configuration: Weight + Offset."))
	FPVDistributionConditionInfluence Generation;

	bool HasActiveCondition() const;

	bool IsActiveCondition(const EPVDistributionCondition InCondition) const;

	void SetConditionState(const EPVDistributionCondition InCondition, const bool InNewState);

	FPVDistributionConditionInfluence* GetInfluence(const EPVDistributionCondition InCondition);

	bool GetInfluence(const EPVDistributionCondition InCondition, FPVDistributionConditionInfluence& OutSettings) const;
};

UENUM()
enum class EPVAimVectorBlendAttribute : uint8
{
	PlantGradient,
	PlantGradientNormalized,
	BranchGradient,
	BranchGradientNormalized,
	WorldUpDot,
};

UENUM()
enum class EPVAimVectorType : uint8
{
	BranchUpFlatten,
	AxisFlatten,
	AxisAim,
	LightOptimal,
	LightAvoid
};

USTRUCT()
struct FPVAimVectorSettings
{
	GENERATED_BODY()

	FPVAimVectorSettings()
	{
		VectorRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(Tooltip="Apply this aim vector to the branch tip too."))
	bool bAffectTip = false;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(Tooltip="What attribute drives the blend ramp.\n\nPicks gradient that drives the ramp's X axis (plant gradient, branch gradient, world-up dot, etc.)."))
	EPVAimVectorBlendAttribute BlendAttribute = EPVAimVectorBlendAttribute::PlantGradient;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(Tooltip="Blend between two aim vectors using the ramp.\n\nWhen enabled, exposes Vector1/Vector2 configuration. When disabled, only Vector2 is used and the previous entry output represents Vector1."))
	bool bDualVectors = false;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(EditCondition="bDualVectors", EditConditionHides, Tooltip="Aim vector type for the blend's start.\n\nChoose from BranchUpFlatten, AxisFlatten, AxisAim, LightOptimal, LightAvoid."))
	EPVAimVectorType Vector1 = EPVAimVectorType::BranchUpFlatten;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(DisplayName="Vector 1 Axis", EditCondition="bDualVectors && (Vector1 == EPVAimVectorType::AxisFlatten || Vector1 == EPVAimVectorType::AxisAim)", EditConditionHides, Tooltip="Custom axis vector for AxisFlatten / AxisAim modes."))
	FVector3f Vector1Axis = FVector3f::UpVector;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(DisplayName="Vector 1 Strength", ClampMin=0.0f, ClampMax=1.0f, EditCondition="bDualVectors", EditConditionHides, Tooltip="How strongly this vector contributes to the final aim.\n\n0 = ignored. 1 = full influence. Combine with the ramp to vary strength along the plant."))
	float Vector1Strength = 1.0f;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(Tooltip="Aim vector type for the blend's end.\n\nChoose from BranchUpFlatten, AxisFlatten, AxisAim, LightOptimal, LightAvoid."))
	EPVAimVectorType Vector2 = EPVAimVectorType::BranchUpFlatten;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(DisplayName="Vector 2 Axis", EditCondition="Vector2 == EPVAimVectorType::AxisFlatten || Vector2 == EPVAimVectorType::AxisAim", EditConditionHides, Tooltip="Custom axis vector for AxisFlatten / AxisAim modes."))
	FVector3f Vector2Axis = FVector3f::UpVector;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(DisplayName="Vector 2 Strength", ClampMin=0.0f, ClampMax=1.0f, Tooltip="How strongly this vector contributes to the final aim.\n\n0 = ignored. 1 = full influence. Combine with the ramp to vary strength along the plant."))
	float Vector2Strength = 1.0f;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve that blends from Vector1 (at ramp 0) to Vector2 (at ramp 1)."))
	FPVFloatRamp VectorRamp;
};

UENUM()
enum class EPVFaceVectorType : uint8
{
	Apical,
	Branch,
	AxisFlatten,
	AxisAim,
	LightOptimal,
	LightAvoid,
};

USTRUCT()
struct FPVFaceVectorSettings
{
	GENERATED_BODY()

	FPVFaceVectorSettings()
	{
		VectorRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(Tooltip="Apply this face vector to the branch tip too."))
	bool bAffectTip = false;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(Tooltip="What attribute drives the blend ramp.\n\nPicks gradient that drives the ramp's X axis (plant gradient, branch gradient, world-up dot, etc.)."))
	EPVAimVectorBlendAttribute BlendAttribute = EPVAimVectorBlendAttribute::PlantGradient;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(Tooltip="Blend between two face vectors using the ramp.\n\nWhen enabled, exposes Vector1/Vector2 configuration. When disabled, only Vector2 is used and the previous entry output represents Vector1."))
	bool bDualVectors = false;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(EditCondition="bDualVectors", EditConditionHides, Tooltip="Face vector type for the blend's start.\n\nChoose from Apical, Branch, AxisFlatten, AxisAim, LightOptimal, LightAvoid."))
	EPVFaceVectorType Vector1 = EPVFaceVectorType::Apical;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(DisplayName="Vector 1 Axis", EditCondition="bDualVectors && (Vector1 == EPVFaceVectorType::AxisFlatten || Vector1 == EPVFaceVectorType::AxisAim)", EditConditionHides, Tooltip="Custom axis vector for AxisFlatten / AxisAim modes."))
	FVector3f Vector1Axis = FVector3f::UpVector;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(DisplayName="Vector 1 Strength", ClampMin=0.0f, ClampMax=1.0f, EditCondition="bDualVectors", EditConditionHides, Tooltip="How strongly this vector contributes to the final face direction.\n\n0 = ignored. 1 = full influence. Combine with the ramp to vary strength along the plant."))
	float Vector1Strength = 1.0f;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(Tooltip="Face vector type for the blend's end.\n\nChoose from Apical, Branch, AxisFlatten, AxisAim, LightOptimal, LightAvoid."))
	EPVFaceVectorType Vector2 = EPVFaceVectorType::Branch;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(DisplayName="Vector 2 Axis", EditCondition="Vector2 == EPVFaceVectorType::AxisFlatten || Vector2 == EPVFaceVectorType::AxisAim", EditConditionHides, Tooltip="Custom axis vector for AxisFlatten / AxisAim modes."))
	FVector3f Vector2Axis = FVector3f::UpVector;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(ClampMin=0.0f, ClampMax=1.0f, DisplayName="Vector 2 Strength", Tooltip="How strongly this vector contributes to the final face direction.\n\n0 = ignored. 1 = full influence. Combine with the ramp to vary strength along the plant."))
	float Vector2Strength = 1.0f;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve that blends from Vector1 (at ramp 0) to Vector2 (at ramp 1)."))
	FPVFloatRamp VectorRamp;
};

USTRUCT()
struct FPVDistributionAimVectorSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(Tooltip="Automatically align the foliage to the branch end direction on tips.\n\nWhen enabled, foliage placed on the tip aligns with the branch tip's direction. Avoids unintended tip orientations."))
	bool bAutoAlignEnd = true;

	UPROPERTY(EditAnywhere, Category="AimVectorSettings", meta=(Tooltip="List of aim-vector strategies for orienting foliage.\n\nAim vector is the direction the leaf should be pointing, root to tip. Each entry blends one or two aim vectors based on a ramp. Most cases need only one entry. Use multiple to layer different orientations (e.g. tip-aiming for the lower half, light-seeking for the upper half). By default with a single entry it blends against the basic phyllotaxy."))
	TArray<FPVAimVectorSettings> AimVectors;
};

USTRUCT()
struct FPVDistributionFaceVectorSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(Tooltip="Automatically align the foliage face to the branch end direction on tips.\n\nWhen enabled, foliage placed on the tip aligns with the branch tip's direction. Avoids unintended tip orientations."))
	bool bAutoAlignEnd = true;

	UPROPERTY(EditAnywhere, Category="FaceVectorSettings", meta=(Tooltip="List of face-vector strategies for orienting foliage.\n\nFace vector governs the direction the foliage should be facing (orthogonal to the aim vector). Each entry blends one or two face vectors based on a ramp. Most cases need only one entry. Use multiple to layer different orientations."))
	TArray<FPVFaceVectorSettings> FaceVectors;
};

UENUM()
enum class EPVRollPitchYawMode : uint8
{
	Roll  UMETA(DisplayName="Roll"),
	Pitch UMETA(DisplayName="Pitch"),
	Yaw   UMETA(DisplayName="Yaw"),
};

USTRUCT()
struct FPVRollPitchYawSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="RollPitchYawSettings", meta=(Tooltip="Which axis to randomize: Roll, Pitch, or Yaw."))
	EPVRollPitchYawMode Mode = EPVRollPitchYawMode::Roll;

	UPROPERTY(EditAnywhere, Category="RollPitchYawSettings", meta=(ClampMin=-1.0f, ClampMax=1.0f, Tooltip="Minimum random range mapped to a rotation angle.\n\n0 to 1 maps to 0° to 360°. Use small ranges for subtle jitter; ±1 for full random rotation."))
	float MinStrength = 0.0f;

	UPROPERTY(EditAnywhere, Category="RollPitchYawSettings", meta=(ClampMin=-1.0f, ClampMax=1.0f, Tooltip="Maximum random range mapped to a rotation angle.\n\n0 to 1 maps to 0° to 360°. Use small ranges for subtle jitter; ±1 for full random rotation."))
	float MaxStrength = 0.0f;

	UPROPERTY(EditAnywhere, Category="RollPitchYawSettings", meta=(Tooltip="Seed for this random rotation.\n\nChange to vary the random pattern without changing strength."))
	int32 RandomSeed = 123456;
};

USTRUCT()
struct FPVDistributionRollPitchYawSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="RollPitchYawSettings", meta=(Tooltip="List of random rotation operations applied after aim/face.\n\nEach entry adds a random Roll, Pitch, or Yaw rotation. Use multiple for layered jitter (e.g. roll for spin, then pitch for tilt)."))
	TArray<FPVRollPitchYawSettings> RollPitchYaw;
};

USTRUCT()
struct FPVDistributionVectorParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Vector Settings")
	FPVDistributionAimVectorSettings AimVectorSettings;

	UPROPERTY(EditAnywhere, Category="Vector Settings")
	FPVDistributionFaceVectorSettings FaceVectorSettings;

	UPROPERTY(EditAnywhere, Category="Vector Settings")
	FPVDistributionRollPitchYawSettings RollPitchYawSettings;
};

UENUM()
enum class EPVDistributionSettingsMode : uint8
{
	ParametricSettings UMETA(DisplayName="Use Parametric Settings"),
	HormoneBasedSettings UMETA(DisplayName="Use Hormone Based Settings")
};

USTRUCT()
struct FPVHormoneDistributionSettings
{
	GENERATED_BODY()

	FPVHormoneDistributionSettings()
	{
		InstanceSpacingRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="Distribution Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Branch points above this ethylene level keep their foliage.\n\nHigher Threshold → more buds/branches retained, making the plant foliage travel further down the trunk. Lower Threshold → fewer retained, concentrates foliage more at tips. Drives the natural look of where foliage is and isn't."))
	float EthyleneThreshold = 0.85f;

	UPROPERTY(EditAnywhere, Category="Distribution Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Minimum distance between placed foliage instances.\n\nHigher spacing yields a more open distribution; lower spacing allows denser placement. Combine with InstanceSpacingRamp to vary along the plant."))
	float InstanceSpacing = 0.3f;

	UPROPERTY(EditAnywhere, Category="Distribution Settings", meta=(PCG_Overridable, XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve that varies instance spacing along plant height.\n\nX = normalized gradient along the plant, 0 at root and 1 at tips. Y = spacing multiplier. Use to reduce or increase density."))
	FPVFloatRamp InstanceSpacingRamp;

	UPROPERTY(EditAnywhere, Category="Distribution Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Blend strength between base spacing and the ramp curve.\n\nLower values favor the base spacing (subtle ramp effect). Higher values apply the ramp more strongly. Use to subtly tune the ramp's influence."))
	float InstanceSpacingRampEffect = 0.0f;

	UPROPERTY(EditAnywhere, Category="Distribution Settings", meta=(PCG_Overridable, ClampMin=-1, ClampMax=1000, Tooltip="Cap on foliage instances per branch (-1 = no cap).\n\nUseful to prevent overcrowding on very long branches. -1 disables the cap."))
	int32 MaxPerBranch = -1;
};

USTRUCT()
struct FPVHormoneScaleSettings
{
	GENERATED_BODY()

	FPVHormoneScaleSettings()
	{
		ScaleRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, UIMin=0.0f, UIMax=5.0f, Tooltip="Base uniform scale for placed foliage.\n\nSets the base uniform scale applied to each instance before randomness and ramps. Increase to enlarge all instances proportionally."))
	float BaseScale = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="How much branch size affects foliage size.\n\n0 = scale independent of branch. 1 = larger branches receive larger foliage. Useful for natural species variation."))
	float BranchScaleImpact = 0.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, UIMin=0.0f, UIMax=5.0f, Tooltip="Lower clamp for final foliage scale.\n\nClamps the final computed scale so it never goes below this value. Use to prevent tiny, hard-to-see instances."))
	float MinScale = 0.5f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, UIMin=0.0f, UIMax=5.0f, Tooltip="Upper clamp for final foliage scale.\n\nClamps the final computed scale so it never exceeds this value. Use to cap unusually large instances for visual consistency."))
	float MaxScale = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, UIMin=0.0f, UIMax=5.0f, Tooltip="Lower bound for per-instance size variation."))
	float RandomScaleMin = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=1.0f, UIMin=1.0f, UIMax=5.0f, Tooltip="Upper bound for per-instance size variation."))
	float RandomScaleMax = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve that varies foliage scale along plant height."))
	FPVFloatRamp ScaleRamp;
};

USTRUCT()
struct FPVHormoneAngleSettings
{
	GENERATED_BODY()

	FPVHormoneAngleSettings()
	{
		AxilAngleRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, Tooltip="Use a custom axil angle instead of the species default.\n\nWhen enabled, uses AxilAngle and the AxilAngleRamp settings to tilt foliage. When off, foliage uses the orientation inherited from the Grower's phyllotaxy."))
	bool OverrideAxilAngle = true;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, EditCondition="OverrideAxilAngle", ClampMin = -90.0f, ClampMax=90.0f, UIMin=-90.0f, UIMax=90.0f, Tooltip="Base tilt from the parent axis.\n\nSets the base axil angle in degrees — the tilt of the instance relative to the parent branch's direction. Acts as the bound when used with a ramp."))
	float AxilAngle = 35.0f;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(EditCondition="OverrideAxilAngle", XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=-1.0f, YAxisMax=1.0f, Tooltip="Curve that varies the axil angle along plant height."))
	FPVFloatRamp AxilAngleRamp;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, EditCondition="OverrideAxilAngle", ClampMin=0.0f, ClampMax=90.0f, UIMin=0.0f, UIMax=90.0f, Tooltip="Maximum angle reached when the ramp evaluates to 1."))
	float AxilAngleRampUpperValue = 45.0f;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, EditCondition="OverrideAxilAngle", ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Blend strength of the axil angle ramp."))
	float AxilAngleRampEffect = 0.0f;
};

USTRUCT()
struct FPVHormonePhyllotaxySettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, Tooltip="Restart the phyllotaxy pattern at each branch.\n\nWhen enabled, each branch's foliage starts the phyllotaxy pattern from rotation 0 in relation to the branch up direction. When disabled the rotation is inherited from the parent and carried over to the branches."))
	bool ResetPhyllotaxy = false;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, Tooltip="Foliage arrangement pattern around the stem.\n\nAlternate (180° flip), Opposite (paired), Whorled (3+ at same node), Spiral (continuous rotation)."))
	EPhyllotaxyType PhyllotaxyType = EPhyllotaxyType::Alternate;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, EditCondition="PhyllotaxyType == EPhyllotaxyType::Spiral", EditConditionHides, Tooltip="Spiral sub-formation angle.\n\nThis is the number of branches that occur in a full 360° revolve. E.g. Distichous is two and by extension; Distichous (180°), Tristichous (120°), Pentastichous (144°), Octastichous (135°), Parastichous (0°)."))
	EPhyllotaxyFormation PhyllotaxyFormation = EPhyllotaxyFormation::Octastichous;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, EditCondition="PhyllotaxyType == EPhyllotaxyType::Whorled", EditConditionHides, ClampMin=1, ClampMax=10, UIMin=1, UIMax=10, Tooltip="Minimum buds per node for Whorled phyllotaxy.\n\nSets the lower bound on instances at each node. Guarantees at least this many. Should be ≤ MaximumNodeBuds."))
	int32 MinimumNodeBuds = 2;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, EditCondition="PhyllotaxyType == EPhyllotaxyType::Whorled", EditConditionHides, ClampMin=1, ClampMax=10, UIMin=1, UIMax=10, Tooltip="Maximum buds per node for Whorled phyllotaxy.\n\nCaps the number of instances spawned at each node. Use in unison with MinimumNodeBuds to set upper and lower bounds."))
	int32 MaximumNodeBuds = 3;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, Tooltip="Place a single foliage instance at each branch tip.\n\nWhen enabled, the branch tip gets exactly one foliage instance. When disabled, the tip gets the same Min/Max bud count as other nodes."))
	bool bSingleBudTip = true;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=360.0f, UIMin=0.0f, UIMax=360.0f, Tooltip="Extra rotation added to each node's phyllotaxy angle.\n\nAdds to the base angle picked by `PhyllotaxyType`/`PhyllotaxyFormation`. Use to break perfect regularity, or for Parastichous patterns where base = 0°."))
	float PhyllotaxyAdditionalAngle = 0.0f;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=360.0f, UIMin=0.0f, UIMax=360.0f, Tooltip="Initial rotation of the phyllotaxy pattern.\n\nPhase-shifts the entire pattern. Useful when the default rotation doesn't align with how your scene's other plants look."))
	float PhyllotaxyOffset = 0.0f;
};

USTRUCT()
struct FPVDistributionHormoneBasedParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Distribution Settings", meta=(ShowOnlyInnerProperties))
	FPVHormoneDistributionSettings DistributionSettings;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(ShowOnlyInnerProperties))
	FPVHormoneScaleSettings ScaleSettings;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(ShowOnlyInnerProperties))
	FPVHormoneAngleSettings AngleSettings;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(ShowOnlyInnerProperties))
	FPVHormonePhyllotaxySettings PhyllotaxySettings;
};


UENUM()
enum class EPVDistributionBasis : uint8
{
	Plant UMETA(DisplayName="Plant"),
	Branch UMETA(DisplayName="Branch")
};

USTRUCT()
struct FPVParametricSpacingSettings
{
	GENERATED_BODY()

	FPVParametricSpacingSettings()
	{
		SpacingRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(PCG_Overridable, UIMin=0, UIMax=100, ClampMin=0, ClampMax=10000, Tooltip="Number of foliage instances per asset.\n\nTotal instances scattered throughout the entire tree. Higher = denser foliage. Combine with `RelativeStart` and `RelativeEnd` to limit the range, and with `SpacingRamp` to vary density along the branch or plant."))
	int32 BranchDensity = 45;

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Fraction along the branch where placement begins.\n\n0 = start from the branch base. 0.5 = start at the midpoint. Useful for plants that have bare lower stems or for separating the distribution into multiple distributors in a chain."))
	float RelativeStart = 0.0f;

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Fraction along the branch where placement ends.\n\n1 = continues to the branch tip. Lower values stop placement before the tip — useful when you want clean tips without foliage."))
	float RelativeEnd = 1.0f;

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(PCG_Overridable, Tooltip="Restrict placement to branches above a starting generation.\n\nWhen on, only branches at or beyond `StartGeneration` receive foliage. Use to keep the trunk bare while putting foliage on side branches."))
	bool LimitStartGeneration = false;

	UPROPERTY(EditAnywhere, Category="Spacing Settings",
		meta=(PCG_Overridable, ClampMin=0, ClampMax=10, EditCondition="LimitStartGeneration", EditConditionHides, Tooltip="First generation to receive foliage.\n\n1 = trunk. 2 = first side branches. Higher numbers exclude lower-order branches from foliage placement."))
	int32 StartGeneration = 0;

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(PCG_Overridable, Tooltip="Restrict placement to branches below an ending generation.\n\nWhen on, foliage stops at `EndGeneration`. Use to keep the finest twigs free of foliage."))
	bool LimitEndGeneration = false;

	UPROPERTY(EditAnywhere, Category="Spacing Settings",
		meta=(PCG_Overridable, ClampMin=0, ClampMax=10, EditCondition="LimitEndGeneration", EditConditionHides, Tooltip="Last generation to receive foliage.\n\nHigher numbers include more generations. 10 = effectively unlimited."))
	int32 EndGeneration = 0;

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(PCG_Overridable, Tooltip="Whether the spacing ramp is normalized to the plant or to each branch.\n\nPlant: ramp covers the whole plant from 0 at the root and 1 at each tip. Branch: ramp covers each branch's length individually."))
	EPVDistributionBasis SpacingBasis = EPVDistributionBasis::Plant;

	UPROPERTY(EditAnywhere, Category="Spacing Settings",
		meta=(PCG_Overridable, XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve that varies instance spacing along the plant or branch.\n\nX = position along the spacing basis. Y = relative output position. Typical use case is leaving start and end as default and add a keypoint in between to concentrate the density of foliage either around the start and end."))
	FPVFloatRamp SpacingRamp;
};

USTRUCT()
struct FPVParametricPhyllotaxySettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, Tooltip="Restart the phyllotaxy pattern at each branch.\n\nWhen enabled, each branch's foliage starts the phyllotaxy pattern from rotation 0 in relation to the branch up direction. When disabled the rotation is inherited from the parent and carried over to the branches."))
	bool ResetPhyllotaxy = false;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, Tooltip="Foliage arrangement pattern around the stem.\n\nAlternate (180° flip), Opposite (paired), Whorled (3+ at same node), Spiral (continuous rotation)."))
	EPhyllotaxyType PhyllotaxyType = EPhyllotaxyType::Spiral;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, EditCondition="PhyllotaxyType == EPhyllotaxyType::Spiral", EditConditionHides, Tooltip="Spiral sub-formation angle.\n\nThis is the number of branches that occur in a full 360° revolve. E.g. Distichous is two and by extension; Distichous (180°), Tristichous (120°), Pentastichous (144°), Octastichous (135°), Parastichous (0°)."))
	EPhyllotaxyFormation PhyllotaxyFormation = EPhyllotaxyFormation::Octastichous;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, EditCondition="PhyllotaxyType == EPhyllotaxyType::Whorled", EditConditionHides, ClampMin=1, ClampMax=10, UIMin=1, UIMax=10, Tooltip="Minimum buds per node for Whorled phyllotaxy.\n\nSets the lower bound on instances at each node. Guarantees at least this many. Should be ≤ MaximumNodeBuds."))
	int32 MinimumNodeBuds = 2;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, EditCondition="PhyllotaxyType == EPhyllotaxyType::Whorled", EditConditionHides, ClampMin=1, ClampMax=10, UIMin=1, UIMax=10, Tooltip="Maximum buds per node for Whorled phyllotaxy.\n\nCaps the number of instances spawned at each node. Use in unison with MinimumNodeBuds to set upper and lower bounds."))
	int32 MaximumNodeBuds = 3;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(PCG_Overridable, Tooltip="Place a single foliage instance at each branch tip.\n\nWhen enabled, the branch tip gets exactly one foliage instance. When disabled, the tip gets the same Min/Max bud count as other nodes."))
	bool bSingleBudTip = true;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=360.0f, UIMin=0.0f, UIMax=360.0f, Tooltip="Extra rotation added to each node's phyllotaxy angle.\n\nAdds to the base angle picked by `PhyllotaxyType`/`PhyllotaxyFormation`. Use to break perfect regularity, or for Parastichous patterns where base = 0°."))
	float PhyllotaxyAdditionalAngle = 0.0f;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings",
		meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=360.0f, UIMin=0.0f, UIMax=360.0f, Tooltip="Initial rotation of the phyllotaxy pattern.\n\nPhase-shifts the entire pattern. Useful when the default rotation doesn't align with how your scene's other plants look."))
	float PhyllotaxyOffset = 0.0f;
};

USTRUCT()
struct FPVParametricAngleSettings
{
	GENERATED_BODY()

	FPVParametricAngleSettings()
	{
		AxilAngleRamp.InitializeLinearCurve();
	}

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=360.0f, Tooltip="Base rotation around the branch axis.\n\nSpins each foliage instance around its parent branch's axis. Use to orient leaves in a specific direction relative to the branch."))
	float Rotation = 0.0f;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, ClampMin=-90.0f, ClampMax=90.0f, UIMin = -90.0f, UIMax = 90.0f, Tooltip="Base tilt of foliage relative to the branch direction.\n\n0° = foliage points along the branch (parallel). 90° = foliage sticks straight out (perpendicular). Combine with `AxilAngleRamp` for per-position variation."))
	float AxilAngle = 35.0f;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, ClampMin=-45.0f, ClampMax=0.0f, Tooltip="Minimum random tilt added to each instance.\n\nLower bound of random variation in axil angle per instance. Use to break uniformity."))
	float RandomizeAxilAngleMinimum = 0.0f;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=45.0f, Tooltip="Maximum random tilt added to each instance.\n\nUpper bound of random variation. Combine with the Minimum for symmetric jitter (e.g. -5 to +5)."))
	float RandomizeAxilAngleMaximum = 0.0f;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(PCG_Overridable, Tooltip="Whether the axil angle ramp is normalized to plant or branch.\n\nPlant: ramp spans the full plant where it's 0 at root and one at all tips. Branch: ramp spans each branch individually."))
	EPVDistributionBasis AxilAngleRampBasis = EPVDistributionBasis::Plant;

	UPROPERTY(EditAnywhere, Category="Angle Settings",
		meta=(PCG_Overridable, XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=-1.0f, YAxisMax=1.0f, Tooltip="Curve that varies axil angle along the plant or branch.\n\nX = position. Y = -1 to 1, mapped into the angle range. Use to make foliage have a steeper angle at branch tips, or to droop near the base."))
	FPVFloatRamp AxilAngleRamp;
};

USTRUCT()
struct FPVParametricScaleSettings
{
	GENERATED_BODY()

	FPVParametricScaleSettings()
	{
		ScaleRamp.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.1f));
	}

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, Tooltip="Whether the scale ramp is normalized to plant or branch."))
	EPVDistributionBasis ScaleRampBasis = EPVDistributionBasis::Plant;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=10.0f, Tooltip="Base uniform scale multiplier for all foliage.\n\nMultiplier applied to every instance before randomness and ramps. 1 = use the mesh's authored size; 0.5 = half-size; 2 = double-size."))
	float BaseScale = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, EditCondition="ScaleRampBasis == EPVDistributionBasis::Branch", EditConditionHides, Tooltip="How much relative branch size affects foliage size.\n\n0 = foliage size independent of relative branch size. 1 = large branches get large foliage, shorter branches get small foliage. Useful for realistic species where bigger branches support bigger leaves."))
	float BranchScaleImpact = 0.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Minimum random scale multiplier per instance."))
	float RandomizeScaleMinimum = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(PCG_Overridable, ClampMin=1.0f, ClampMax=2.0f, Tooltip="Maximum random scale multiplier per instance."))
	float RandomizeScaleMaximum = 1.0f;

	UPROPERTY(EditAnywhere, Category="Scale Settings",
		meta=(PCG_Overridable, XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve that varies foliage scale along the plant or branch.\n\nX = position. Y = scale multiplier. Use to make foliage smaller near branch tips or larger near the trunk."))
	FPVFloatRamp ScaleRamp;
};

USTRUCT()
struct FPVDistributionParametricParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Spacing Settings", meta=(ShowOnlyInnerProperties))
	FPVParametricSpacingSettings SpacingSettings;

	UPROPERTY(EditAnywhere, Category="Phyllotaxy Settings", meta=(ShowOnlyInnerProperties))
	FPVParametricPhyllotaxySettings PhyllotaxySettings;

	UPROPERTY(EditAnywhere, Category="Angle Settings", meta=(ShowOnlyInnerProperties))
	FPVParametricAngleSettings AngleSettings;

	UPROPERTY(EditAnywhere, Category="Scale Settings", meta=(ShowOnlyInnerProperties))
	FPVParametricScaleSettings ScaleSettings;
};
