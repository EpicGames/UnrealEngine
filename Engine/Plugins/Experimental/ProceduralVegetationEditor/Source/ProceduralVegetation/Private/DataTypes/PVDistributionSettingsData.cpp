// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVDistributionSettingsData.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoDistributionParametricSettings,   UPVDistributionParametricSettingsData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoDistributionHormoneBasedSettings, UPVDistributionHormoneBasedSettingsData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoDistributionVectorSettings,       UPVDistributionVectorSettingsData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoDistributionConditionSettings,           UPVDistributionConditionSettingsData);
