// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Serialization/ArchiveCrc32.h"
#include "Data/Registry/PCGDataType.h"
#include "DataTypes/PVDistributionParams.h"
#include "PVDistributionSettingsData.generated.h"

#define PV_DISTRIBUTION_SETTINGS_OVERRIDE_CRC(Params)\
virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override\
{\
	Super::AddToCrc(Ar, bFullDataCrc); \
	if (bFullDataCrc)\
	{\
		FString ClassName = StaticClass()->GetPathName();\
		Ar << ClassName;\
		Ar << Params;\
	}\
	else\
	{\
		AddUIDToCrc(Ar);\
	}\
}\

USTRUCT()
struct FPVDataTypeInfoDistributionParametricSettings : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVDistributionParametricSettingsData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVDistributionParametricParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoDistributionParametricSettings)
	PV_DISTRIBUTION_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoDistributionHormoneBasedSettings : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVDistributionHormoneBasedSettingsData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVDistributionHormoneBasedParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoDistributionHormoneBasedSettings)
	PV_DISTRIBUTION_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoDistributionVectorSettings : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVDistributionVectorSettingsData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVDistributionVectorParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoDistributionVectorSettings)
	PV_DISTRIBUTION_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoDistributionConditionSettings : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVDistributionConditionSettingsData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVDistributionConditionParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoDistributionConditionSettings)
	PV_DISTRIBUTION_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};
