// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVGrafterPaletteData.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrafterPalette, UPVGrafterPaletteData);

void UPVGrafterPaletteData::Initialize(TArray<TObjectPtr<UPVGrowthData>>&& InGrowthDataElements)
{
	GrowthDataElements = MoveTemp(InGrowthDataElements);
}

void UPVGrafterPaletteData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);
	
	if (bFullDataCrc)
	{
		FString ClassName = StaticClass()->GetPathName();

		Ar << ClassName;
		
		int32 Num = GrowthDataElements.Num();
		Ar << Num;
		
		for (const auto& Element : GrowthDataElements)
		{
			bool bIsValid = (Element != nullptr);
			Ar << bIsValid;
			
			if (bIsValid)
			{
				Element->AddToCrc(Ar, true);
			}
		}
	}
	else
	{
		AddUIDToCrc(Ar);
	}
}

// UPVData::CopyInternal allocates UPVData and copies only Collection — override to produce the correct type and preserve GrowthDataElements.
UPCGSpatialData* UPVGrafterPaletteData::CopyInternal(FPCGContext* Context) const
{
	UPVGrafterPaletteData* NewData = FPCGContext::NewObject_AnyThread<UPVGrafterPaletteData>(Context);
	TArray<TObjectPtr<UPVGrowthData>> Elements = GrowthDataElements;
	NewData->Initialize(MoveTemp(Elements));
	return NewData;
}
