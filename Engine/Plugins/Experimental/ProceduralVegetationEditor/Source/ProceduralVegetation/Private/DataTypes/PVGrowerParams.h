// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/PVFloatRamp.h"
#include "Nodes/PVObjectInteractionSettings.h"
#include "PVGrowerParams.generated.h"
class UStaticMesh;

USTRUCT()
struct FPVAuxinParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Advanced Auxin Controls", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How quickly auxin concentration drops with distance from the tip.\n\n0 = uniform auxin throughout the branch (very strong suppression everywhere). Higher = auxin only suppresses buds near the tip, with lateral buds further down free to activate. Controls the spatial pattern of apical dominance."))
	float AuxinFalloff = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Advanced Auxin Controls", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Strength of tip-driven lateral bud suppression.\n\nHigh = single dominant leader (pines, firs). Low = freely-branching, bushy structure (shrubs). Bud activation requires `ApicalHormone < (1 - ApicalDominance)`."))
	float ApicalDominance = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Advanced Auxin Controls", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Base level of root-zone auxin.\n\nWhen > 0.05, suppresses axillary sprouting near the base of the plant (mimics the natural suppression of below-ground side shoots). Useful for preventing low branches on tree trunks."))
	float RadicalAuxin = 0.0f;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FPVGrowerGravityParams::MinGravitationalDot instead."))
	float MinGravitationalDot = 0.0f;
};

UENUM()
enum class EPVHitGroundBehaviour : uint8
{
	Kill,
	Deflect,
	Ignore,
};

USTRUCT()
struct FPVGrowerGravityParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Basic", meta=(UIMin=0.0f, UIMax=10.0f, Tooltip="Strength of gravity (m/s²).\n\nDefault 9.8 matches Earth gravity. Higher values exaggerate sag; lower values produce stiffer-looking plants regardless of mass. Useful for stylized art direction."))
	float GravitationalForce = 9.8f;

	UPROPERTY(EditAnywhere , Category = "Basic" , meta=(UIMin=0.0f, UIMax=2.0f, ClampMin=0.0f, Tooltip="Weight per unit of wood volume.\n\nHigher = heavier branches that sag more under their own weight. Pair with `CellDensity` for realistic species — dense hardwoods are both heavier and stiffer; balsa is light and flexible."))
	float CellWeight = 0.05f;

	UPROPERTY(EditAnywhere , Category = "Basic", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, Tooltip="Weight added per leaf instance.\n\nLeaves often dominate gravity stress on young branches. Higher values make leaf-heavy branches droop more (catalpa, banana plant)."))
	float FoliageWeight = 0.05f;

	UPROPERTY(EditAnywhere , Category = "Basic", meta=(DisplayName = "Branch Break Tension Threshold", UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How much bend a branch tolerates before snapping.\n\n0 = any bend breaks the branch (extreme). 1 = branches tolerate 180° bends without breaking. 0.3-0.4 produces realistic snap-off behavior for over-stressed limbs."))
	float BranchWeightBreakThreshold = 0.33f;

	UPROPERTY(EditAnywhere , Category = "Basic", meta=(Tooltip="What happens when a drooping branch touches the ground.\n\nKill = remove the branch. Deflect = redirect along the ground. Ignore = let it pass through. Use Kill for clean pruning, Deflect for creeping vines, Ignore for purely artistic ground-following."))
	EPVHitGroundBehaviour HitGroundBehaviour = EPVHitGroundBehaviour::Kill;

	UPROPERTY(EditAnywhere , Category = "Basic", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Determines how much a branch needs to face upwards upon birth to be spawned."))
	float MinGravitationalDot = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Reinforcement", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Extra stiffness applied to the main trunk.\n\nOverride for the physics: higher values keep the trunk vertical regardless of mass calculations. Useful when the simulated bend is unrealistic — e.g. for stout, mature trees that physics would otherwise show as bending excessively."))
	float TrunkReinforcement = 0.5f;

	UPROPERTY(EditAnywhere , Category = "Reinforcement", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Extra stiffness applied to inherited input skeletons.\n\nWhen chaining Growers (output of one feeds another), this stiffens the input skeleton so the chained growth doesn't deform what came before. Set near 1 to keep the input skeleton rigid; near 0 to let it deform naturally."))
	float InputReinforcement = 0.5f;

	UPROPERTY()
	bool bCustom =	true;

	UPROPERTY(EditAnywhere , Category = "Cell Density", meta=(Tooltip="Wood stiffness range, young (min) to mature (max).\n\nValues are Young's modulus in MPa. The simulation lerps from min (flexible young wood, default 7000 MPa) to max (stiff mature wood, default 10000 MPa) over `CellDevelopmentTime` cycles. Real wood E-modulus values: balsa ~3000, pine ~9000, oak ~11000, ironwood ~16000."))
	FFloatRange CellDensity = FFloatRange(7000, 10000);

	UPROPERTY(EditAnywhere , Category = "Cell Density", meta=(UIMin=1.0f, UIMax=100.0f, ClampMin=1.0f, Tooltip="Cycles until wood reaches mature stiffness.\n\nHow many growth cycles a branch needs to age from young (CellDensity.Min) to mature (CellDensity.Max). Default 5 means a branch born this cycle is fully mature 5 cycles later. Higher values keep branches flexible longer, increasing gravity-driven bending."))
	float CellDevelopmentTime = 5.0f;
};

USTRUCT()
struct FPVFoliageParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Basic Settings" , meta=(UIMin=0.001f, UIMax=5.0f, ClampMin=0.001f, Tooltip="Initial leaf scale when a leaf spawns.\n\nScale multiplier for a newly-spawned leaf, before ethylene-driven growth. Smaller = young leaves grow visibly over time. Combine with `EndScale` and `DevelopmentTime` for the growth curve."))
	float StartScale = 0.2f;

	UPROPERTY(EditAnywhere, Category = "Basic Settings" , meta=(UIMin=0.001f, UIMax=5.0f, ClampMin=0.001f, Tooltip="Final leaf scale once fully grown.\n\nScale a leaf reaches once it has accumulated enough ethylene to be fully mature. Functions as the upper bounds for how large leaf can get during its life span."))
	float EndScale = 0.8f;

	UPROPERTY(EditAnywhere, Category = "Basic Settings", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Fraction of total cycles for a leaf to reach full scale.\n\n0 = leaves spawn at `EndScale` immediately. 1 = leaves take all `GrowthCycles` to reach `EndScale`. Useful when modeling young saplings (low value) or simulating short summer-only growth (high value)."))
	float DevelopmentTime = 0.1f;

	UPROPERTY(EditAnywhere, Category = "Basic Settings" , meta=(UIMin=0, UIMax=5, ClampMin=0, ClampMax=10, Tooltip="Leaf density multiplier.\n\nHigher = more leaves per branch point. Affects gravity load (heavier canopy = more sag) and light occlusion (denser canopy = more shade on lower branches)."))
	int Density = 3;

	UPROPERTY(EditAnywhere, Category = "Hormone Control", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How fast ethylene accumulates on each leaf per cycle.\n\nHigher = leaves drop sooner. Lower = leaves persist longer."))
	float EthyleneBuildup = 0.15f;

	UPROPERTY(EditAnywhere, Category = "Hormone Control" , meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Ethylene level at which leaves drop.\n\nWhen per-leaf ethylene exceeds this value, the leaf is shed. Combine with `EthyleneBuildup` for the leaf-life cycle: high threshold + low buildup = long-lived leaves; vice versa for short-lived."))
	float EthyleneThreshold = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Hormone Control" , meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Stress-driven random leaf shedding.\n\nIndependent of ethylene. Each cycle, this fraction of leaves is randomly removed regardless of phyllotaxy. Use to simulate stressed plants (drought, salt) or to artistically thin the canopy."))
	float AbscisicAcid = 0;

	UPROPERTY(EditAnywhere, Category = "Hormone Control" , meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="High amounts of auxin counteracts the effect of ethylene-driven shedding.\n\nHigher = leaves cling longer despite ethylene buildup. Useful for plants where leaves persist through stress."))
	float AuxinRetention = 0.0;
};

static int PhyllotaxyTypeAngles[] = {180,0,90,90,0};

UENUM()
enum class EPVGrowthPhyllotaxyType : uint8
{
	Alternate,
	Opposite,
	Decussate,
	Whorled,
	Spiral,
};

UENUM()
enum class EPVGrowthPhyllotaxyFormation : uint8
{
	Disticious = 180,
	Tristicious = 120,
	Pentasticious = 144,
	Octasticious = 135,
	Parasticious = 0,
};

USTRUCT()
struct FPVGrowerSettingsTargetInfo
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category = "Target")
	bool bTrunk = true;

	UPROPERTY(EditAnywhere , Category = "Target")
	bool bBranch = true;

	FString ToString() const
	{
		auto AddTargetInString = ([](FString& Targets, const FString& CurrentTarget)
			{
				if (CurrentTarget.IsEmpty())
					return;

				if (!Targets.IsEmpty())
				{
					Targets += ", ";
				}

				Targets += CurrentTarget;
			});

		FString Targets;

		if (bTrunk)
		{
			AddTargetInString(Targets, TEXT("Trunk"));
		}
		if (bBranch)
		{
			AddTargetInString(Targets, TEXT("Branch"));
		}

		if (Targets.IsEmpty())
		{
			Targets = TEXT("None");
		}

		return FString::Printf(TEXT("Target(s): %s"), *Targets);
	}
};

USTRUCT()
struct FPVGrowerPhyllotaxyTargetInfo : public FPVGrowerSettingsTargetInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Target", meta = (DisplayAfter = "bBranch"))
	bool bFoliage = false;

	FString ToString() const
	{
		auto AddTargetInString = ([](FString& TargetsStr, const FString& CurrentTarget)
		{
			if (CurrentTarget.IsEmpty()) return;
			if (!TargetsStr.IsEmpty()) TargetsStr += ", ";
			TargetsStr += CurrentTarget;
		});

		FString TargetsStr;
		if (bTrunk)   AddTargetInString(TargetsStr, TEXT("Trunk"));
		if (bBranch)  AddTargetInString(TargetsStr, TEXT("Branch"));
		if (bFoliage) AddTargetInString(TargetsStr, TEXT("Foliage"));
		if (TargetsStr.IsEmpty()) TargetsStr = TEXT("None");
		return FString::Printf(TEXT("Target(s): %s"), *TargetsStr);
	}
};

USTRUCT()
struct FPVGrowerPhyllotaxyParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Axial Angle", meta=(UIMin=0.0f, UIMax=90.0f, ClampMin=-90.0f, ClampMax=90.0f, Tooltip="Angle in degrees between child branches and the parent direction.\n\n0° = child branch points along the parent axis (rare). 90° = child sticks straight out (perpendicular). Negative values up to -90° (typed manually) make the child point backward. Larger angles widen the canopy; smaller angles produce columnar shapes."))
	float AxilAngle = 35.0f;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta=(Tooltip="Pattern for arranging buds around the stem.\n\nAlternate (180° flip per node), Opposite (paired buds), Decussate (paired buds rotating 90°), Whorled (2+ buds per node), or Spiral (continuous rotation using the Formation sub-type)."))
	EPVGrowthPhyllotaxyType Type = EPVGrowthPhyllotaxyType::Spiral;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta = (EditCondition = "Type == EPVGrowthPhyllotaxyType::Spiral", EditConditionHides, Tooltip="Sub-angle for spiral phyllotaxy.\n\nRepresents the number of branches that occurs in a 360 degree rotation. Distichous (180°), Tristichous (120°), Pentastichous (144°), Octastichous (135°, the default — closest to Fibonacci), or Parastichous (0° — free-form, requires manual Additional Angle / Offset)."))
	EPVGrowthPhyllotaxyFormation Formation = EPVGrowthPhyllotaxyFormation::Octasticious;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy" , meta = (EditCondition = "Type == EPVGrowthPhyllotaxyType::Whorled", EditConditionHides, Tooltip="Minimum number of buds per node for Whorled phyllotaxy.\n\nEach whorl spawns between `Min` and `Max` branches, picked at random. Set `Min == Max` for predictable counts. Typical values: 3-5 for a open whorl, 6-8 for an tight one."))
	uint32 Min = 3;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta = (EditCondition = "Type == EPVGrowthPhyllotaxyType::Whorled", EditConditionHides, Tooltip="Maximum number of buds per node for Whorled phyllotaxy.\n\nUpper bound on whorl branch count. See `Min` above."))
	uint32 Max = 6;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta=(UIMin=0.0f, UIMax=360.0f, Tooltip="Extra rotation added to each node's phyllotaxy angle.\n\nAdds to the base angle picked by `Type`/`Formation`. E.g., Octastichous base = 135°; with AdditionalAngle = 10°, effective per-node rotation = 145°. Used to break perfect regularity or to author Parastichous patterns (where base = 0°, so this drives the entire rotation)."))
	float AdditionalAngle = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta=(UIMin=0.0f, UIMax=360.0f, Tooltip="Initial rotation of the phyllotaxy pattern.\n\nPhase-shifts the whole spiral. Doesn't change per-node spacing, just where the first bud starts. Useful for an overall offset without other changes to the phyllotaxy."))
	float Offset = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Behavior", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Flatten the 3D phyllotaxy onto a 2D plane.\n\nAt 0, the phyllotaxy spirals fully in 3D. At 1, branches flattens out relative to the branches up direction. Useful for plants that have a spiral phyllotaxy but appears to flatten out."))
	float Flatten = 0.0f;

	UPROPERTY()
	int32 ShowStagger = 1;

	UPROPERTY(EditAnywhere , Category = "Behavior", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, EditCondition="ShowStagger > 0", EditConditionHides, Tooltip="Jitter the apical bud opposite to activating axillary buds.\n\nAdds asymmetry to the growth direction when a side branch activates — the apical tip slightly bends away from the new side branch. Higher values create more wandering, irregular branch paths."))
	float Stagger = 0.0f;

	UPROPERTY()
	int32 ShowReset = 1;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta=(EditCondition="ShowReset > 0", EditConditionHides, Tooltip="Reset phyllotaxy at each branch boundary.\n\nWhen enabled, each new branch starts its phyllotaxy fresh relative to the branch up direction. When disabled, the rotation is successively inherited from its previous generation and resumes its rotation from that point."))
	bool bReset = false;
};

USTRUCT()
struct FPVGrowerBifurcationParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category = "Basic", meta=(Tooltip="Enable tip splitting (bifurcation).\n\nWhen enabled, branch tips can split into multiple branches when cytokinin builds up. Common in birch, spruce and most common fruit trees. When disabled, growth is purely apical-axillary."))
	bool bEnableBifurcation = false;

	UPROPERTY(EditAnywhere , Category = "Basic", meta = (EditCondition = "bEnableBifurcation == true", UIMin=0.0f, UIMax=90.0f, ClampMin=-90.0f, ClampMax=90.0f, Tooltip="Angle between bifurcated branches in degrees.\n\nWhen a tip splits, new branches are spaced this many degrees apart. 60° = wide V-split; 180° = horizontal split; 30° = narrow split. Combine with `SplitBias` to asymmetrize."))
	float SplitAngle = 60.0f;

	UPROPERTY(EditAnywhere , Category = "Basic", meta = (EditCondition = "bEnableBifurcation == true", UIMin=-0.5f, UIMax=0.5f, ClampMin=-0.5f, ClampMax=0.5f, Tooltip="Asymmetry between bifurcated branches.\n\n0 = even split (both branches get same angle). 1 = one branch keeps the original axis, the other gets the full `SplitAngle`. Values 0.3-0.7 produce natural-looking unequal splits."))
	float SplitBias = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Basic", meta = (EditCondition = "bEnableBifurcation == true", UIMin=2, UIMax=10, ClampMin=2, Tooltip="Minimum number of new branches at each split.\n\nMost species split into 2 (dichotomous); set Min=Max=3 for trichotomous splits common in some palms. Random pick between Min and Max per split event."))
	uint32 SplitMin = 2;

	UPROPERTY(EditAnywhere , Category = "Basic", meta = (EditCondition = "bEnableBifurcation == true", UIMin=2, UIMax=10, ClampMin=2, Tooltip="Maximum number of new branches at each split.\n\nMost species split into 2 (dichotomous); set Min=Max=3 for trichotomous splits common in some palms. Random pick between Min and Max per split event."))
	uint32 SplitMax = 2;

	UPROPERTY(EditAnywhere , Category = "Hormone Controls", meta = (EditCondition = "bEnableBifurcation == true", UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Cytokinin level required to trigger a split.\n\nWhen a plant is unable so support its own weight it starts producing cytokinin. When a bud's cytokinin exceeds this threshold, it's eligible to bifurcate. 1.0 = saturation required (rare splits). 0.5 = splits happen more often. Combine with `CytokininBuildup` to tune frequency."))
	float SplitThreshold = 0.33f;

	UPROPERTY(EditAnywhere , Category = "Hormone Controls", meta = (EditCondition = "bEnableBifurcation == true", UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How fast cytokinin accumulates per cycle.\n\nHigher = splits happen sooner. Combine with `SplitThreshold` to control overall split frequency."))
	float CytokininBuildup = 0.33f;

	UPROPERTY(EditAnywhere , Category = "Hormone Controls", meta = (EditCondition = "bEnableBifurcation == true", UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Random variation added to cytokinin buildup each cycle.\n\nHigher = less predictable split timing across branches. 0 = perfectly synchronized splits."))
	float CytokininRandomness = 0.33f;
};

USTRUCT()
struct FPVPhototropismParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Phototropic Settings", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How strongly growing tips bend toward light.\n\n0 = no light-seeking (tips grow straight). 1 = tips fully follow optimal light direction. Values 0-0.5 are typical for natural plants — higher overrides phyllotaxy and can produce unnatural results."))
	float Apical = 0.25;

	UPROPERTY(EditAnywhere , Category = "Phototropic Settings", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Bias between light-seeking and shadow-avoidance for tips.\n\n0 = bend toward the brightest direction. 1 = bend away from the darkest direction. The two are subtly different — shadow-avoidance tends to spread branches away from neighbors (canopy filling), while light-seeking tends to bend toward the sun."))
	float ApicalBias = 0.1f;

	UPROPERTY(EditAnywhere , Category = "Phototropic Settings", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How strongly new side branches choose a light-seeking direction at birth.\n\nOnly affects the first segment of a new side branch. Useful when you want branches to immediately tilt toward light at activation. Higher values can override phyllotaxy aggressively."))
	float Axillary = 0.1f;

	UPROPERTY(EditAnywhere , Category = "Phototropic Settings", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Bias between light-seeking and shadow-avoidance for new side branches.\n\nSame idea as `ApicalBias` but for axillary phototropism."))
	float AxillaryBias = 0.75f;

	UPROPERTY(EditAnywhere , Category = "Photosynthetic Impact", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Minimum light required for a bud to activate.\n\n0 = buds can activate regardless of light. 1 = full light required (extreme — most buds will fail). Combine with foliage occlusion to create realistic shade-sensitive plants. Distinct from light senescence: this gates initial sprouting, senescence gates branch survival."))
	float LightRequirement = 0.0f;
};

USTRUCT()
struct FPVDirectionalParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Random", meta=(UIMin=0.0f, UIMax=45.0f, Tooltip="Random angle added to each apical growth step.\n\nPer-step random tilt of the growing tip in degrees. 0 = perfectly straight. 5-15 = natural wobble. Higher values produce wandering, irregular branch paths."))
	float ApicalRandomAngle = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Random", meta=(UIMin=0.0f, UIMax=45.0f, Tooltip="Random angle added to each new side branch's first segment.\n\nPer-branch initial direction jitter. Useful to break the regularity of phyllotaxy without changing the pattern itself."))
	float AxillaryRandomAngle = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Random", meta=(UIMin=0.0f, UIMax=45.0f, Tooltip="Random angle added to bifurcated branches.\n\nOnly effective when bifurcation is enabled. Adds variation to the split angles for natural-looking splits."))
	float CodominentRandomAngle = 0.0f;
};

UENUM()
enum class EPVRampBasis : uint8 { PlantTargetLength, BranchTargetLength };

USTRUCT()
struct FPVTrunkGrowthParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Basic Global", meta=(UIMin=1, ClampMin=1, Tooltip="Maximum branching depth.\n\n1 = trunk only. 5 = trunk + 4 levels of side branches. Higher values allow finer detail but increase generation time. Most trees look complete with 8-15."))
	uint8 MaxGeneration = 25;

	UPROPERTY(EditAnywhere , Category = "Basic Global", meta=(UIMin=0.0005f, UIMax=0.01f, ClampMin=0.0001f, Tooltip="How much each branch thickens per cycle (in meters).\n\nSimulates secondary wood growth (girth increase). Default 0.0033 m/cycle = 3.3 mm/year. Thicker branches are stiffer (resist gravity better) and immune to senescence above `RetentionRadius`."))
	float IncrementalRadius = 0.0033f;

	UPROPERTY(EditAnywhere , Category = "Basic Global", meta=(UIMin=0.2f, UIMax=15.0f, ClampMin=0.001f, Tooltip="Target height for the main trunk in meters.\n\nThe trunk's priority gradient is normalized to this distance. After reaching this height, the apical priority typically drops to near-zero (determinate growth) or stays positive (indeterminate growth) depending on the gradient curve."))
	float PlantTargetLength = 5.0f;

	UPROPERTY(EditAnywhere , Category = "Basic Global", meta=(Tooltip="Target length for side branches; 0 = unlimited.\n\nSame as `PlantTargetLength` but applied to branches. 0 = branches grow until resources run out. Set a finite value to create uniform side-branch lengths."))
	float BranchTargetLength = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Internode Length" , meta=(UIMin=0.0f, UIMax=1.0f, Tooltip="Length of each internode in meters.\n\nDistance between consecutive growth points along a branch. Smaller = denser, compact plants. Larger = open, airy structure. This is the core control for plant scale relative to `GrowthCycles`. Default 0.25 m = 25 cm per growth step."))
	float SegmentLength = 0.25;

	UPROPERTY(EditAnywhere , Category = "Internode Length" , meta=(UIMin=0.1f, UIMax=2.0f, ClampMin=0.001f, Tooltip="Multiplier on SegmentLength for side branches.\n\nBelow 1 = side branches grow shorter steps than the trunk (tree-like tapering). Above 1 = side branches outgrow the trunk (spreading, bush-like). 1.0 = uniform across trunk and branches."))
    float BranchScale = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Internode Length", meta=(UIMin=-1.0f, UIMax=1.0f, ClampMin=-1.0f, ClampMax=1.0f, Tooltip="How light level affects internode length.\n\nPositive: branches in bright spots grow longer segments (sun-loving species). Negative: shaded branches stretch longer toward light (etiolation — climbing vines, shade-stressed saplings)."))
	float LengthLightImpact = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Internode Length", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Random variation in light-driven length response.\n\nHigher values produce less uniform light response, making the plant look more natural and less computed."))
	float LightRandomness = 0.0f;

	UPROPERTY(EditAnywhere , Category = "Internode Length", meta=(UIMin=0.0f, UIMax=1.0f, Tooltip="Minimum guaranteed fraction of SegmentLength per step.\n\nEven when other factors (light, hormones, randomness) would shorten a step, this bias ensures at least this fraction is added. Prevents branches from stopping entirely. Higher = more consistent growth despite stress."))
	float LengthBias = 0.0f;

	UPROPERTY(/*meta=(UIMin=0.0f, UIMax=1.0f)*/)
	float AuxinImpact = 0.0f;

	UPROPERTY()
	float LengthSeedScaleImpact = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Axillary Resources", meta=(UIMin=0.0f, UIMax=2.0f, ClampMin=0.0f, Tooltip="Axillary priority multiplier for the trunk (Generation 1).\n\nScales the axillary priority gradient's output for the trunk. Higher = more side branches on the trunk. Lower = sparser trunk branching."))
	float AxillaryParentGrowth = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Axillary Resources", meta=(UIMin=0.0f, UIMax=2.0f, ClampMin=0.0f, Tooltip="Axillary priority multiplier for side branches (Generation 2+).\n\nSame as `AxillaryParentGrowth` but applied to side branches. Use to control how dense the inner foliage is independent of trunk branch density."))
	float AxillaryChildGrowth = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Axillary Resources", meta=(Tooltip="Allow failed axillary buds to retry in later cycles.\n\nWhen on, buds that failed to activate this cycle (due to light, randomness, or apical dominance) are still eligible to activate next cycle. When disabled, a failed bud is permanently dormant. Enabled creates more deterministic plants; on is more natural."))
	bool bAxillaryRetry = true;

	UPROPERTY(EditAnywhere , Category = "Axillary Resources", meta=(Tooltip="Use a separate axillary priority ramp for side branches.\n\nWhen enabled, side branches use `AxillaryPriorityChildGradient` instead of inheriting the trunk's `AxillaryPriorityGradient`. Useful for plants where branching density differs along trunk vs. side branches."))
	bool bAxillaryUseChildGradient = false;

	UPROPERTY(EditAnywhere , Category = "Axillary Resources" , DisplayName = "Ramp Basis", meta=(Tooltip="Length used to normalize the axillary priority ramp.\n\n`PlantTargetLength`: ramp spans the whole plant terminal length. `BranchTargetLength`: ramp spans each branch's individual length. Choose based on whether side-branch density should depend on plant-wide height or per-branch distance."))
	EPVRampBasis AxillaryRampBasis = EPVRampBasis::PlantTargetLength;

	UPROPERTY(EditAnywhere, Category= "Axillary Resources", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve controlling axillary bud activation likelihood vs. position.\n\nX axis = normalized position along trunk (or branch, per ramp basis). Y axis = priority threshold. Buds whose random roll falls below this curve activate. Set Y to 0 at the bottom to prevent branches in the lower trunk; set Y high at the top for denser canopy branching."))
	FPVFloatRamp AxillaryPriorityGradient;

	UPROPERTY(EditAnywhere , Category = "Axillary Resources" , DisplayName = "Child Ramp Basis", meta=(EditCondition = "bAxillaryUseChildGradient == true", EditConditionHides))
	EPVRampBasis AxillaryChildRampBasis = EPVRampBasis::PlantTargetLength;

	UPROPERTY(EditAnywhere, Category= "Axillary Resources", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, EditCondition = "bAxillaryUseChildGradient == true", EditConditionHides, Tooltip="Separate axillary priority ramp for side branches.\n\nUse independently from `AxillaryPriorityGradient` to give side branches different density behavior."))
	FPVFloatRamp AxillaryPriorityChildGradient;

	UPROPERTY(EditAnywhere , Category = "Apical Resources", meta=(UIMin=0.0f, UIMax=2.0f, ClampMin=0.0f, Tooltip="Apical priority multiplier for the trunk.\n\nHigher = the trunk's apical tip extends more reliably each cycle. Lower = the trunk slows or stops earlier (more determinate growth)."))
	float ApicalParentGrowth = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Apical Resources", meta=(UIMin=0.0f, UIMax=2.0f, ClampMin=0.0f, Tooltip="Apical priority multiplier for side branches.\n\nHigher = side branches keep extending longer. Lower = side branches stop after fewer cycles, producing short stubs."))
	float ApicalChildGrowth = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Apical Resources" , DisplayName = "Ramp Basis", meta=(Tooltip="Length used to normalize the apical priority ramp.\n\nSame idea as `AxillaryRampBasis`. Default is `BranchTargetLength` — branches use their own target. Switch to `PlantTargetLength` for plant-wide apical scaling."))
	EPVRampBasis ApicalRampBasis = EPVRampBasis::PlantTargetLength;

	UPROPERTY(EditAnywhere, Category= "Apical Resources", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip="Curve controlling apical tip extension likelihood vs. position.\n\nX axis = normalized position. Y axis = priority threshold. Apical activation requires a random roll below Y. Set Y at the high end to 0 for determinate plants (trunk stops at PlantTargetLength). Keep Y > 0 throughout for indeterminate (continuously growing) plants."))
	FPVFloatRamp ApicalPriorityGradient;

	UPROPERTY(EditAnywhere , Category = "Apical Resources", meta=(Tooltip="Use a separate apical priority ramp for side branches.\n\nWhen on, side branches use `ApicalPriorityChildGradient` instead of `ApicalPriorityGradient`. Useful for letting side branches keep growing past the trunk's stop point."))
	bool bApicalUseChildGradient = false;

	UPROPERTY(EditAnywhere , Category = "Apical Resources" , DisplayName = "Child Ramp Basis", meta=(EditCondition = "bApicalUseChildGradient == true", EditConditionHides))
	EPVRampBasis ApicalChildRampBasis = EPVRampBasis::PlantTargetLength;

	UPROPERTY(EditAnywhere, Category= "Apical Resources", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, EditCondition = "bApicalUseChildGradient == true", EditConditionHides, Tooltip="Separate apical priority ramp for side branches."))
	FPVFloatRamp ApicalPriorityChildGradient;

	UPROPERTY(EditAnywhere , Category = "Advanced Globals", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How much seed-point scale influences final plant size.\n\nAt 1.0, the seed point's scale attribute directly multiplies the plant's size. At 0.0, seed scale is ignored. Useful when scattering seed points of varying sizes from PCG and you want either uniform plants (0) or size-variable plants (1)."))
	float SeedScaleEffect = 1.0f;

	UPROPERTY(EditAnywhere , Category = "Advanced Globals", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="How much each branch in a Whorled phyllotaxy reduces the parent's effective radius.\n\nOnly relevant when Whorled phyllotaxy spawns multiple side branches at one node. Higher = more radius reduction per branch (mimics the natural taper of a heavily-branched node). 0 = no reduction."))
	float WhorledRadiusImpact = 0.5f;

	UPROPERTY(EditAnywhere , Category = "Advanced Globals", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Flattens the crown by letting lower branches grow longer.\n\nHigher values allow lower lateral branches to elongate beyond their parents' length, producing a corymb (flat-topped) crown shape — common in hawthorn or broccoli."))
	float Corymb = 0.0f;

	FPVTrunkGrowthParams()
	{
		ApicalPriorityGradient.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
		ApicalPriorityChildGradient.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
		AxillaryPriorityGradient.InitializeLinearCurve();
		AxillaryPriorityChildGradient.InitializeLinearCurve();
	}
};

USTRUCT()
struct FPVGuideParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "GuideSettings")
	float GuidFollowStrength = 0.0;

	UPROPERTY(EditAnywhere , Category = "GuideSettings")
	float GuideSearchDistance = 0.0;

	UPROPERTY(EditAnywhere , Category = "GuideSettings")
	float GuideGravityOverride = 0.0;

	UPROPERTY(EditAnywhere , Category = "GuideSettings")
	bool bUse = false;
};

UENUM()
enum class EPVAbscissionMode : uint8
{
	Kill,
	Degrade
};

USTRUCT()
struct FPVAgeSenescenceParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision" ,  meta=(Tooltip = "Reset branch age when resuming growth from an input skeleton.\n\nWhen chaining Growers, this controls whether inherited branches start aging fresh (true) or continue from where they left off (false). Use false for true 'resumed growth'; use true when chaining for stylistic effect without aging continuity."))
	bool bResetOnResumeGrowth = true;
	
	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision", meta=(Tooltip="How aged branches are removed: Degrade (gradual) or Kill (instant).\n\nDegrade = the branch's brittleness increases each cycle until a gravity break occurs (natural-looking). Kill = the branch is fully removed after the senescence delay (clean and predictable)."))
	EPVAbscissionMode Mode = EPVAbscissionMode::Degrade;

	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision" , meta=(UIMin=1, UIMax=50, Tooltip="Cycle age after which branches become vulnerable to age-abscission.\n\nBranches younger than this are immune. After this age, they enter the senescent state and are subject to `DegradationAmount` (Degrade mode) or `SenescenceMin/Max` countdown (Kill mode)."))
	int SenescenceAge = 20;

	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision" , meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, EditCondition = "Mode == EPVAbscissionMode::Degrade", Tooltip="How much a senescent branch weakens each cycle.\n\nEach cycle past `SenescenceAge`, the branch's brittleness increases by this amount. Eventually the branch becomes more susceptible to break under gravity. Higher = faster weakening. 0 = no degradation (branch lingers indefinitely)."))
	float DegradationAmount = 0;

	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision" , meta=(UIMin=0, EditCondition = "Mode == EPVAbscissionMode::Kill", Tooltip="Minimum random range of cycles a senescent branch survives before removal.\n\nAfter `SenescenceAge`, a random duration between Min and Max is rolled per branch. The branch survives that many additional cycles, then is removed."))
	int SenescenceMin = 1;

	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision" , meta=(UIMin=0, EditCondition = "Mode == EPVAbscissionMode::Kill", Tooltip="Maximum random range of cycles a senescent branch survives before removal.\n\nAfter `SenescenceAge`, a random duration between Min and Max is rolled per branch. The branch survives that many additional cycles, then is removed."))
	int SenescenceMax = 5;

	UPROPERTY(EditAnywhere , Category = "Gravity Senescence/Abscision" , meta=(UIMin=0.0f, UIMax=1.0f, Tooltip="Fraction of expected radius that immunizes a branch from age-death.\n\nSame concept as Light Senescence's RetentionRadius. Branches thicker than this fraction × expected radius survive past their senescence age."))
	float RetentionRadius = 0;
};

USTRUCT()
struct FPVLightSenescenceParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Light Senescence/Abscission", meta=(UIMin=0.0f, UIMax=1.0f, ClampMin=0.0f, ClampMax=1.0f, Tooltip="Light level below which branches enter shade-death.\n\nWhen a branch's measured light drops below this value for enough cycles, it begins dying back. 0 = never (all branches survive any shade). 1 = full sun required (extreme — most branches will die). 0.3-0.5 produces realistic shade pruning."))
	float SenescenceThreshold = 0.33f;

	UPROPERTY(EditAnywhere , Category = "Light Senescence/Abscission" , meta=(UIMin=0, Tooltip="Minimum random range of cycles a shaded branch survives before dying.\n\nOnce a branch's light drops below `SenescenceThreshold`, a random duration between Min and Max is rolled. The branch survives that many cycles in shade, then is removed (in Kill mode) or has fully degraded (in Degrade mode)."))
	int SenescenceMin = 1;

	UPROPERTY(EditAnywhere , Category = "Light Senescence/Abscission" , meta=(UIMin=0, Tooltip="Maximum random range of cycles a shaded branch survives before dying.\n\nOnce a branch's light drops below `SenescenceThreshold`, a random duration between Min and Max is rolled. The branch survives that many cycles in shade, then is removed (in Kill mode) or has fully degraded (in Degrade mode)."))
	int SenescenceMax = 5;

	UPROPERTY(EditAnywhere , Category = "Light Senescence/Abscission" , meta=(UIMin=0.0f, UIMax=1.0f, Tooltip="Fraction of expected radius that immunizes a branch from shade-death.\n\nBranches thicker than this fraction × the expected radius for their age survive even when light senescence would kill them. Allows old, thick branches to persist in deep shade. 0 = no immunity; 0.5 = branches at half expected radius survive; 1.0 = only fully-mature branches survive."))
	float RetentionRadius = 0;
};

USTRUCT()
struct FPVGrowerAuxinWithTargets
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category = "Targets", meta = (ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVGrowerSettingsTargetInfo Targets;

	UPROPERTY(EditAnywhere , Category = "Branching", meta = (ShowOnlyInnerProperties))
	FPVAuxinParams Params;
};

USTRUCT()
struct FPVGrowerPhototropismWithTargets
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category = "Targets", meta = (ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVGrowerSettingsTargetInfo Targets;

	UPROPERTY(EditAnywhere , Category = "Phototropism", meta = (ShowOnlyInnerProperties))
	FPVPhototropismParams Params;
};

USTRUCT()
struct FPVGrowerDirectionalWithTargets
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category = "Targets", meta = (ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVGrowerSettingsTargetInfo Targets;

	UPROPERTY(EditAnywhere , Category = "RandomAngle", meta = (ShowOnlyInnerProperties))
	FPVDirectionalParams Params;
};

USTRUCT()
struct FPVGrowerPhyllotaxyWithTargets
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere , Category = "Targets", meta = (ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVGrowerPhyllotaxyTargetInfo Targets;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta = (ShowOnlyInnerProperties))
	FPVGrowerPhyllotaxyParams Params;
};

USTRUCT()
struct FPVGrowerBifurcationWithTargets
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere , Category = "Targets", meta = (ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVGrowerSettingsTargetInfo Targets;

	UPROPERTY(EditAnywhere , Category = "Bifurcation", meta = (ShowOnlyInnerProperties))
	FPVGrowerBifurcationParams Params;
};

UENUM()
enum class EPVPresetTarget : uint8
{
	Trunk,
	Branch
};

/**
 * Pre-calculated geometry for a single leaf mesh.
 * Built once when LeafMesh is selected in the editor; vertices/indices are
 * reused every growth cycle without re-extracting from the render data.
 */
struct FPVLeafMeshGeometry
{
	TArray<FVector3f> Vertices;  // object-space vertex positions (cm)
	TArray<uint32>    Indices;   // triangle index buffer
	FVector3f         ObjMin = FVector3f(0.f);  // object-space AABB min
	FVector3f         ObjMax = FVector3f(0.f);  // object-space AABB max

	bool IsValid() const { return Vertices.Num() > 0 && Indices.Num() > 0; }
	void Reset()         { Vertices.Empty(); Indices.Empty(); ObjMin = ObjMax = FVector3f(0.f); }
};

/** A single leaf placement: world-space transform and the branch it belongs to. */
struct FPVLeafTransform
{
	FTransform Transform;
	int32      BranchNumber = 0;
};

USTRUCT()
struct FPVGrowerParams
{
	GENERATED_BODY()

	FPVGrowerParams()
	{
		Phyllotaxy.ShowReset = 0;
		LeafPhyllotaxy.ShowStagger = 0;
	}

public:
	UPROPERTY(EditAnywhere , Category = "Basic" , meta=(UIMin=1.0f, UIMax=45.0f, PCG_Overridable, Tooltip="How many growing iterations to simulate.\n\nEach cycle can be considered one growing season, but certain plants can have multiple cycles each season. Higher counts produce larger, more complex plants with more visible aging effects. Start with 5-10 for typical trees; use 20-45 for very mature plants. Note that simulation cost grows roughly linearly with cycle count."))
	uint32 GrowthCycles = 1;

	UPROPERTY(EditAnywhere , Category = "Basic", meta =(PCG_NotOverridable, Tooltip="Random seed for all stochastic choices.\n\nControls every random decision in the simulation: which buds activate, axillary angle jitter, bifurcation timing, etc. Same seed + same settings = identical plant. Change to explore natural variations of the same configuration."))
	uint32 RandomSeed = 7023;
	
	UPROPERTY(EditAnywhere , Category = "Phyllotaxy | Trunk", meta =(PCG_NotOverridable))
	FPVGrowerPhyllotaxyParams Phyllotaxy;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta =(PCG_NotOverridable, Tooltip="Use the trunk's phyllotaxy for side branches.\n\nWhen on, side branches inherit the trunk's phyllotaxy settings. Turn off to configure side-branch phyllotaxy independently — useful for species like spruce (whorled trunk, opposite branches)."))
	bool bBranchPhyllotaxySameAsTrunk = true;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy | Branch" , meta = (EditCondition = "bBranchPhyllotaxySameAsTrunk == false" , EditConditionHides, PCG_NotOverridable))
	FPVGrowerPhyllotaxyParams BranchPhyllotaxy;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy", meta =(PCG_NotOverridable, Tooltip="Use the branch phyllotaxy for leaf placement.\n\nWhen on, leaves use the same arrangement pattern as side branches. Turn off to give leaves their own phyllotaxy — useful when you want branches in a whorled pattern but leaves in a spiral, for example."))
	bool bLeafPhyllotaxySameAsBranch = true;

	UPROPERTY(EditAnywhere , Category = "Phyllotaxy | Leaf" , meta = (EditCondition = "bLeafPhyllotaxySameAsBranch == false" , EditConditionHides, PCG_NotOverridable))
	FPVGrowerPhyllotaxyParams LeafPhyllotaxy;

	UPROPERTY(EditAnywhere , Category = "Growth", meta =(PCG_NotOverridable))
	FPVTrunkGrowthParams TrunkGrowth;

	UPROPERTY(EditAnywhere , Category = "Phototropism | Trunk", meta =(PCG_NotOverridable))
	FPVPhototropismParams Phototropism;

	UPROPERTY(EditAnywhere , Category = "Phototropism", meta =(PCG_NotOverridable, Tooltip="Use the trunk's phototropism for side branches.\n\nTurn off to configure side-branch phototropism independently."))
	bool bBranchPhototropismSameAsTrunk = true;

	UPROPERTY(EditAnywhere , Category = "Phototropism | Branch" , meta = (EditCondition = "bBranchPhototropismSameAsTrunk == false" , EditConditionHides, PCG_NotOverridable))
	FPVPhototropismParams BranchPhototropism;

	UPROPERTY(EditAnywhere , Category = "Phototropism", meta =(PCG_NotOverridable, EditCondition = bSenescence))
	FPVLightSenescenceParams LightSenescence;

	UPROPERTY(EditAnywhere , Category = "Gravity", meta =(PCG_NotOverridable))
	FPVGrowerGravityParams GravityParams;

	UPROPERTY(EditAnywhere, Category = "Gravity", meta =(PCG_NotOverridable, Tooltip="Master toggle for age and light senescence.\n\nWhen on, branches can die from old age (`FPVAgeSenescenceParams`) or sustained shade (`FPVLightSenescenceParams`). When off, both senescence panels are inactive and branches never die naturally. Turn off only for short-cycle test plants or stylized art direction."))
	bool bSenescence = true;

	UPROPERTY(EditAnywhere , Category = "Gravity", meta =(PCG_NotOverridable, EditCondition = bSenescence))
	FPVAgeSenescenceParams AgeSenescence;

	UPROPERTY(EditAnywhere , Category = "Bifurcation | Trunk", meta =(PCG_NotOverridable))
	FPVGrowerBifurcationParams Bifurcation;

	UPROPERTY(EditAnywhere , Category = "Bifurcation | Branch" , meta = (PCG_NotOverridable))
	FPVGrowerBifurcationParams BranchBifurcation;

	UPROPERTY(EditAnywhere , Category = "Directional | Trunk", meta =(PCG_NotOverridable))
	FPVDirectionalParams Directional;

	UPROPERTY(EditAnywhere , Category = "Directional", meta =(PCG_NotOverridable, Tooltip="Use the trunk's directional jitter for side branches."))
	bool bBranchDirectionalSameAsTrunk = true;

	UPROPERTY(EditAnywhere , Category = "Directional | Branch", meta = (EditCondition = "bBranchDirectionalSameAsTrunk == false" , EditConditionHides, PCG_NotOverridable))
	FPVDirectionalParams BranchDirectional;

	UPROPERTY(EditAnywhere , Category = "Auxin | Trunk", meta =(PCG_NotOverridable))
	FPVAuxinParams Auxin;

	UPROPERTY(EditAnywhere , Category = "Auxin", meta =(PCG_NotOverridable, Tooltip="Use the trunk's auxin for side branches."))
	bool bBranchAuxinConditionSameAsTrunk = true;

	UPROPERTY(EditAnywhere , Category = "Auxin | Branch" , meta = (EditCondition = "bBranchAuxinConditionSameAsTrunk == false" , EditConditionHides, PCG_NotOverridable))
	FPVAuxinParams BranchAuxin;

	UPROPERTY(EditAnywhere, Category = "Foliage", meta =(PCG_NotOverridable))
	FPVFoliageParams Foliage;

	UPROPERTY()
	FPVGuideParams GuideSettings;

	UPROPERTY(EditAnywhere , Category = "Object Interaction", meta =(PCG_NotOverridable, Tooltip="List of colliders that interact with growth (avoid, trim inside, trim outside).\n\nEach entry defines a collider shape (box, sphere, capsule) and a CollisionType. Growth either avoids the collider, is trimmed where it enters the collider, or is trimmed where it leaves the collider."))
	TArray<FPVColliderParams> ColliderSettings;

	/** Leaf mesh used as a light occluder during each growth cycle.
	 *  Geometry is pre-calculated when this is set; only per-leaf instance
	 *  transforms are computed at runtime. */
	UPROPERTY(EditAnywhere, Category = "Foliage", meta=(PCG_NotOverridable, Tooltip="Mesh used as the per-leaf light occluder.\n\nA simple static mesh representing one leaf, used to ray-trace light occlusion in `DetectLight`. The default lightweight the default mesh is recommended unless you need accurate shadowing from custom leaf shapes (which costs more per cycle)."))
	TObjectPtr<UStaticMesh> FoliageMesh;

	/** Pre-calculated vertex/index data for LeafMesh.
	 *  Populated by PostEditChangeProperty or lazily on first growth cycle. */
	FPVLeafMeshGeometry CachedLeafMeshGeometry;

	void GetPhyllotaxy(FPVGrowerPhyllotaxyParams& OutPhyllotaxy, int Generation) const;

	float GetPhyllotaxyAngle(const FPVGrowerPhyllotaxyParams& InPhyllotaxy) const;

	void GetAuxin(FPVAuxinParams& OutAuxin, int Generation) const;

	void GetPhototropism(FPVPhototropismParams& OutPhototropism, int Generation) const;

	void GetDirectionalParams(FPVDirectionalParams& OutDirectional, int Generation) const;
	
	void GetBifurcation(FPVGrowerBifurcationParams& OutBifurcation, int Generation) const;
};