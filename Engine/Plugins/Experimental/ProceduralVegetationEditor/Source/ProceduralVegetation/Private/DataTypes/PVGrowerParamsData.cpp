// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVGrowerParamsData.h"

#include "Serialization/ArchiveCrc32.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerParams, UPVGrowerParamsData);

void UPVGrowerParamsData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);

	if (bFullDataCrc)
	{
		FString ClassName = StaticClass()->GetPathName();
		Ar << ClassName;
		Ar << Params;
	}
	else
	{
		AddUIDToCrc(Ar);
	}
}
