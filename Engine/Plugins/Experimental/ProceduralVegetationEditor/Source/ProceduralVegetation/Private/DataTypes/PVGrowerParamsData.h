// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowerParams.h"
#include "Data/Registry/PCGDataType.h"
#include "PVGrowerParamsData.generated.h"

class FArchiveCrc32;

USTRUCT()
struct FPVDataTypeInfoGrowerParams : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerParamsData : public UPVData
{
	GENERATED_BODY()

public:
	FPVGrowerParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerParams)
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	//~ End UPCGData interface
};
