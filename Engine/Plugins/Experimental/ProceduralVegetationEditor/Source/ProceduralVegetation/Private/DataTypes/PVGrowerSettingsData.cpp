// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVGrowerSettingsData.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerPhyllotaxy, UPVGrowerPhyllotaxyData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerTrunkGrowth, UPVGrowerTrunkGrowthData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerBifurcation, UPVGrowerBifurcationData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerPhototropism, UPVGrowerPhototropismData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerAgeSenescence, UPVGrowerAgeSenescenceData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerLightSenescence, UPVGrowerLightSenescenceData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerDirectional, UPVGrowerDirectionalData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerFoliage, UPVGrowerFoliageData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerAuxin, UPVGrowerAuxinData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowerGravity, UPVGrowerGravityData);