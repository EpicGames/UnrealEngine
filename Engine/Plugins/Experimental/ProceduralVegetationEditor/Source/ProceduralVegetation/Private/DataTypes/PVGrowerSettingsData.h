// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "DataTypes/PVGrowerParams.h"
#include "Serialization/ArchiveCrc32.h"
#include "Data/Registry/PCGDataType.h"
#include "PVGrowerSettingsData.generated.h"

#define PV_GROWER_SETTINGS_OVERRIDE_CRC(Params)\
virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override\
{\
	Super::AddToCrc(Ar, bFullDataCrc); \
	if (bFullDataCrc)\
	{\
		FString ClassName = StaticClass()->GetPathName();\
		Ar << ClassName;\
		Ar << Params;\
	}\
	else\
	{\
		AddUIDToCrc(Ar);\
	}\
}\

USTRUCT()
struct FPVDataTypeInfoGrowerPhyllotaxy : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerPhyllotaxyData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVGrowerPhyllotaxyWithTargets ParamsWithTargets;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerPhyllotaxy)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(ParamsWithTargets);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerAgeSenescence : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerAgeSenescenceData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVAgeSenescenceParams Params;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerAgeSenescence)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerLightSenescence : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerLightSenescenceData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVLightSenescenceParams Params;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerLightSenescence)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerDirectional : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerDirectionalData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVGrowerDirectionalWithTargets ParamsWithTargets;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerDirectional)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(ParamsWithTargets);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerAuxin : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerAuxinData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVGrowerAuxinWithTargets ParamsWithTargets;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerAuxin)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(ParamsWithTargets);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerGravity : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerGravityData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVGrowerGravityParams Params;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerGravity)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerBifurcation : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerBifurcationData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVGrowerBifurcationWithTargets ParamsWithTargets;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerBifurcation)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(ParamsWithTargets);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerFoliage : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerFoliageData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVFoliageParams Params;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerFoliage)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerPhototropism : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerPhototropismData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVGrowerPhototropismWithTargets ParamsWithTargets;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerPhototropism)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(ParamsWithTargets);
	//~ End UPCGData interface
};

USTRUCT()
struct FPVDataTypeInfoGrowerTrunkGrowth : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerTrunkGrowthData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVTrunkGrowthParams Params;
	
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowerTrunkGrowth)
	PV_GROWER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};