// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVGrowthData.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoGrowth, UPVGrowthData);