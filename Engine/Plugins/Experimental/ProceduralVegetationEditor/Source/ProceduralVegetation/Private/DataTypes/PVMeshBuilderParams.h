// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Utils/PVFloatRamp.h"
#include "Engine/Texture2D.h"

#include "PVMeshBuilderParams.generated.h"

/**
 * Shared helper types for branch radii and skeleton shaping.
 * Originally defined in PVMeshBuilder.h; kept here so sub-param structs
 * can reference them without a circular dependency.
 */

/** A single per-generation noise layer used by FPVMeshBuilderSkeletonShapingParams. */
USTRUCT()
struct FPVSkeletonShapingEntry
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", DisplayName="Generation",
		meta=(ClampMin=1, UIMin=1, Tooltip=
			"The branch-hierarchy generation this entry targets.\n\nGeneration 1 is the trunk. Generation 2 is the first tier of lateral branches, and so on. When Impact Remaining Generations is disabled the noise is applied only to branches whose hierarchy number equals this value."))
	int32 Generation = 1;

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", DisplayName="Impact Remaining Generations",
		meta=(Tooltip=
			"When enabled, this entry also governs all higher-numbered generations that have no entry of their own.\n\nA more specific entry (exact generation match, or a higher-generation fallback entry) always takes precedence over this one. Disable to restrict the effect to the exact generation number above."))
	bool bImpactRemainingGenerations = true;

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", DisplayName="Noise Strength",
		meta=(ClampMin=0.0f, ClampMax=200.0f, UIMin=0.0f, UIMax=50.0f, Tooltip=
			"Amplitude of noise displacement in cm.\n\nHow far branches can bend at their peak displacement. 0 = straight branches. 5 cm = subtle bending. 20+ = dramatic curvature."))
	float NoiseStrength = 5.0f;

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", DisplayName="Noise Frequency",
		meta=(ClampMin=0.0001f, ClampMax=1.0f, UIMin=0.001f, UIMax=0.2f, Tooltip=
			"Spatial frequency of the noise field for this generation.\n\nHigher values produce tighter, more frequent bends; lower values produce broad, sweeping curves."))
	float NoiseFrequency = 0.05f;

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", DisplayName="Noise Seed",
		meta=(Tooltip=
			"Offsets the Perlin noise sampling position for this generation. Each unique value produces a distinct deformation pattern."))
	int32 NoiseSeed = 0;

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", DisplayName="Smoothness",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Controls the smoothness of the branch curve. Higher values produce a smoother arc by applying more Laplacian smoothing passes to the branch point positions. Works independently of noise amplitude."))
	float Smoothness = 0.0f;
};

/** Per-generation ramp entry for FPVMeshBuilderBranchRadiusParams::BranchGenerationRamps. */
USTRUCT()
struct FPVGenerationRamp
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Branch Radius", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f))
	FPVFloatRamp Ramp;

	FPVGenerationRamp()
	{
		Ramp.InitializeLinearCurve();
	}
};

/** Per-generation scale entry for FPVMeshBuilderBranchRadiusParams::BranchGenerationScales. */
USTRUCT()
struct FPVGenerationScale
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Branch Radius", meta=(ClampMin=0.0f, UIMin=0.0f, Tooltip="Uniform scale multiplier applied to branch radius for this generation."))
	float Scale = 1.0f;
};

/**
 * Sub-param structs used by the Mesher settings override nodes.
 * Each struct mirrors one UPROPERTY category from FPVMeshBuilderParams so that a dedicated
 * "Create Mesher <Category> Settings" node can override that category without touching the rest.
 * FPVMeshBuilderParams itself uses these structs as members.
 */

// ─── Mesh Details ─────────────────────────────────────────────────────────────

USTRUCT()
struct FPVMeshBuilderMeshDetailParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Mesh Details", DisplayName="Skeletal Resolution",
		meta=(ClampMin=1, ClampMax=100, UIMin=1, UIMax=100, Tooltip=
			"Number of segments the longest branch is divided into for skeleton interpolation.\n\nHigher values add more interpolated points for smoother branch curvature at the cost of vertex count."
		))
	int32 SkeletonResolution = 10;

	UPROPERTY(EditAnywhere, Category="Mesh Details", DisplayName="Point Removal",
		meta=(ClampMin=0.0f, ClampMax=0.1f, UIMin=0.0f, UIMax=0.1f, Tooltip=
			"Removes small points to simplify the mesh.\n\nReduces the points considered for mesh generation. Depends on Hull, Main Trunk, Ground, and Scale Retention settings. Higher = more aggressive simplification."
		))
	float PointRemoval = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details", DisplayName="Segment Reduction",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"How aggressively segments are reduced.\n\nSimplifies the branch polyline while preserving overall shape. Higher = fewer edges, lighter mesh. Combine with Retention settings to preserve important regions."
		))
	float SegmentReduction = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details", DisplayName="Min Divisions",
		meta=(ClampMin=3, ClampMax=1024, UIMin=3, UIMax=36, Tooltip=
			"Minimum radial polygon count around branch cross-sections.\n\nLower bound on cross-section detail. 3 = triangular cross-sections (very angular); higher values for smooth round branches."
		))
	int32 MinDivisions = 6;

	UPROPERTY(EditAnywhere, Category="Mesh Details", DisplayName="Max Divisions",
		meta=(ClampMin=3, ClampMax=1024, UIMin=3, UIMax=36, Tooltip=
			"Maximum radial divisions.\n\nUpper limit for cross-section detail on trunks/branches. Balance with Min Divisions to control density across sizes."
		))
	int32 MaxDivisions = 12;

	UPROPERTY(EditAnywhere, Category="Mesh Details", DisplayName="Add End Caps",
		meta=(Tooltip=
			"Add spherical end caps to branch tips.\n\nWhen on, each branch tip gets a hemispherical cap. Avoids open-ended cylinders. Turn off if your tip foliage will hide the open end anyway."
		))
	bool AddEndCaps = true;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Reduction Accuracy",
		meta=(ClampMin=1, ClampMax=15, UIMin=1, UIMax=15, Tooltip=
			"Iterations of the simplification algorithm.\n\nMore iterations = more accurate reduction at the cost of processing power."
		))
	int32 Accuracy = 5;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Longest Segment Allowed (meters)",
		meta=(ClampMin=0.0f, ClampMax=50.0f, UIMin=0.0f, UIMax=10.0f, Tooltip=
			"Maximum allowed segment length.\n\nCaps segment length after simplification to maintain even geometry and prevent stretched polygons."
		))
	float LongestSegmentLength = 10.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Shortest Segment Allowed (meters)",
		meta=(ClampMin=0.0f, ClampMax=50.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Minimum allowed segment length.\n\nSegments shorter than this are merged or removed to avoid unnecessary fine detail that inflates polycount."
		))
	float ShortestSegmentLength = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Segment Retention Impact",
		meta=(ClampMin=0.0f, ClampMax=5.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"How strongly retention rules can reintroduce reduced points.\n\nRetention rules (Hull, Main Trunk, Ground, Scale) can reintroduce points the simplifier removed. This multiplier governs how strongly they push back. 0 = retention has no effect; 1 = full impact."
		))
	float SegmentRetentionImpact = 1.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Hull Retention",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Strength of preserving the outer hull.\n\nAdjusts retention of the mesh silhouette during reduction. Higher values protect external shape; lower values allow more simplification."
		))
	float HullRetention = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Main Trunk Retention",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Preserves detail in the main trunk.\n\nControls how much trunk geometry is retained during simplification. High values keep trunk smooth and detailed; low values reduce complexity."
		))
	float MainTrunkRetention = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Ground Retention",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Preserves detail near ground regions.\n\nAdjusts retention for areas close to the base of the plant. Use higher values to keep root/ground transitions detailed."
		))
	float GroundRetention = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Hull Retention Gradient",
		meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip=
			"Gradient for preserving outer hull detail.\n\nA ramp that varies outer-hull retention across the model. Use to keep silhouettes crisp where needed while simplifying elsewhere."
		))
	FPVFloatRamp HullRetentionGradient;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Main Trunk Retention Gradient",
		meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip=
			"Gradient controlling trunk detail retention.\n\nVaries trunk retention via a ramp. Useful to preserve key trunk regions while simplifying others."
		))
	FPVFloatRamp MainTrunkRetentionGradient;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Ground Retention Gradient",
		meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip=
			"Gradient for preserving ground-region detail.\n\nA ramp that modulates retention around base regions. Tune to keep roots/basal features while allowing simplification above."
		))
	FPVFloatRamp GroundRetentionGradient;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Scale Retention",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Preserves detail relative to branch size.\n\nRetains more detail on larger/thicker structures and simplifies smaller ones to prioritize visually important geometry."
		))
	float ScaleRetention = 0.0f;

	UPROPERTY(EditAnywhere, Category="Mesh Details|Advanced", DisplayName="Scale Retention Gradient",
		meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f, Tooltip=
			"Gradient controlling size-based retention.\n\nUses a ramp to vary how size-dependent retention is applied across the model. Keep large structures crisp while simplifying finer parts."
		))
	FPVFloatRamp ScaleRetentionGradient;

	FPVMeshBuilderMeshDetailParams()
	{
		HullRetentionGradient.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
		MainTrunkRetentionGradient.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
		GroundRetentionGradient.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
		ScaleRetentionGradient.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
	}
};

// ─── Profile Details ──────────────────────────────────────────────────────────

USTRUCT()
struct FPVMeshBuilderProfileDetailParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Profile Details", DisplayName="Falloff",
		meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=0.0f, YAxisMax=1.0f,
			Tooltip = "How the profile shape is applied along trunk/branch length.\n\nX = position from base to tip. Y = profile intensity. Can create root flare (wider profile at base tapering to round at top)."
		))
	FPVFloatRamp FallOff;

	UPROPERTY(EditAnywhere, Category="Profile Details", DisplayName="Scale",
		meta=(ClampMin=0.0f, ClampMax=10.0f, UIMin=0.0f, UIMax=10.0f, Tooltip="Scale of the profile applied\n\nCan be used to create a wider base in conjection with Profile Falloff"))
	float Scale = 1.0f;

	UPROPERTY(EditAnywhere, Category="Profile Details", meta=(Tooltip = "Apply the profile to side branches, not just the trunk.\n\nTurn on for species with consistent cross-section throughout (e.g. cacti)."))
	bool bApplyToBranches = false;

	FPVMeshBuilderProfileDetailParams()
	{
		FallOff.InitializeLinearCurve(FVector2f(0.f, 1.f), FVector2f(1.f, 0.f));
	}
};

// ─── Branch Radius ────────────────────────────────────────────────────────────

USTRUCT()
struct FPVMeshBuilderBranchRadiusParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Branch Radius", DisplayName="DaVinci Rule Strength",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Blend toward physically-plausible branch tapering (Da Vinci's rule).\n\n0 = use existing radius distribution unchanged. 1 = fully apply the cross-sectional area conservation law (parent cross-section = sum of child cross-sections). Higher values produce more realistic branch tapering."
		))
	float DaVinciRuleStrength = 0.0f;

	UPROPERTY(EditAnywhere, Category="Branch Radius", DisplayName="Branch Generation Ramps",
		meta=(Tooltip=
			"Per-generation ramp curves that remap branch radii within each generation.\n\nArray index 0 targets generation 1 (trunk), index 1 targets generation 2 (first lateral branches), etc. X axis: normalized radius within the generation (0 = thinnest, 1 = thickest). Y axis: output radius as a fraction of the generation maximum. The last entry also governs all higher generations with no dedicated entry. Applied after the DaVinci Rule, before Min Radius."
		))
	TArray<FPVGenerationRamp> GenerationRamps;

	UPROPERTY(EditAnywhere, Category="Branch Radius", DisplayName="Branch Generation Scales",
		meta=(Tooltip=
			"Per-generation uniform scale multipliers applied to branch radius.\n\nArray index 0 targets generation 1 (trunk), index 1 targets generation 2 (first lateral branches), etc. The last entry also governs all higher generations with no dedicated entry."
		))
	TArray<FPVGenerationScale> GenerationScales;

	UPROPERTY(EditAnywhere, Category="Branch Radius", DisplayName="Min Radius",
		meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip=
			"Minimum radius as a fraction of the max.\n\nRaises the thinnest branches to this floor while leaving the thickest unchanged. 0 = no floor (natural tapering). Higher = thicker twigs (better for older mature plants)."
		))
	float MinRadius = 0.0f;
};

// ─── Displacement ─────────────────────────────────────────────────────────────

USTRUCT()
struct FPVMeshBuilderDisplacementParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Displacement",
		meta=(Tooltip=
			"Texture used to drive mesh displacement.\n\nAssigns the texture map that controls displacement. Bright and dark areas in the texture push or pull the mesh surface to create additional geometric detail such as bark roughness, grooves, or ridges.\n\nOnly power of 2 textures with source texture formats (TSF_RGBA32F, TSF_RGBA16F, TSF_BGRA8, TSF_R32F, TSF_R16F, TSF_G8) are supported.\n\nFor formats TSF_RGBA32F, TSF_RGBA16F and TSF_BGRA8 data from R channel is used."
		))
	TObjectPtr<UTexture2D> Texture = nullptr;

	UPROPERTY(EditAnywhere, Category="Displacement", DisplayName="Scale",
		meta=(Tooltip= "Intensity of displacement effect.\n\nHigher = more exaggerated surface depth. Lower = subtler. Balance with mesh stability — extreme values can cause polygons to overlap.")
		)
	float Strength = 0.5f;

	UPROPERTY(EditAnywhere, Category="Displacement", DisplayName="Bias",
		meta=(Tooltip = "Shifts the midpoint of displacement.\n\nOffsets the neutral level of the displacement map. Positive bias lifts the surface outward. Useful for correcting textures that displace unevenly or need centering.")
		)
	float Bias = 0.0f;

	UPROPERTY(EditAnywhere, Category="Displacement", DisplayName="UV Scale",
		meta=(Tooltip=
			"Tiling of the displacement texture (U, V).\n\nHigher values increase tiling frequency (smaller repeats). Lower values stretch the texture. 1 = native tile size."
		))
	FVector2f UVScale = FVector2f(1.0f, 1.0f);

	UPROPERTY(EditAnywhere, Category="Displacement", DisplayName="Generation Upper Limit",
		meta=(ClampMin=1, ClampMax=10, UIMin=1, UIMax=10, Tooltip=
			"Apply displacement only up to this generation.\n\n1 = trunk only. 2 = trunk + first branches. Higher = more generations covered. Limits performance cost on fine twigs where displacement would be invisible."))
	int32 GenerationUpperLimit = 1;
};

// ─── Skeleton Shaping ─────────────────────────────────────────────────────────

USTRUCT()
struct FPVMeshBuilderSkeletonShapingParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping",
		meta=(Tooltip=
			"Per-generation noise layers.\n\nEach entry targets a specific generation. Higher-generation entries override lower-generation fallbacks when bImpactRemainingGenerations is on."))
	TArray<FPVSkeletonShapingEntry> Entries;
};
