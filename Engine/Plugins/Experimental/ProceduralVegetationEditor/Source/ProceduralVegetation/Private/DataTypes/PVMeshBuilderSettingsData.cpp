// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVMeshBuilderSettingsData.h"
#include "PCGContext.h"
#include "Data/PCGPointData.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMeshBuilderMeshDetail,      UPVMeshBuilderMeshDetailData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMeshBuilderProfileDetail,   UPVMeshBuilderProfileDetailData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMeshBuilderBranchRadius,    UPVMeshBuilderBranchRadiusData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMeshBuilderDisplacement,    UPVMeshBuilderDisplacementData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMeshBuilderSkeletonShaping, UPVMeshBuilderSkeletonShapingData);
PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMeshBuilderMaterialDetail,  UPVMeshBuilderMaterialDetailData);
