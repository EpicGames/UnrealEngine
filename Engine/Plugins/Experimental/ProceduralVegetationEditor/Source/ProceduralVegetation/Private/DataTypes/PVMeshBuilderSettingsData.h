// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGData.h"
#include "Serialization/ArchiveCrc32.h"
#include "Data/Registry/PCGDataType.h"
#include "DataTypes/PVMeshBuilderParams.h"
#include "Implementations/PVMaterialSettings.h"
#include "PVMeshBuilderSettingsData.generated.h"

#define PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params)\
virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override\
{\
	Super::AddToCrc(Ar, bFullDataCrc); \
	if (bFullDataCrc)\
	{\
		FString ClassName = StaticClass()->GetPathName();\
		Ar << ClassName;\
		Ar << Params;\
	}\
	else\
	{\
		AddUIDToCrc(Ar);\
	}\
}\

// ─── Mesh Details ─────────────────────────────────────────────────────────────

USTRUCT()
struct FPVDataTypeInfoMeshBuilderMeshDetail : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderMeshDetailData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVMeshBuilderMeshDetailParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoMeshBuilderMeshDetail)
	PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

// ─── Profile Details ──────────────────────────────────────────────────────────

USTRUCT()
struct FPVDataTypeInfoMeshBuilderProfileDetail : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderProfileDetailData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVMeshBuilderProfileDetailParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoMeshBuilderProfileDetail)
	PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

// ─── Branch Radius ────────────────────────────────────────────────────────────

USTRUCT()
struct FPVDataTypeInfoMeshBuilderBranchRadius : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderBranchRadiusData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVMeshBuilderBranchRadiusParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoMeshBuilderBranchRadius)
	PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

// ─── Displacement ─────────────────────────────────────────────────────────────

USTRUCT()
struct FPVDataTypeInfoMeshBuilderDisplacement : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderDisplacementData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVMeshBuilderDisplacementParams Params;

	/** Cached pixel values extracted from Params.Texture, computed once when this data is produced. */
	TArray<float> Values;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoMeshBuilderDisplacement)
	PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

// ─── Skeleton Shaping ─────────────────────────────────────────────────────────

USTRUCT()
struct FPVDataTypeInfoMeshBuilderSkeletonShaping : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderSkeletonShapingData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVMeshBuilderSkeletonShapingParams Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoMeshBuilderSkeletonShaping)
	PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};

// ─── Material Details ─────────────────────────────────────────────────────────

USTRUCT()
struct FPVDataTypeInfoMeshBuilderMaterialDetail : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderMaterialDetailData : public UPCGData
{
	GENERATED_BODY()

public:
	FPVMaterialSettings Params;

	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoMeshBuilderMaterialDetail)
	PV_MESH_BUILDER_SETTINGS_OVERRIDE_CRC(Params);
	//~ End UPCGData interface
};
