// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVMeshData.h"

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoMesh, UPVMeshData);