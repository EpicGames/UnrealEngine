// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/PVTrunkTextureSetupData.h"
#include "Serialization/ArchiveCrc32.h"

bool FPVTrunkTextureSetupInfo::IsOverflowing() const
{
	for (auto Generation : GenerationUVs)
	{
		if (Generation.GetOverflowValue() > 0)
		{
			return true;
		}
	}

	return false;
}

PCG_DEFINE_TYPE_INFO(FPVDataTypeInfoTrunkTextureSetup, UPVTrunkTextureSetupData);

void UPVTrunkTextureSetupData::AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const
{
	Super::AddToCrc(Ar, bFullDataCrc);
		
	if (bFullDataCrc)
	{
		FString ClassName = StaticClass()->GetPathName();
		Ar << ClassName;
		Ar << TrunkTextureSetupInfo;
	}
	else
	{
		AddUIDToCrc(Ar);
	}
}