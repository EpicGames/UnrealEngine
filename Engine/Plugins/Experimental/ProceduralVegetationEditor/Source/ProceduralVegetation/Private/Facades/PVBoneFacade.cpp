// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVBoneFacade.h"
#include "ProceduralVegetationModule.h"
#include "Facades/PVAttributesNames.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "Helpers/PVUtilities.h"
#include "Helpers/PVAttributesHelper.h"

class FGeometryCollection;

namespace PV::Facades
{
	struct FConnectingSegment
	{
		TArray<FBoneNode> Nodes;
	};
	
	FBoneFacade::FBoneFacade(FManagedArrayCollection& InCollection)
		: BoneName(InCollection, AttributeNames::BoneName, GroupNames::BonesGroup),
		BoneParentIndex(InCollection, AttributeNames::BoneParentIndex, GroupNames::BonesGroup),
		BoneId(InCollection, AttributeNames::BoneId, GroupNames::BonesGroup),
		BonePose(InCollection, AttributeNames::BonePose, GroupNames::BonesGroup),
		BonePointIndex(InCollection, AttributeNames::BonePointIndex, GroupNames::BonesGroup),
		BoneAbsolutePosition(InCollection, AttributeNames::BoneAbsolutePosition, GroupNames::BonesGroup),
		BoneBranchIndex(InCollection, AttributeNames::BoneBranchIndex, GroupNames::BonesGroup),
		VertexPointIds(InCollection, AttributeNames::VertexPointIds, GroupNames::VerticesGroup)
	{}

	FBoneFacade::FBoneFacade(const FManagedArrayCollection& InCollection)
		: BoneName(InCollection, AttributeNames::BoneName, GroupNames::BonesGroup),
		BoneParentIndex(InCollection, AttributeNames::BoneParentIndex, GroupNames::BonesGroup),
		BoneId(InCollection, AttributeNames::BoneId, GroupNames::BonesGroup),
		BonePose(InCollection, AttributeNames::BonePose, GroupNames::BonesGroup),
		BonePointIndex(InCollection, AttributeNames::BonePointIndex, GroupNames::BonesGroup),
		BoneAbsolutePosition(InCollection, AttributeNames::BoneAbsolutePosition, GroupNames::BonesGroup),
		BoneBranchIndex(InCollection, AttributeNames::BoneBranchIndex, GroupNames::BonesGroup),
		VertexPointIds(InCollection, AttributeNames::VertexPointIds, GroupNames::VerticesGroup)
	{}

	void FBoneFacade::DefineSchema(FManagedArrayCollection& InCollection)
	{
		InCollection.AddAttribute<int32>(AttributeNames::BoneId, GroupNames::BonesGroup);
		InCollection.AddAttribute<FString>(AttributeNames::BoneName, GroupNames::BonesGroup);
		InCollection.AddAttribute<int32>(AttributeNames::BoneParentIndex, GroupNames::BonesGroup);
		InCollection.AddAttribute<FTransform>(AttributeNames::BonePose, GroupNames::BonesGroup);
		InCollection.AddAttribute<int32>(AttributeNames::BonePointIndex, GroupNames::BonesGroup);
		InCollection.AddAttribute<int32>(AttributeNames::BoneBranchIndex, GroupNames::BonesGroup);
		InCollection.AddAttribute<int32>(AttributeNames::VertexPointIds, GroupNames::VerticesGroup);
		InCollection.AddAttribute<FVector3f>(AttributeNames::BoneAbsolutePosition, GroupNames::BonesGroup);
	}

	bool FBoneFacade::IsValid() const
	{
		return VertexPointIds.IsValid();
	}

	void FBoneFacade::GetPointIds(TArray<int32>& OutVertexPointIds) const
	{
		for (auto VertexBoneID : VertexPointIds.Get())
		{
			OutVertexPointIds.Add(VertexBoneID);
		}
	}

	const TManagedArray<FString>& FBoneFacade::GetBoneNames() const
	{
		return BoneName.Get();
	}

	const TManagedArray<int32>& FBoneFacade::GetBoneParentIndices() const
	{
		return BoneParentIndex.Get();
	}

	const TManagedArray<FTransform>& FBoneFacade::GetBonePoses() const
	{
		return BonePose.Get();
	}

	const TManagedArray<int32>& FBoneFacade::GetVertexPointIds() const
	{
		return VertexPointIds.Get();
	}

	TManagedArray<int32>& FBoneFacade::ModifyVertexPointIds()
	{
		return VertexPointIds.Modify();
	}

	TArray<FBoneNode> GetParentBranchBones(const  TArray<FBoneNode>& InBoneNodes, int32 InParentIndex)
	{
		TArray<FBoneNode> ParentBranchBones;
		for (const auto& Bone : InBoneNodes)
		{
			if (InParentIndex == Bone.BranchIndex)
			{
				ParentBranchBones.Add(Bone);
			}
		}
		return ParentBranchBones;
	}

	FBoneNode* FindParentBoneForBranch(TArray<FBoneNode>& BoneNodes, FVector BonePosition, float BoneScale,
	                            FString& OutMessage, bool bFallbackToClosestBone = false)
	{
		FBoneNode* FoundBone = nullptr;
		FBoneNode* ClosestBone = nullptr;
		float MinDist = FLT_MAX;

		for (int32 NodeIdx = 0; NodeIdx < BoneNodes.Num(); ++NodeIdx)
		{
			FVector NodePosition = BoneNodes[NodeIdx].AbsolutePosition;
			FVector NextPosition = NodePosition;
			if (BoneNodes.IsValidIndex(NodeIdx + 1))
			{
				NextPosition = BoneNodes[NodeIdx + 1].AbsolutePosition;
			}

			float Dist = FMath::PointDistToSegment(BonePosition, NodePosition, NextPosition);
			MinDist = FMath::Min(MinDist, Dist);
			if (Dist <= BoneScale * 2)
			{
				FoundBone = &BoneNodes[NodeIdx];
			}

			if (Dist == MinDist)
			{
				ClosestBone = &BoneNodes[NodeIdx];
			}
		}

		if (!FoundBone && bFallbackToClosestBone && ClosestBone)
		{
			OutMessage = FString::Printf(TEXT("FindParentBoneForBranch fallback to closest bone Index : %i closest bone Branch %i"), ClosestBone->BoneIndex, ClosestBone->BranchIndex);
			return ClosestBone;
		}

		return FoundBone;
	}

	void ApplyBoneReduction(TArray<FConnectingSegment>& ConnectingSegments , TArray<FBoneNode>& OutReducedBones, float ReductionStrength)
	{
		for (auto& Segment : ConnectingSegments)
		{
			check(Segment.Nodes.Num() > 0);
			float NumBones = FMath::Lerp(Segment.Nodes.Num(), 1.0f, FMath::Clamp(ReductionStrength,0,1));

			for (int32 j = 0; j < NumBones; j++)
			{
				OutReducedBones.Emplace(Segment.Nodes[j]);
			}

			if (NumBones != Segment.Nodes.Num())
			{
				int32 LastIndex = Segment.Nodes.Num() - 1;
				FBoneNode LastBone = Segment.Nodes[LastIndex];
				
				int LastOutBoneIndex = OutReducedBones.Num() - 1;
				FBoneNode& LastOutBone = OutReducedBones[LastOutBoneIndex];
				FVector RelativePosition = LastOutBone.BoneTransform.GetLocation() + (LastBone.AbsolutePosition - LastOutBone.AbsolutePosition);
				int32 ParentIndex = LastOutBone.ParentBoneIndex;
				LastOutBone = LastBone;
				LastOutBone.BoneTransform = FTransform(RelativePosition);
				LastOutBone.ParentBoneIndex = ParentIndex;
			}
		}
	}

	void ReindexBones(TArray<FBoneNode>& OutReducedBones)
	{
		TMap<int32, int32> OldToNewIndexMap;
		for (int i=0; i < OutReducedBones.Num(); i++)
		{
			FBoneNode& Bone = OutReducedBones[i];
			OldToNewIndexMap.Add(Bone.BoneIndex, i);
			Bone.BoneIndex = i;
		}

		for (auto& Bone : OutReducedBones)
		{
			if (int32* NewParentID = OldToNewIndexMap.Find(Bone.ParentBoneIndex))
			{
				Bone.ParentBoneIndex = *NewParentID;
				Bone.BoneName = FName(FString::Printf(TEXT("Bone_%i"), Bone.BoneIndex));
			}
		}
	}

	void SetBranchSimulationGroupIndex(const FManagedArrayCollection& Collection, const FPointFacade& PointFacade, FBranchFacade& BranchFacade)
	{
		const auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(Collection);
		const auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(Collection);
		const auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(Collection);

		TArray<float> BranchRootPScales;
		int32 NumBranches = BranchFacade.GetElementCount();
		BranchRootPScales.Reserve(NumBranches);

		float Min = FLT_MAX;
		float Max = UE_KINDA_SMALL_NUMBER;

		for (int32 BranchIdx = 0; BranchIdx < NumBranches; ++BranchIdx)
		{
			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIdx);
			check(BranchPoints.Num() > 0);

			float Pscale = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIdx, 0);
			BranchRootPScales.Emplace(Pscale);

			Min = FMath::Min(Min, Pscale);
			Max = FMath::Max(Max, Pscale);
		}
		
		for (auto& Pscale : BranchRootPScales)
		{
			const UE::Math::TVector2<float> InputRange(Min, Max);
			const UE::Math::TVector2<float> OutputRange(1, 0);
			Pscale = FMath::GetMappedRangeValueClamped(InputRange, OutputRange, Pscale);
		}

		for (int32 BranchIdx = 0; BranchIdx < NumBranches; ++BranchIdx)
		{
			TArray<int32> ParentIndices = BranchFacade.GetParentBranchIndices(BranchIdx);
			int32 ParentIndex = BranchFacade.GetParentIndex(BranchIdx);

			if (BranchRootPScales.IsValidIndex(ParentIndex) && ParentIndex > 0 )
			{
				BranchRootPScales[ParentIndex] = FMath::Min(BranchRootPScales[BranchIdx], BranchRootPScales[ParentIndex]);
			}
		}

		for (int32 BranchIdx = 0; BranchIdx < NumBranches; ++BranchIdx)
		{
			int32 LogicalDepth = FMath::RoundToInt32(BranchRootPScales[BranchIdx]*2);
			BranchFacade.SetBranchSimulationGroupIndex(BranchIdx, LogicalDepth);
		}
	}

	void FBoneFacade::SetWindSimulationData(FManagedArrayCollection& Collection)
	{
		FPointFacade PointFacade = FPointFacade(*BoneName.GetCollection());
		FBranchFacade BranchFacade = FBranchFacade(*BoneName.GetCollection());
		
		SetBranchSimulationGroupIndex(Collection, PointFacade, BranchFacade);
	}
	
	bool FBoneFacade::CreateBoneData(TArray<FBoneNode>& Bones,const float ReductionStrength,bool bSetWindData /*= true*/)
	{
		FPointFacade PointFacade(VertexPointIds.GetConstCollection());
		FBranchFacade BranchFacade(VertexPointIds.GetConstCollection());
		FPlantFacade PlantFacade(VertexPointIds.GetConstCollection());

		const auto PointScaleAttribute = FPointScaleAttribute::GetAttribute(VertexPointIds.GetConstCollection());
		const auto BranchPointsAttribute = FBranchPointsAttribute::GetAttribute(VertexPointIds.GetConstCollection());
		const auto BranchParentNumberAttribute = FBranchParentNumberAttribute::GetAttribute(VertexPointIds.GetConstCollection());
		const auto NjordPixelIndexAttribute = PV::FPointNjordPixelIndexAttribute::FindAttribute(VertexPointIds.GetConstCollection());

		if (!(Utilities::IsValidGrowthData(VertexPointIds.GetConstCollection()) && PlantFacade.IsValid()))
		{
			return false;	
		}

		if (bSetWindData)
		{
			SetWindSimulationData(*VertexPointIds.GetCollection());
		}
		
		const TManagedArray<FVector3f>& Positions = PointFacade.GetPositions();

		constexpr int RootBoneIndex = 0;
		FBoneNode RootNode{ "Root",RootBoneIndex, INDEX_NONE, INDEX_NONE, INDEX_NONE, 1.0, FTransform{FVector::Zero()}};
		Bones.Emplace(RootNode);

		int32 CurrentBoneIndex = 1;
		for (const TMap<int32, int32> PlantNumbersToTrunkIDs = PlantFacade.GetPlantNumbersToTrunkIndicesMap();
		 const TPair<int32, int32> Pair : PlantNumbersToTrunkIDs)
		{
			const int32 PlantNumber = Pair.Key;
			const int32 TrunkIndex = Pair.Value;

			check(BranchFacade.GetPoints(TrunkIndex).Num() > 0);
			const int32 RootPointIndex = BranchFacade.GetPoints(TrunkIndex)[0];
		
			check(Positions.Num() > 0 && Positions.Num() >= RootPointIndex);
			const FVector RootPointPosition = FVector(Positions[RootPointIndex]);
			const int32 TrunkBoneIndex = CurrentBoneIndex;
			FBoneNode TrunkBoneNode{ "Root", TrunkBoneIndex, RootBoneIndex, RootPointIndex, TrunkIndex, 1.0, FTransform{RootPointPosition}};
			CurrentBoneIndex++;
			Bones.Emplace(MoveTemp(TrunkBoneNode));

			int ParentBoneIndex = TrunkBoneIndex;
			FVector PreviousPosition = RootPointPosition;

			TArray<int32> SortedBranchIndices = PlantFacade.GetBranchIndicesSortedByHierarchyNumber(PlantNumber);
			if (SortedBranchIndices.Num() == 0)
			{
				UE_LOGF(LogProceduralVegetation, Log, "Cannot create bones, no branches found for Plant Number {%i}", PlantNumber);
				continue;
			}
			for (int32 i = 0; i < SortedBranchIndices.Num(); ++i)
			{
				int32 BranchIndex = SortedBranchIndices[i];
				int32 BranchNumber = BranchFacade.GetBranchNumber(BranchIndex);
				
				auto BranchPoints = BranchFacade.GetPoints(BranchIndex);
			
				check(BranchPoints.IsValidIndex(0));
				int32 ParentIndex = BranchFacade.GetParentIndex(BranchIndex);
				TArray<FBoneNode> ParentBoneNodes = GetParentBranchBones(Bones, ParentIndex);
			
				FString Message;
				const FVector BranchRootPosition = FVector(Positions[BranchPoints[0]]);
				const float BranchRootScale = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, 0);
				FBoneNode* FoundBoneNode = FindParentBoneForBranch(ParentBoneNodes, BranchRootPosition, BranchRootScale, Message, true);

				if (FoundBoneNode == nullptr && i != 0)
				{
					UE_LOGF(LogProceduralVegetation, Log, "Parent bone not found for branch (%i) ParentIndex {%i}", BranchIndex, ParentIndex);
				}
				
				if (!Message.IsEmpty())
				{
					UE_LOGF(LogProceduralVegetation, Log, "%ls Branch (%i) ParentIndex {%i}", *Message, BranchIndex, ParentIndex);
				}
			
				ParentBoneIndex = FoundBoneNode ? FoundBoneNode->BoneIndex : TrunkBoneIndex;
				PreviousPosition = FoundBoneNode ? FVector(FoundBoneNode->AbsolutePosition) : RootPointPosition;

				TArray<int32> ConnectingPoints;

				TArray<FConnectingSegment> ConnectingSegments;
				FConnectingSegment ConnectingSegment;
			
				for (int Idx = 0; Idx < BranchPoints.Num(); Idx++)
				{
					const int32 PointIdx = BranchPoints[Idx];
					check(Positions.Num() > PointIdx && PointIdx >= 0);
					const FVector PointPosition = FVector(Positions[PointIdx]);
					const float PScale = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, Idx);
				
					FVector NextPosition = PointPosition;
					if (Idx + 1 < BranchPoints.Num())
					{
						NextPosition = FVector(Positions[BranchPoints[Idx + 1]]);
					}
				
					float NjordPixelID = NjordPixelIndexAttribute.IsValidIndex(PointIdx) ? NjordPixelIndexAttribute[PointIdx] : -1;

					bool IsConnectingPoint = false;
					for (auto ChildBranch : BranchFacade.GetChildren(BranchIndex))
					{
						const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch);
						if (ChildBranchIndex != INDEX_NONE)
						{
							const TArray<int32>& ChildPoints = BranchFacade.GetPoints(ChildBranchIndex);
							check(ChildPoints.Num() > 0);
							FVector FirstPointPosition = FVector(Positions[ChildPoints[0]]);

							float PointDistToSegment = FMath::PointDistToSegment(FirstPointPosition, PointPosition, NextPosition);
							if (PointDistToSegment <= PScale && !ConnectingPoints.Contains(ChildPoints[0]))
							{
								ConnectingPoints.Add(ChildPoints[0]);
								IsConnectingPoint = true;
							}
						}
						else
						{
							UE_LOGF(LogProceduralVegetation, Warning, "Child branch number {%i} not found in facade, for branch {%i}", ChildBranch, BranchFacade.GetBranchNumber(BranchIndex));
						}
					}

					bool IsLastPoint = Idx == BranchPoints.Num() - 1;
				
					//UE_LOGF(LogProceduralVegetation, Warning, "Iterating Branch Points NjordPixelId, Id{%f} PointIndex{%i} PointFacade.GetBudNumber(BranchPoints[PointIndex]){%i} BranchID{%i} ParentBoneIndex{%i} bFound{%i} FMath::Frac(NjordPixelID){%f}",
					//	NjordPixelID, PointIdx, PointFacade.GetBudNumber(PointIdx), BranchIndex, ParentBoneIndex, FoundBoneNode ? 1 : 0 , FMath::Frac(NjordPixelID));
				
					if ((FMath::Frac(NjordPixelID) <= 0.0001f || IsConnectingPoint || IsLastPoint))
					{
						FBoneNode BoneNode;
						BoneNode.BoneName = *("Bone_" + FString::FromInt(CurrentBoneIndex));
						BoneNode.BoneIndex = CurrentBoneIndex;
						BoneNode.PointIndex = PointIdx;
						BoneNode.ParentBoneIndex = ParentBoneIndex;
						BoneNode.BranchIndex = BranchIndex;
						BoneNode.NjordPixelID = NjordPixelID;
						FVector Position = FVector(PointPosition) - PreviousPosition;
						BoneNode.AbsolutePosition = PointPosition;
						BoneNode.BoneTransform = FTransform{Position};
						//Bones.Emplace(MoveTemp(BoneNode));
						ParentBoneIndex = CurrentBoneIndex;
						PreviousPosition = FVector(PointPosition);
						CurrentBoneIndex++;

						int AddedNodeIndex = ConnectingSegment.Nodes.Emplace(MoveTemp(BoneNode));
						if (IsConnectingPoint || IsLastPoint)
						{
							int AddedIndex = ConnectingSegments.Add(MoveTemp(ConnectingSegment));
							ConnectingSegment = FConnectingSegment();
						}
						//UE_LOGF(LogProceduralVegetation, Warning, "Creating Bone NjordPixelId, Id{%f} BoneId{%i} PointIndex{%i} ParentBoneIndex{%i} BranchID{%i}",
						//	BoneNode.NjordPixelID, BoneNode.BoneIndex, BoneNode.PointIndex,BoneNode.ParentBoneIndex,BoneNode.BranchIndex);
					}
				}
				ApplyBoneReduction(ConnectingSegments, Bones, ReductionStrength);
			}
		}

		ReindexBones(Bones);

		return true;
	}

	FBoneNode* FBoneFacade::FindClosestBone(const FManagedArrayCollection& Collection, TArray<FBoneNode>& Bones, const int32 PointID)
	{
		FBranchFacade BranchFacade(Collection);

		const auto NjordPixelIndexAttribute = PV::FPointNjordPixelIndexAttribute::FindAttribute(Collection);
		
		if (!NjordPixelIndexAttribute.IsValid())
		{
			UE_LOGF(LogProceduralVegetation, Warning, "BoneFacade::FindClosestBone called with invalid NjordPixelIndexAttribute.");
			return nullptr;
		}

		const float NjordValue = NjordPixelIndexAttribute.IsValidIndex(PointID) ? NjordPixelIndexAttribute[PointID] : -1;
		
		FBoneNode* BoneNode = Bones.FindByPredicate([&NjordValue](const FBoneNode& Node)
		{
			return FMath::FloorToInt(Node.NjordPixelID) == FMath::FloorToInt(NjordValue);
		});

		if (BoneNode)
		{
			return BoneNode;
		}

		int32 BranchIndex = BranchFacade.GetBranchIndexFromPointIndex(PointID);

		if (BranchIndex == -1)
		{
			UE_LOGF(LogProceduralVegetation, Warning, "Unable to find the branch for point index %i", PointID);
			return nullptr;
		}
		
		for (FBoneNode& Bone : Bones)
		{
			if (Bone.BranchIndex == BranchIndex && Bones.IsValidIndex(Bone.ParentBoneIndex))
			{
				FBoneNode ParentBone = Bones[Bone.ParentBoneIndex];
				if (NjordValue > ParentBone.NjordPixelID && NjordValue <= Bone.NjordPixelID)
				{
					return &Bone;
				}
			}
		}
		
		return BoneNode;
	}

	TArray<FBoneNode> FBoneFacade::GetBoneDataFromCollection() const
	{
		TArray<FBoneNode> Bones;
		
		if (!IsValid())
		{
			UE_LOGF(LogProceduralVegetation, Warning, "Trying to get bones with invalid BoneFacade.");
			return Bones;
		}

		const auto NjordPixelIndexAttribute = PV::FPointNjordPixelIndexAttribute::FindAttribute(BoneName.GetConstCollection());
		
		if (!NjordPixelIndexAttribute.IsValid())
		{
			UE_LOGF(LogProceduralVegetation, Warning, "BoneFacade::GetBoneDataFromCollection() called with invalid NjordPixelIndexAttribute.");
			return Bones;
		}

		for (int32 i = 0; i < BoneName.Num(); i++)
		{
			FBoneNode BoneNode;
			BoneNode.BoneName = FName(BoneName[i]);
			BoneNode.ParentBoneIndex = BoneParentIndex[i];
			BoneNode.PointIndex = BonePointIndex[i];
			BoneNode.BranchIndex = BoneBranchIndex[i];
			BoneNode.NjordPixelID = NjordPixelIndexAttribute.IsValidIndex(BoneNode.PointIndex) ? NjordPixelIndexAttribute[BoneNode.PointIndex] : -1;
			BoneNode.BoneTransform = BonePose[i];
			BoneNode.AbsolutePosition = FVector(BoneAbsolutePosition[i]);
			BoneNode.BoneIndex = BoneId[i];
			Bones.Add(BoneNode);
		}
		
		return Bones;
	}

	void FBoneFacade::SetBoneDataToCollection(TArray<FBoneNode>& Bones)
	{
		if (!IsValid())
		{
			UE_LOGF(LogProceduralVegetation, Warning, "Cannot Set Bones, FBoneFacade is not valid, Schema not defined.");
			return;
		}
		
		int32 NumElements = BoneName.AddElements(Bones.Num());
		
		int BoneIndex = 0;
		for (const auto& Bone : Bones)
		{
			BoneName.ModifyAt(BoneIndex,Bone.BoneName.ToString());
			BoneParentIndex.ModifyAt(BoneIndex,Bone.ParentBoneIndex);
			BonePointIndex.ModifyAt(BoneIndex,Bone.PointIndex);
			BoneBranchIndex.ModifyAt(BoneIndex,Bone.BranchIndex);
			BoneId.ModifyAt(BoneIndex,Bone.BoneIndex);
			BonePose.ModifyAt(BoneIndex,Bone.BoneTransform);
			BoneAbsolutePosition.ModifyAt(BoneIndex,FVector3f(Bone.AbsolutePosition));
			BoneIndex++;
		}
	}
}
