// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVBranchFacade.h"
#include "Facades/PVAttributesNames.h"
#include "Facades/PVFacadeCommon.h"

namespace PV::Facades
{
	FBranchFacade::FBranchFacade(FManagedArrayCollection& InCollection)
		: Collection(&InCollection)
		, Parents(InCollection, AttributeNames::BranchParents, GroupNames::BranchGroup)
		, Children(InCollection, AttributeNames::BranchChildren, GroupNames::BranchGroup)
		, BranchPoints(InCollection, AttributeNames::BranchPoints, GroupNames::BranchGroup)
		, BranchNumbers(InCollection, AttributeNames::BranchNumber, GroupNames::BranchGroup)
		, BranchSourceBudNumber(InCollection, AttributeNames::BranchSourceBudNumber, GroupNames::BranchGroup)
		, BranchFoliageIDs(InCollection, AttributeNames::BranchFoliageIDs, GroupNames::BranchGroup)
		, BranchUVMaterial(InCollection, AttributeNames::BranchUVMaterial, GroupNames::BranchGroup)
		, BranchHierarchyNumber(InCollection, AttributeNames::BranchHierarchyNumber, GroupNames::BranchGroup)
		, BranchSimulationGroupIndex(InCollection, AttributeNames::BranchSimulationGroupIndex, GroupNames::BranchGroup)
		, TrunkMaterialPathAttribute(InCollection, AttributeNames::TrunkMaterialPath, GroupNames::DetailsGroup)
		, TrunkURangeAttribute(InCollection, AttributeNames::TrunkURange, GroupNames::DetailsGroup)
		, BranchParentNumbers(InCollection, AttributeNames::BranchParentNumber, GroupNames::BranchGroup)
		, PlantNumbers(InCollection, AttributeNames::PlantNumber, GroupNames::BranchGroup)
	{
	}

	FBranchFacade::FBranchFacade(const FManagedArrayCollection& InCollection)
		: Collection(nullptr)
		, Parents(InCollection, AttributeNames::BranchParents, GroupNames::BranchGroup)
		, Children(InCollection, AttributeNames::BranchChildren, GroupNames::BranchGroup)
		, BranchPoints(InCollection, AttributeNames::BranchPoints, GroupNames::BranchGroup)
		, BranchNumbers(InCollection, AttributeNames::BranchNumber, GroupNames::BranchGroup)
		, BranchSourceBudNumber(InCollection, AttributeNames::BranchSourceBudNumber, GroupNames::BranchGroup)
		, BranchFoliageIDs(InCollection, AttributeNames::BranchFoliageIDs, GroupNames::BranchGroup)
		, BranchUVMaterial(InCollection, AttributeNames::BranchUVMaterial, GroupNames::BranchGroup)
		, BranchHierarchyNumber(InCollection, AttributeNames::BranchHierarchyNumber, GroupNames::BranchGroup)
		, BranchSimulationGroupIndex(InCollection, AttributeNames::BranchSimulationGroupIndex, GroupNames::BranchGroup)
		, TrunkMaterialPathAttribute(InCollection, AttributeNames::TrunkMaterialPath, GroupNames::DetailsGroup)
		, TrunkURangeAttribute(InCollection, AttributeNames::TrunkURange, GroupNames::DetailsGroup)
		, BranchParentNumbers(InCollection, AttributeNames::BranchParentNumber, GroupNames::BranchGroup)
		, PlantNumbers(InCollection, AttributeNames::PlantNumber, GroupNames::BranchGroup)
	{
	}

	bool FBranchFacade::IsValid() const
	{
		return Parents.IsValid()
			&& Children.IsValid()
			&& BranchPoints.IsValid()
			&& BranchNumbers.IsValid()
			&& BranchSourceBudNumber.IsValid()
			&& BranchHierarchyNumber.IsValid()
			&& BranchParentNumbers.IsValid()
			&& PlantNumbers.IsValid();
	}

	int32 FBranchFacade::GetElementCount() const
	{
		return Parents.Num();
	}

	const TArray<int32>& FBranchFacade::GetPoints(int32 Index) const
	{
		if (BranchPoints.IsValid() && BranchPoints.IsValidIndex(Index))
		{
			return BranchPoints[Index];
		}

		static const TArray<int32> EmptyPoints;
		return EmptyPoints;
	}

	int32 FBranchFacade::GetRootPoint(int32 Index) const
	{
		const TArray<int32>& Points = GetPoints(Index);
		return Points.Num()
			? Points[0]
			: INDEX_NONE;
	}

	void FBranchFacade::SetPoints(int32 Index, TArray<int32> InPoints)
	{
		if (BranchPoints.IsValid() && BranchPoints.IsValidIndex(Index))
		{
			BranchPoints.Modify()[Index] = MoveTemp(InPoints);
		}
	}

	const TArray<int32>& FBranchFacade::GetChildren(int32 Index) const
	{
		if (Children.IsValid() && Children.IsValidIndex(Index))
		{
			return Children[Index];
		}

		static const TArray<int32> EmptyChildren;
		return EmptyChildren;
	}

	TArray<int32> FBranchFacade::GetImmediateChildren(int32 Index) const
	{
		const int32 ParentBranchNumber = GetBranchNumber(Index);
		return GetChildren(Index).FilterByPredicate(
			[&, this](const int32 Child)
				{
					const int32 ChildIndex = GetBranchNumbers().Find(Child);
					if (ChildIndex != INDEX_NONE && GetParentBranchNumber(ChildIndex) == ParentBranchNumber)
					{
						return true;
					}
					return false;
				}
		);
	}

	void FBranchFacade::SetChildren(int32 Index, TArray<int32> InChildren)
	{
		if (Children.IsValid() && Children.IsValidIndex(Index))
		{
			Children.Modify()[Index] = MoveTemp(InChildren);
		}
	}

	const TArray<int32>& FBranchFacade::GetParents(int32 Index) const
	{
		if (Parents.IsValid() && Parents.IsValidIndex(Index))
		{
			return Parents[Index];
		}

		static const TArray<int32> EmptyParents;
		return EmptyParents;
	}

	int32 FBranchFacade::GetParentIndex(int32 BranchIndex) const
	{
		if (BranchParentNumbers.IsValid() && BranchParentNumbers.IsValidIndex(BranchIndex))
		{
			return BranchNumbers.Get().Find(BranchParentNumbers[BranchIndex]);
		}

		return INDEX_NONE;
	}

	int32 FBranchFacade::GetParentBranchNumber(int32 BranchIndex) const
	{
		if (BranchParentNumbers.IsValid() && BranchParentNumbers.IsValidIndex(BranchIndex))
		{
			return BranchParentNumbers[BranchIndex];
		}

		return INDEX_NONE;
	}

	void FBranchFacade::SetParentBranchNumber(int32 BranchIndex, int32 InParentBranchNumber)
	{
		if (BranchParentNumbers.IsValid() && BranchParentNumbers.IsValidIndex(BranchIndex))
		{
			BranchParentNumbers.ModifyAt(BranchIndex, InParentBranchNumber);
		}
	}

	const TManagedArray<int32>& FBranchFacade::GetParentBranchNumbers() const
	{
		return BranchParentNumbers.Get();
	}

	void FBranchFacade::SetParents(int32 Index, TArray<int32> InParents)
	{
		if (Parents.IsValid() && Parents.IsValidIndex(Index))
		{
			Parents.Modify()[Index] = MoveTemp(InParents);
		}
	}

	int32 FBranchFacade::GetBranchNumber(int32 Index) const
	{
		if (BranchNumbers.IsValid() && BranchNumbers.IsValidIndex(Index))
		{
			return BranchNumbers[Index];
		}

		return INDEX_NONE;
	}

	void FBranchFacade::SetBranchNumber(int32 Index, int32 InBranchNumber)
	{
		if (BranchNumbers.IsValid() && BranchNumbers.IsValidIndex(Index))
		{
			BranchNumbers.ModifyAt(Index, InBranchNumber);
		}
	}

	const TArray<int32>& FBranchFacade::GetBranchFoliageIDs(int32 Index) const
	{
		if (BranchFoliageIDs.IsValid() && BranchFoliageIDs.IsValidIndex(Index))
		{
			return BranchFoliageIDs[Index];
		}

		static const TArray<int32> EmptyChildren;
		return EmptyChildren;
	}

	void FBranchFacade::SetBranchFoliageIDs(int32 Index, TArray<int32> InBranchFoliageIDs)
	{
		if (BranchFoliageIDs.IsValid() && BranchFoliageIDs.IsValidIndex(Index))
		{
			BranchFoliageIDs.Modify()[Index] = MoveTemp(InBranchFoliageIDs);
		}
	}

	int32 FBranchFacade::GetBranchPlantNumber(int32 Index) const
	{
		if (PlantNumbers.IsValid() && PlantNumbers.IsValidIndex(Index))
		{
			return PlantNumbers[Index];
		}
		return INDEX_NONE;
	}

	void FBranchFacade::SetBranchPlantNumber(int32 Index, int32 InPlantNumber)
	{
		if (PlantNumbers.IsValid() && PlantNumbers.IsValidIndex(Index))
		{
			PlantNumbers.ModifyAt(Index, InPlantNumber);
		}
	}

	int32 FBranchFacade::GetBranchSourceBudNumber(int32 Index) const
	{
		if (BranchSourceBudNumber.IsValid() && BranchSourceBudNumber.IsValidIndex(Index))
		{
			return BranchSourceBudNumber[Index];
		}

		return INDEX_NONE;
	}

	void FBranchFacade::SetBranchSourceBudNumber(int32 Index, int32 InSourceBudNumber)
	{
		if (BranchSourceBudNumber.IsValid() && BranchSourceBudNumber.IsValidIndex(Index))
		{
			BranchSourceBudNumber.ModifyAt(Index, InSourceBudNumber);
		}
	}

	TArray<int32> FBranchFacade::GetParentBranchIndices(int32 BranchIndex) const
	{
		TArray<int32> ParentIndices;
		if (IsValid() && Parents.Num() > 0)
		{
			auto ParentsBranchNumbers = Parents[BranchIndex];
			for (auto Parent : ParentsBranchNumbers)
			{
				ParentIndices.Add(BranchNumbers.Get().Find(Parent));
			}
		}

		return ParentIndices;
	}

	int32 FBranchFacade::GetParentBranchIndex(int32 BranchIndex) const
	{
		if (IsValid() && Parents.IsValidIndex(BranchIndex))
		{
			if (TArray<int32> ParentsBranchNumbers = Parents[BranchIndex];
				ParentsBranchNumbers.Num() > 0)
			{
				const int32 ParentBranchNumber = ParentsBranchNumbers.Last();
				return BranchNumbers.Get().Find(ParentBranchNumber);
			}
		}

		return INDEX_NONE;
	}

	int32 FBranchFacade::GetHierarchyGenerationNumber(int32 Index) const
	{
		// The generation attributes contain incorrect information at the moment 
		// thus the number of parents can be used to derive the correct number
		if (Parents.IsValid() && Parents.IsValidIndex(Index))
		{
			return Parents[Index].Num();
		}

		return 0;
	}

	const TManagedArray<int32>& FBranchFacade::GetBranchNumbers() const
	{
		return BranchNumbers.Get();
	}

	int32 FBranchFacade::GetBranchUVMaterial(int32 Index) const
	{
		if (BranchUVMaterial.IsValid() && BranchUVMaterial.IsValidIndex(Index))
		{
			return BranchUVMaterial[Index];
		}

		return INDEX_NONE;
	}

	void FBranchFacade::SetBranchUVMaterial(int32 Index, int32 InMaterial)
	{
		if (!BranchUVMaterial.IsValid())
		{
			BranchUVMaterial.Add();
		}
		if (BranchUVMaterial.IsValid() && BranchUVMaterial.IsValidIndex(Index))
		{
			BranchUVMaterial.ModifyAt(Index, InMaterial);
		}
	}

	int32 FBranchFacade::GetBranchHierarchyNumber(int32 Index) const
	{
		if (BranchHierarchyNumber.IsValid() && BranchHierarchyNumber.IsValidIndex(Index))
		{
			return BranchHierarchyNumber[Index];
		}

		return INDEX_NONE;
	}

	const TManagedArray<int32>& FBranchFacade::GetBranchHierarchyNumbers() const
	{
		return BranchHierarchyNumber.Get();
	}

	void FBranchFacade::SetBranchHierarchyNumber(int32 Index, int32 InBranchHierarchyNumber)
	{
		if (BranchHierarchyNumber.IsValid() && BranchHierarchyNumber.IsValidIndex(Index))
		{
			BranchHierarchyNumber.ModifyAt(Index, InBranchHierarchyNumber);
		}
	}

	int32 FBranchFacade::GetBranchSimulationGroupIndex(int32 Index) const
	{
		if (BranchSimulationGroupIndex.IsValid() && BranchSimulationGroupIndex.IsValidIndex(Index))
		{
			return BranchSimulationGroupIndex[Index];
		}

		return INDEX_NONE;
	}

	void FBranchFacade::SetBranchSimulationGroupIndex(int32 Index, int32 InSimulationGroupIndex)
	{
		if (!BranchSimulationGroupIndex.IsValid())
		{
			BranchSimulationGroupIndex.Add();
		}
		if (BranchSimulationGroupIndex.IsValid() && BranchSimulationGroupIndex.IsValidIndex(Index))
		{
			BranchSimulationGroupIndex.ModifyAt(Index, InSimulationGroupIndex);
		}
	}

	void FBranchFacade::SetTrunkMaterialPath(const FString& InPath)
	{
		check(!IsConst());

		int32 NumElements = Collection->NumElements(GroupNames::DetailsGroup);

		if (NumElements == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		if (!TrunkMaterialPathAttribute.IsValid())
		{
			TrunkMaterialPathAttribute.Add();
		}

		TrunkMaterialPathAttribute.Modify()[0] = InPath;
	}

	FString FBranchFacade::GetTrunkMaterialPath() const
	{
		if (TrunkMaterialPathAttribute.IsValid() && TrunkMaterialPathAttribute.IsValidIndex(0))
		{
			return TrunkMaterialPathAttribute[0];
		}

		return FString();
	}

	void FBranchFacade::SetTrunkURange(TArray<FVector2f> InURange)
	{
		check(!IsConst());

		int32 NumElements = Collection->NumElements(GroupNames::DetailsGroup);

		if (NumElements == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		if (!TrunkURangeAttribute.IsValid())
		{
			TrunkURangeAttribute.Add();
		}

		TrunkURangeAttribute.Modify()[0] = MoveTemp(InURange);
	}

	const TArray<FVector2f>& FBranchFacade::GetTrunkURange() const
	{
		if (TrunkURangeAttribute.IsValid() && TrunkURangeAttribute.IsValidIndex(0))
		{
			return TrunkURangeAttribute[0];
		}

		static TArray<FVector2f> TrunkURange;
		return TrunkURange;
	}

	void FBranchFacade::CopyEntry(int32 FromIndex, int32 ToIndex)
	{
		if (IsValid() && BranchNumbers.IsValidIndex(FromIndex) && BranchNumbers.IsValidIndex(ToIndex))
		{
			Parents.ModifyAt(ToIndex, Parents[FromIndex]);
			Children.ModifyAt(ToIndex, Children[FromIndex]);
			BranchPoints.ModifyAt(ToIndex, BranchPoints[FromIndex]);
			BranchNumbers.ModifyAt(ToIndex, BranchNumbers[FromIndex]);
			BranchSourceBudNumber.ModifyAt(ToIndex, BranchSourceBudNumber[FromIndex]);
			BranchHierarchyNumber.ModifyAt(ToIndex, BranchHierarchyNumber[FromIndex]);
			BranchParentNumbers.ModifyAt(ToIndex, BranchParentNumbers[FromIndex]);
			PlantNumbers.ModifyAt(ToIndex, PlantNumbers[FromIndex]);
			if (BranchFoliageIDs.IsValid())
			{
				BranchFoliageIDs.ModifyAt(ToIndex, BranchFoliageIDs[FromIndex]);
			}
			if (BranchUVMaterial.IsValid())
			{
				BranchUVMaterial.ModifyAt(ToIndex, BranchUVMaterial[FromIndex]);
			}
			if (BranchSimulationGroupIndex.IsValid())
			{
				BranchSimulationGroupIndex.ModifyAt(ToIndex, BranchSimulationGroupIndex[FromIndex]);
			}
		}
	}

	void FBranchFacade::RemoveEntries(int32 NumEntries, int32 StartIndex)
	{
		if (IsValid() && BranchNumbers.IsValidIndex(StartIndex) && StartIndex + NumEntries <= BranchNumbers.Num())
		{
			BranchNumbers.RemoveElements(NumEntries, StartIndex);
		}
	}

	void FBranchFacade::GetSortedBranchIndicesByHierarchy(TArray<int32>& OutSortedIndices) const
	{
		for (int32 Index = 0; Index < GetElementCount(); Index++)
		{
			OutSortedIndices.Add(Index);
		}

		OutSortedIndices.Sort([&](int32 Index1, int32 Index2)
			{
				int32 Index1Hierarchy = GetHierarchyGenerationNumber(Index1);
				int32 Index2Hierarchy = GetHierarchyGenerationNumber(Index2);

				if (Index1Hierarchy == Index2Hierarchy)
				{
					return Index1 < Index2;
				}

				return Index1Hierarchy < Index2Hierarchy;
			});
	}

	int32 FBranchFacade::GetBranchIndexFromPointIndex(int32 PointIndex) const
	{
		for (int32 BranchIndex = 0; BranchIndex < GetElementCount(); BranchIndex++)
		{
			TArray<int32> PointInexes = GetPoints(BranchIndex);
			if (PointInexes.Contains(PointIndex))
			{
				return BranchIndex;
			}
		}

		return INDEX_NONE;
	}

	int32 FBranchFacade::AddElements(int NumElements)
	{
		if (!Parents.IsValid())
		{
			Parents.Add();
		}

		return Parents.AddElements(NumElements);
	}

	void FBranchFacade::FillParents(const TArray<TArray<int32>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(Parents, InputArray);
	}

	void FBranchFacade::RecomputeBranchChildren()
	{
		const int32 NumberOfElements = BranchParentNumbers.Num();
		TMap<int32, TArray<int32>> ImmediateChildren;
		ImmediateChildren.Reserve(NumberOfElements);

		for (int32 i = 0; i < NumberOfElements; i++)
		{
			const int32 BranchNumber = BranchNumbers[i];
			const int32 BranchParentNumber = BranchParentNumbers[i];

			if (BranchParentNumber != 0)
			[[likely]]
			{
				ImmediateChildren.FindOrAdd(BranchParentNumber).Add(BranchNumber);
			}
		}

		TMap<int32, TArray<int32>> Memo;
		Memo.Reserve(NumberOfElements);

		for (int32 i = 0; i < NumberOfElements; i++)
		{
			const int32 BranchNumber = BranchNumbers[i];
			TSet<int32> VisitingSet;
			Children.ModifyAt(i, ComputeDescendantsForBranchNumber(BranchNumber, ImmediateChildren, Memo, VisitingSet));
		}
	}

	TArray<int32> FBranchFacade::ComputeDescendantsForBranchNumber(
		const int32 BranchNumber,
		const TMap<int32, TArray<int32>>& ImmediateChildren,
		TMap<int32, TArray<int32>>& Memo,
		TSet<int32>& VisitingSet)
	{
		if (const TArray<int32>* Cached = Memo.Find(BranchNumber))
		{
			return *Cached;
		}

		// Cycle detected — return empty to break the recursion
		if (VisitingSet.Contains(BranchNumber))
		{
			return TArray<int32>();
		}

		const TArray<int32>* DirectChildren = ImmediateChildren.Find(BranchNumber);
		if (!DirectChildren)
		{
			Memo.Add(BranchNumber, TArray<int32>());
			return TArray<int32>();
		}

		VisitingSet.Add(BranchNumber);

		TSet<int32> DescendantSet(*DirectChildren);
		for (const int32 Child : *DirectChildren)
		{
			for (const int32 Descendant : ComputeDescendantsForBranchNumber(Child, ImmediateChildren, Memo, VisitingSet))
			{
				DescendantSet.Add(Descendant);
			}
		}
		
		VisitingSet.Remove(BranchNumber);

		TArray<int32> Descendants = DescendantSet.Array();
		Memo.Add(BranchNumber, Descendants);
		return Descendants;
	}

	TArray<int32> FBranchFacade::ComputeParentsForBranchNumber(const int32 BranchNumber, const TMap<int32, int32>& BranchNumbersToBranchParentNumbers,
	                                                           TMap<int32, TArray<int32>>& Memo, TSet<int32>& VisitedParents)
	{
		if (const TArray<int32>* Cached = Memo.Find(BranchNumber))
		{
			return *Cached;
		}
		
		if (VisitedParents.Contains(BranchNumber))
		{
			return TArray<int32>({0});
		}
		
		const int32* BranchParentNumberPtr = BranchNumbersToBranchParentNumbers.Find(BranchNumber);
		if (!BranchParentNumberPtr)
		{
			return TArray<int32>();
		}
		
		const int32 BranchParentNumber = *BranchParentNumberPtr;
		TArray<int32> Parents;
		if (BranchParentNumber == 0)
		{
			Parents = {0};
		}
		else
		{
			Parents = ComputeParentsForBranchNumber(BranchParentNumber, BranchNumbersToBranchParentNumbers, Memo, VisitedParents);
			Parents.Add(BranchParentNumber);
		}

		Memo.Add(BranchNumber, Parents);
		return Parents;
	}

	void FBranchFacade::RecomputeBranchParents()
	{
		const int32 NumberOfElements = BranchNumbers.Num();
		TMap<int32, int32> BranchNumbersToBranchParentNumbers;
		BranchNumbersToBranchParentNumbers.Reserve(NumberOfElements);

		for (int32 i = 0; i < NumberOfElements; i++)
		{
			BranchNumbersToBranchParentNumbers.Add(BranchNumbers[i], BranchParentNumbers[i]);
		}

		TMap<int32, TArray<int32>> Memo;
		Memo.Reserve(NumberOfElements);

		for (int32 i = 0; i < NumberOfElements; i++)
		{
			const int32 BranchNumber = BranchNumbers[i];
			TSet<int32> VisitedParents;
			const TArray<int32> ParentsForBranchNumber = ComputeParentsForBranchNumber(BranchNumber, BranchNumbersToBranchParentNumbers, Memo,
				VisitedParents);
			Parents.ModifyAt(i, ParentsForBranchNumber);
		}
	}

	void FBranchFacade::FillChildren(const TArray<TArray<int32>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(Children, InputArray);
	}

	void FBranchFacade::FillBranchPoints(const TArray<TArray<int32>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BranchPoints, InputArray);
	}

	void FBranchFacade::FillBranchNumbers(const TArray<int32>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BranchNumbers, InputArray);
	}

	void FBranchFacade::FillBranchSourceBudNumber(const TArray<int32>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BranchSourceBudNumber, InputArray);
	}

	void FBranchFacade::FillBranchHierarchyNumber(const TArray<int32>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BranchHierarchyNumber, InputArray);
	}

	void FBranchFacade::FillBranchParentNumbers(const TArray<int32>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BranchParentNumbers, InputArray);
	}

	void FBranchFacade::FillPlantNumbers(const TArray<int32>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(PlantNumbers, InputArray);
	}

	bool FBranchFacade::IsTrunk(int32 Index) const
	{
		if (BranchParentNumbers.IsValid() && BranchParentNumbers.IsValidIndex(Index))
		{
			return BranchParentNumbers[Index] == 0;
		}

		return false;
	}

	void FBranchFacade::GetParents(int32 Index, TArray<int32>& OutValue) const
	{
		GetValueFromAccessor<TArray<int32>>(Parents, OutValue, Index, TArray<int32>());
	}

	void FBranchFacade::GetChildren(int32 Index, TArray<int32>& OutValue) const
	{
		GetValueFromAccessor<TArray<int32>>(Children, OutValue, Index, TArray<int32>());
	}

	void FBranchFacade::GetBranchPoints(int32 Index, TArray<int32>& OutValue) const
	{
		GetValueFromAccessor<TArray<int32>>(BranchPoints, OutValue, Index, TArray<int32>());
	}

	void FBranchFacade::GetBranchNumber(int32 Index, int32& OutValue) const
	{
		GetValueFromAccessor<int32>(BranchNumbers, OutValue, Index, -1);
	}

	void FBranchFacade::GetBranchSourceBudNumber(int32 Index, int32& OutValue) const
	{
		GetValueFromAccessor<int32>(BranchSourceBudNumber, OutValue, Index, -1);
	}

	const TManagedArray<int32>& FBranchFacade::GetBranchSourceBudNumbers() const
	{
		return BranchSourceBudNumber.Get();
	}

	void FBranchFacade::SetBranchSourceBudNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BranchSourceBudNumber, InputArray, StartIndex);
	}

	void FBranchFacade::GetBranchHierarchyNumber(int32 Index, int32& OutValue) const
	{
		GetValueFromAccessor<int32>(BranchHierarchyNumber, OutValue, Index, -1);
	}

	void FBranchFacade::GetBranchParentNumber(int32 Index, int32& OutValue) const
	{
		GetValueFromAccessor<int32>(BranchParentNumbers, OutValue, Index, -1);
	}

	void FBranchFacade::GetPlantNumber(int32 Index, int32& OutValue) const
	{
		GetValueFromAccessor<int32>(PlantNumbers, OutValue, Index, -1);
	}

	void FBranchFacade::SetBranchNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BranchNumbers, InputArray, StartIndex);
	}

	void FBranchFacade::SetParentBranchNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BranchParentNumbers, InputArray, StartIndex);
	}

	void FBranchFacade::SetBranchHierarchyNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BranchHierarchyNumber, InputArray, StartIndex);
	}
}
