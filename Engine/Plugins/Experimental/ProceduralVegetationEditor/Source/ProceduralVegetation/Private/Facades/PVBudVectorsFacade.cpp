// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVBudVectorsFacade.h"

#include "Facades/PVAttributesNames.h"

namespace PV::Facades
{
	FBudVectorsFacade::FBudVectorsFacade(FManagedArrayCollection& InCollection)
			: BudDirection(InCollection, AttributeNames::BudDirection, GroupNames::PointGroup)
	{}

	FBudVectorsFacade::FBudVectorsFacade(const FManagedArrayCollection& InCollection)
		: BudDirection(InCollection, AttributeNames::BudDirection, GroupNames::PointGroup)
	{}

	bool FBudVectorsFacade::IsValid() const
	{
		return BudDirection.IsValid();
	}

	const TArray<FVector3f>& FBudVectorsFacade::GetBudDirection(int32 Index) const
	{
		if(BudDirection.IsValid() && BudDirection.IsValidIndex(Index))
		{
			return BudDirection[Index];
		}

		static const TArray<FVector3f> EmptyBudDirections;
		return EmptyBudDirections;
	}

	TManagedArray<TArray<FVector3f>>& FBudVectorsFacade::ModifyBudDirections()
	{
		return BudDirection.Modify();
	}

	const TManagedArray<TArray<FVector3f>>& FBudVectorsFacade::GetBudDirections() const
	{
		return BudDirection.Get();
	}
}
