// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace PV::Facades
{
	#define FILL_COLLECTION_ATTRIBUTE(Accessor, InputArray) \
	if (!Accessor.IsValid()) \
	{ \
	Accessor.Add(); \
	} \
	if (Accessor.Num() < InputArray.Num()) \
	{ \
	Accessor.AddElements(InputArray.Num() - Accessor.Num()); \
	} \
	for (int32 i = 0; i < InputArray.Num(); i++) \
	{ \
	Accessor.ModifyAt(i, InputArray[i]); \
	}
	
	#define FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(Accessor, InputArray, StartIndex) \
	if (Accessor.Num() - StartIndex < InputArray.Num()) \
	{ \
		Accessor.AddElements(InputArray.Num() - (Accessor.Num() - StartIndex)); \
	} \
	for (int32 i = 0; i < InputArray.Num(); i++) \
	{ \
		Accessor.ModifyAt(i + StartIndex, InputArray[i]); \
	}
}
