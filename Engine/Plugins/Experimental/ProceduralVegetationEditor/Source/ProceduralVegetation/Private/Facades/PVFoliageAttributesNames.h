// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace PV::GroupNames
{
	inline static const FName FoliageConditionGroup = FName(TEXT("FoliageConditions"));
}

namespace PV::FoliageAttributeNames
{

	//Foliage Attributes
	inline static const FName FoliageNameID = FName(TEXT("FoliageNameID"));
	inline static const FName FoliageBranchID = FName(TEXT("FoliageBranchID"));
	inline static const FName FoliagePivotPoint = FName(TEXT("FoliagePivotPoint"));
	inline static const FName FoliageUPVector = FName(TEXT("FoliageUPVector"));
	inline static const FName FoliageNormalVector = FName(TEXT("FoliageNormalVector"));
	inline static const FName FoliageScale = FName(TEXT("FoliageScale"));
	inline static const FName FoliageLengthFromRoot = FName(TEXT("LengthFromRoot"));
	inline static const FName FoliageConditionLight = FName(TEXT("ConditionLight"));
	inline static const FName FoliageConditionTip = FName(TEXT("ConditionTip"));
	inline static const FName FoliageConditionScale = FName(TEXT("ConditionScale"));
	inline static const FName FoliageConditionUpAlignment = FName(TEXT("ConditionUpAlignment"));
	inline static const FName FoliageConditionHealth = FName(TEXT("ConditionHealth"));
	inline static const FName FoliageConditionHeight = FName(TEXT("ConditionHeight"));
	inline static const FName FoliageConditionGeneration = FName(TEXT("ConditionGeneration"));
	inline static const FName FoliageParentBoneID = FName(TEXT("FoliageParentBoneID"));

	//FoliageInfo Attributes
	inline static const FName FoliageUseAsMask = FName(TEXT("UseAsMask"));
	inline static const FName FoliageName = FName(TEXT("Name"));
	inline static const FName FoliageAttributeLight = FName(TEXT("Light"));
	inline static const FName FoliageAttributeScale = FName(TEXT("Scale"));
	inline static const FName FoliageAttributeTip = FName(TEXT("Tip"));
	inline static const FName FoliageAttributeUpAlignment = FName(TEXT("UpAlignment"));
	inline static const FName FoliageAttributeHealth = FName(TEXT("Health"));
	inline static const FName FoliageAttributeHeight = FName(TEXT("Height"));
	inline static const FName FoliageAttributeGeneration = FName(TEXT("Generation"));

	// Condition
	inline static const FName ConditionName = FName(TEXT("Name"));
	inline static const FName ConditionWeight = FName(TEXT("Weight"));
	inline static const FName ConditionOffset = FName(TEXT("Offset"));
}

