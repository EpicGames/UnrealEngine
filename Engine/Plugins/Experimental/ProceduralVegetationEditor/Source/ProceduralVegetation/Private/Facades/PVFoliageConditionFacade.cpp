// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVFoliageConditionFacade.h"
#include "PVFoliageAttributesNames.h"

namespace PV::Facades
{
	FFoliageConditionFacade::FFoliageConditionFacade(FManagedArrayCollection& InCollection)
		: NameAttribute(InCollection, FoliageAttributeNames::ConditionName, GroupNames::FoliageConditionGroup)
		, WeightAttribute(InCollection, FoliageAttributeNames::ConditionWeight, GroupNames::FoliageConditionGroup)
		, OffsetAttribute(InCollection, FoliageAttributeNames::ConditionOffset, GroupNames::FoliageConditionGroup)
	{
		DefineSchema(InCollection);
	}

	FFoliageConditionFacade::FFoliageConditionFacade(const FManagedArrayCollection& InCollection)
	: NameAttribute(InCollection, FoliageAttributeNames::ConditionName, GroupNames::FoliageConditionGroup)
	, WeightAttribute(InCollection, FoliageAttributeNames::ConditionWeight, GroupNames::FoliageConditionGroup)
	, OffsetAttribute(InCollection, FoliageAttributeNames::ConditionOffset, GroupNames::FoliageConditionGroup)
	{
	}

	void FFoliageConditionFacade::DefineSchema(FManagedArrayCollection& InCollection)
	{
		if (!InCollection.HasGroup(GroupNames::FoliageConditionGroup))
		{
			InCollection.AddGroup(GroupNames::FoliageConditionGroup);
		}

		NameAttribute.Add();
		WeightAttribute.Add();
		OffsetAttribute.Add();
	}

	bool FFoliageConditionFacade::IsValid() const
	{
		return NameAttribute.IsValid() && WeightAttribute.IsValid() && OffsetAttribute.IsValid();
	}

	FFoliageConditonInfo FFoliageConditionFacade::GetEntry(const int32 Index) const
	{
		FFoliageConditonInfo ReturnData = {};
		if (IsValid() && Index > -1)
		{
			if (NameAttribute.IsValidIndex(Index))
			{
				ReturnData.Name = NameAttribute.Get()[Index];
			}
			if (WeightAttribute.IsValidIndex(Index))
			{
				ReturnData.Weight = WeightAttribute.Get()[Index];
			}
			if (OffsetAttribute.IsValidIndex(Index))
			{
				ReturnData.Offset = OffsetAttribute.Get()[Index];
			}
		}
		return ReturnData;
	}

	int32 FFoliageConditionFacade::Add(const FFoliageConditonInfo& InputData)
	{
		if (IsValid())
		{
			const int32 NewIndex = NameAttribute.AddElements(1);
			NameAttribute.Modify()[NewIndex] = InputData.Name;
			WeightAttribute.Modify()[NewIndex] = InputData.Weight;
			OffsetAttribute.Modify()[NewIndex] = InputData.Offset;

			return NewIndex;
		}
		return INDEX_NONE;
	}

	void FFoliageConditionFacade::Set(const int32 Index, const FFoliageConditonInfo& InputData)
	{
		if (IsValid() && Index >= 0)
		{
			NameAttribute.ModifyAt(Index, InputData.Name);
			WeightAttribute.ModifyAt(Index, InputData.Weight);
			OffsetAttribute.ModifyAt(Index, InputData.Offset);
		}
	}

	TArray<FFoliageConditonInfo> FFoliageConditionFacade::GetData() const
	{
		TArray<FFoliageConditonInfo> Infos;
		
		if (NameAttribute.IsValid())
		{
			for (int32 i = 0; i < NameAttribute.Num(); i++)
			{
				Infos.Add(GetEntry(i));
			}
		}

		return Infos;
	}

	void FFoliageConditionFacade::SetData(const TArray<FFoliageConditonInfo>& Infos)
	{
		int32 NumElements = NameAttribute.AddElements(Infos.Num());
		
		for (int32 Index = 0; Index < Infos.Num(); ++Index)
		{
			FFoliageConditonInfo Info = Infos[Index];

			Set(Index, Info);
		}
	}
}
