// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "DataTypes/PVDistributionParams.h"
#include "Implementations/PVFoliage.h"

namespace PV::Facades
{
	struct FFoliageConditonInfo
	{
		FString Name;
		float Weight;
		float Offset;

		EPVDistributionCondition GetType() const
		{
			const UEnum* EnumPtr = StaticEnum<EPVDistributionCondition>();

			int64 EnumValue = EnumPtr->GetValueByNameString(Name);

			if (EnumValue == INDEX_NONE)
				return EPVDistributionCondition::None;

			return static_cast<EPVDistributionCondition>(EnumValue);
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("Name = %s, Weight = %f, Reduction = %f"), *Name, Weight, Offset);
		}
	};

	class FFoliageConditionFacade
	{
	public:
		FFoliageConditionFacade(FManagedArrayCollection& InCollection);
		FFoliageConditionFacade(const FManagedArrayCollection& InCollection);

		bool IsValid() const;

		int32 NumEntries() const { return NameAttribute.Num(); };
		
		int32 Add(const FFoliageConditonInfo& InputData);

		void Set(const int32 Index, const FFoliageConditonInfo& InputData);
		void SetData(const TArray<FFoliageConditonInfo>& Infos);
		
		FFoliageConditonInfo GetEntry(const int32 Index) const;

		TArray<FFoliageConditonInfo> GetData() const;

	protected:
		void DefineSchema(FManagedArrayCollection& InCollection);
		
		TManagedArrayAccessor<FString> NameAttribute;
		TManagedArrayAccessor<float> WeightAttribute;
		TManagedArrayAccessor<float> OffsetAttribute;
	};
}
