// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Facades/PVFoliageFacade.h"
 
#include "ProceduralVegetationModule.h"
#include "PVFacadeCommon.h"
#include "PVFoliageAttributesNames.h"
#include "VisualizeTexture.h"
#include "Chaos/Deformable/MuscleActivationConstraints.h"
#include "Facades/PVAttributesNames.h"
#include "Implementations/PVFoliage.h"
 
namespace PV::Facades
{
	FFoliageFacade::FFoliageFacade(FManagedArrayCollection& InCollection, int32 InitialSize)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, FoliageInfoUseAsMaskAttribute(InCollection, FoliageAttributeNames::FoliageUseAsMask, GroupNames::FoliageNamesGroup)
		, FoliageInfoNamesAttribute(InCollection, FoliageAttributeNames::FoliageName, GroupNames::FoliageNamesGroup)
		, FoliageInfoLightAttribute(InCollection, FoliageAttributeNames::FoliageAttributeLight, GroupNames::FoliageNamesGroup)
		, FoliageInfoScaleAttribute(InCollection, FoliageAttributeNames::FoliageAttributeScale, GroupNames::FoliageNamesGroup)
		, FoliageInfoTipAttribute(InCollection, FoliageAttributeNames::FoliageAttributeTip, GroupNames::FoliageNamesGroup)
		, FoliageInfoUpAlignmentAttribute(InCollection, FoliageAttributeNames::FoliageAttributeUpAlignment, GroupNames::FoliageNamesGroup)
		, FoliageInfoHealthAttribute(InCollection, FoliageAttributeNames::FoliageAttributeHealth, GroupNames::FoliageNamesGroup)
		, FoliageInfoHeightAttribute(InCollection, FoliageAttributeNames::FoliageAttributeHeight, GroupNames::FoliageNamesGroup)
		, FoliageInfoGenerationAttribute(InCollection, FoliageAttributeNames::FoliageAttributeGeneration, GroupNames::FoliageNamesGroup)
		, NameIdsAttribute(InCollection, FoliageAttributeNames::FoliageNameID, GroupNames::FoliageGroup)
		, BranchIdsAttribute(InCollection, FoliageAttributeNames::FoliageBranchID, GroupNames::FoliageGroup)
		, PivotPointsAttribute(InCollection, FoliageAttributeNames::FoliagePivotPoint, GroupNames::FoliageGroup)
		, UpVectorsAttribute(InCollection, FoliageAttributeNames::FoliageUPVector, GroupNames::FoliageGroup)
		, NormalVectorsAttribute(InCollection, FoliageAttributeNames::FoliageNormalVector, GroupNames::FoliageGroup)
		, ScalesAttribute(InCollection, FoliageAttributeNames::FoliageScale, GroupNames::FoliageGroup)
		, LengthFromRootAttribute(InCollection, FoliageAttributeNames::FoliageLengthFromRoot, GroupNames::FoliageGroup)
		, ConditionLightAttribute(InCollection, FoliageAttributeNames::FoliageConditionLight, GroupNames::FoliageGroup)
		, ConditionScaleAttribute(InCollection, FoliageAttributeNames::FoliageConditionScale, GroupNames::FoliageGroup)
		, ConditionTipAttribute(InCollection, FoliageAttributeNames::FoliageConditionTip, GroupNames::FoliageGroup)
		, ConditionUpAlignmentAttribute(InCollection, FoliageAttributeNames::FoliageConditionUpAlignment, GroupNames::FoliageGroup)
		, ConditionHealthAttribute(InCollection, FoliageAttributeNames::FoliageConditionHealth, GroupNames::FoliageGroup)
		, ConditionHeightAttribute(InCollection, FoliageAttributeNames::FoliageConditionHeight, GroupNames::FoliageGroup)
		, ConditionGenerationAttribute(InCollection, FoliageAttributeNames::FoliageConditionGeneration, GroupNames::FoliageGroup)
		, ParentBoneIdsAttribute(InCollection, FoliageAttributeNames::FoliageParentBoneID, GroupNames::FoliageGroup)
		, FoliageIdsAttribute(InCollection, AttributeNames::BranchFoliageIDs, GroupNames::BranchGroup)
		, FoliagePathAttribute(InCollection, AttributeNames::FoliagePath, GroupNames::DetailsGroup)
	{
		DefineSchema(InitialSize);
	}
 
	FFoliageFacade::FFoliageFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, FoliageInfoUseAsMaskAttribute(InCollection, FoliageAttributeNames::FoliageUseAsMask, GroupNames::FoliageNamesGroup)
		, FoliageInfoNamesAttribute(InCollection, FoliageAttributeNames::FoliageName, GroupNames::FoliageNamesGroup)
		, FoliageInfoLightAttribute(InCollection, FoliageAttributeNames::FoliageAttributeLight, GroupNames::FoliageNamesGroup)
		, FoliageInfoScaleAttribute(InCollection, FoliageAttributeNames::FoliageAttributeScale, GroupNames::FoliageNamesGroup)
		, FoliageInfoTipAttribute(InCollection, FoliageAttributeNames::FoliageAttributeTip, GroupNames::FoliageNamesGroup)
		, FoliageInfoUpAlignmentAttribute(InCollection, FoliageAttributeNames::FoliageAttributeUpAlignment, GroupNames::FoliageNamesGroup)
		, FoliageInfoHealthAttribute(InCollection, FoliageAttributeNames::FoliageAttributeHealth, GroupNames::FoliageNamesGroup)
		, FoliageInfoHeightAttribute(InCollection, FoliageAttributeNames::FoliageAttributeHeight, GroupNames::FoliageNamesGroup)
		, FoliageInfoGenerationAttribute(InCollection, FoliageAttributeNames::FoliageAttributeGeneration, GroupNames::FoliageNamesGroup)
		, NameIdsAttribute(InCollection, FoliageAttributeNames::FoliageNameID, GroupNames::FoliageGroup)
		, BranchIdsAttribute(InCollection, FoliageAttributeNames::FoliageBranchID, GroupNames::FoliageGroup)
		, PivotPointsAttribute(InCollection, FoliageAttributeNames::FoliagePivotPoint, GroupNames::FoliageGroup)
		, UpVectorsAttribute(InCollection, FoliageAttributeNames::FoliageUPVector, GroupNames::FoliageGroup)
		, NormalVectorsAttribute(InCollection, FoliageAttributeNames::FoliageNormalVector, GroupNames::FoliageGroup)
		, ScalesAttribute(InCollection, FoliageAttributeNames::FoliageScale, GroupNames::FoliageGroup)
		, LengthFromRootAttribute(InCollection, FoliageAttributeNames::FoliageLengthFromRoot, GroupNames::FoliageGroup)
		, ConditionLightAttribute(InCollection, FoliageAttributeNames::FoliageConditionLight, GroupNames::FoliageGroup)
		, ConditionScaleAttribute(InCollection, FoliageAttributeNames::FoliageConditionScale, GroupNames::FoliageGroup)
		, ConditionTipAttribute(InCollection, FoliageAttributeNames::FoliageConditionTip, GroupNames::FoliageGroup)
		, ConditionUpAlignmentAttribute(InCollection, FoliageAttributeNames::FoliageConditionUpAlignment, GroupNames::FoliageGroup)
		, ConditionHealthAttribute(InCollection, FoliageAttributeNames::FoliageConditionHealth, GroupNames::FoliageGroup)
		, ConditionHeightAttribute(InCollection, FoliageAttributeNames::FoliageConditionHeight, GroupNames::FoliageGroup)
		, ConditionGenerationAttribute(InCollection, FoliageAttributeNames::FoliageConditionGeneration, GroupNames::FoliageGroup)
		, ParentBoneIdsAttribute(InCollection, FoliageAttributeNames::FoliageParentBoneID, GroupNames::FoliageGroup)
		, FoliageIdsAttribute(InCollection, AttributeNames::BranchFoliageIDs, GroupNames::BranchGroup)
		, FoliagePathAttribute(InCollection, AttributeNames::FoliagePath, GroupNames::DetailsGroup)
	{
	}
 
	void FFoliageFacade::DefineSchema(int32 InitialSize)
	{
		check(!IsConst());
 
		if (!Collection->HasGroup(GroupNames::FoliageGroup))
		{
			Collection->AddGroup(GroupNames::FoliageGroup);
			if (InitialSize > 0)
			{
				Collection->AddElements(InitialSize, GroupNames::FoliageGroup);
			}
		}
 
		if (!Collection->HasGroup(GroupNames::FoliageNamesGroup))
		{
			Collection->AddGroup(GroupNames::FoliageNamesGroup);
		}
 
		if (!Collection->HasGroup(GroupNames::DetailsGroup))
		{
			Collection->AddGroup(GroupNames::DetailsGroup);
		}
 
		FoliageInfoUseAsMaskAttribute.Add();
		FoliageInfoNamesAttribute.Add();
		FoliageInfoLightAttribute.Add();
		FoliageInfoScaleAttribute.Add();
		FoliageInfoTipAttribute.Add();
		FoliageInfoUpAlignmentAttribute.Add();
		FoliageInfoHealthAttribute.Add();
		FoliageInfoHeightAttribute.Add();
		FoliageInfoGenerationAttribute.Add();
		NameIdsAttribute.Add();
		BranchIdsAttribute.Add();
		PivotPointsAttribute.Add();
		UpVectorsAttribute.Add();
		NormalVectorsAttribute.Add();
		ScalesAttribute.Add();
		LengthFromRootAttribute.Add();
		ConditionLightAttribute.Add();
		ConditionTipAttribute.Add();
		ConditionUpAlignmentAttribute.Add();
		ConditionHealthAttribute.Add();
		ConditionScaleAttribute.Add();
		ConditionHeightAttribute.Add();
		ConditionGenerationAttribute.Add();
		ParentBoneIdsAttribute.Add();
		FoliageIdsAttribute.Add();
		FoliagePathAttribute.Add();
	}
 
	bool FFoliageFacade::IsValid() const
	{
		return FoliageInfoNamesAttribute.IsValid() && FoliageInfoTipAttribute.IsValid() && FoliageInfoUpAlignmentAttribute.IsValid() && 
			FoliageInfoHealthAttribute.IsValid() && FoliageInfoLightAttribute.IsValid() && FoliageInfoScaleAttribute.IsValid() && 
			FoliageInfoHeightAttribute.IsValid() && FoliageInfoGenerationAttribute.IsValid() &&
			NameIdsAttribute.IsValid() && BranchIdsAttribute.IsValid() && PivotPointsAttribute.IsValid() &&
			UpVectorsAttribute.IsValid() && NormalVectorsAttribute.IsValid() && ScalesAttribute.IsValid() && LengthFromRootAttribute.IsValid() &&
			FoliageIdsAttribute.IsValid() && ParentBoneIdsAttribute.IsValid() && ConditionTipAttribute.IsValid() && ConditionLightAttribute.IsValid() && 
			ConditionUpAlignmentAttribute.IsValid() && ConditionHealthAttribute.IsValid() && ConditionScaleAttribute.IsValid() && 
			ConditionHeightAttribute.IsValid() && ConditionGenerationAttribute.IsValid();
	}
 
	FFoliageEntryData FFoliageFacade::GetFoliageEntry(int32 Index) const
	{
		FFoliageEntryData ReturnData = {};
		if (IsValid() && Index > -1)
		{
			if (NameIdsAttribute.IsValidIndex(Index))
			{
				ReturnData.NameId = NameIdsAttribute.Get()[Index];
			}
			if (BranchIdsAttribute.IsValidIndex(Index))
			{
				ReturnData.BranchId = BranchIdsAttribute.Get()[Index];
			}
			if (PivotPointsAttribute.IsValidIndex(Index))
			{
				ReturnData.PivotPoint = PivotPointsAttribute.Get()[Index];
			}
			if (UpVectorsAttribute.IsValidIndex(Index))
			{
				ReturnData.UpVector = UpVectorsAttribute.Get()[Index];
			}
			if (NormalVectorsAttribute.IsValidIndex(Index))
			{
				ReturnData.NormalVector = NormalVectorsAttribute.Get()[Index];
			}
			if (ScalesAttribute.IsValidIndex(Index))
			{
				ReturnData.Scale = ScalesAttribute.Get()[Index];
			}
			if (LengthFromRootAttribute.IsValidIndex(Index))
			{
				ReturnData.LengthFromRoot = LengthFromRootAttribute.Get()[Index];
			}
 
			if (ConditionLightAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionLight = ConditionLightAttribute.Get()[Index];
			}
 
			if (ConditionScaleAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionScale = ConditionScaleAttribute.Get()[Index];
			}
 
			if (ConditionTipAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionTip = ConditionTipAttribute.Get()[Index];
			}
 
			if (ConditionUpAlignmentAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionUpAlignment = ConditionUpAlignmentAttribute.Get()[Index];
			}
 
			if (ConditionHealthAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionHealth = ConditionHealthAttribute.Get()[Index];
			}

			if (ConditionHeightAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionHeight = ConditionHeightAttribute.Get()[Index];
			}

			if (ConditionGenerationAttribute.IsValidIndex(Index))
			{
				ReturnData.ConditionGeneration = ConditionGenerationAttribute.Get()[Index];
			}

			if (ParentBoneIdsAttribute.IsValidIndex(Index))
			{
				ReturnData.ParentBoneID = ParentBoneIdsAttribute.Get()[Index];
			}
		}
		return ReturnData;
	}
 
	float FFoliageFacade::GetLengthFromRoot(int32 Index) const
	{
		if (LengthFromRootAttribute.IsValid() && LengthFromRootAttribute.IsValidIndex(Index))
		{
			return LengthFromRootAttribute[Index];
		}
 
		static constexpr float DefaultLengthFromRoot = 0.0;
		return DefaultLengthFromRoot;
	}
 
	void FFoliageFacade::SetLengthFromRoot(int32 Index, float Input)
	{
		check(!IsConst());
		if (IsValid() && LengthFromRootAttribute.IsValidIndex(Index))
		{
			LengthFromRootAttribute.ModifyAt(Index, Input);
		}
	}
 
	int32 FFoliageFacade::GetParentBoneID(int32 Index) const
	{
		if (ParentBoneIdsAttribute.IsValid() && ParentBoneIdsAttribute.IsValidIndex(Index))
		{
			return ParentBoneIdsAttribute[Index];
		}
 
		return INDEX_NONE;
	}
 
	void FFoliageFacade::SetParentBoneID(int32 Index, int32 Input)
	{
		check(!IsConst());
		if (IsValid() && ParentBoneIdsAttribute.IsValidIndex(Index))
		{
			ParentBoneIdsAttribute.ModifyAt(Index, Input);
		}
	}
 
	const TArray<int32>& FFoliageFacade::GetFoliageEntryIdsForBranch(int32 Index) const
	{
		if (FoliageIdsAttribute.IsValid() && FoliageIdsAttribute.IsValidIndex(Index))
		{
			return FoliageIdsAttribute[Index];
		}
 
		static const TArray<int32> EmptyArray;
		return EmptyArray;
	}
 
	int32 FFoliageFacade::AddFoliageEntry(const FFoliageEntryData& InputData)
	{
		check(!IsConst());
		if (IsValid())
		{
			const int32 NewIndex = NameIdsAttribute.AddElements(1);
			NameIdsAttribute.Modify()[NewIndex] = InputData.NameId;
			BranchIdsAttribute.Modify()[NewIndex] = InputData.BranchId;
			PivotPointsAttribute.Modify()[NewIndex] = InputData.PivotPoint;
			UpVectorsAttribute.Modify()[NewIndex] = InputData.UpVector;
			NormalVectorsAttribute.Modify()[NewIndex] = InputData.NormalVector;
			ScalesAttribute.Modify()[NewIndex] = InputData.Scale;
			LengthFromRootAttribute.Modify()[NewIndex] = InputData.LengthFromRoot;
			ConditionLightAttribute.Modify()[NewIndex] = InputData.ConditionLight;
			ConditionScaleAttribute.Modify()[NewIndex] = InputData.ConditionScale;
			ConditionUpAlignmentAttribute.Modify()[NewIndex] = InputData.ConditionUpAlignment;
			ConditionHealthAttribute.Modify()[NewIndex] = InputData.ConditionHealth;
			ConditionTipAttribute.Modify()[NewIndex] = InputData.ConditionTip;
			ConditionHeightAttribute.Modify()[NewIndex] = InputData.ConditionHeight;
			ConditionGenerationAttribute.Modify()[NewIndex] = InputData.ConditionGeneration;

			ParentBoneIdsAttribute.Modify()[NewIndex] = InputData.ParentBoneID;
 
			return NewIndex;
		}
		return INDEX_NONE;
	}
 
	void FFoliageFacade::SetFoliageEntry(int32 Index, const FFoliageEntryData& InputData)
	{
		check(!IsConst());
		if (IsValid() && Index >= 0)
		{
			NameIdsAttribute.ModifyAt(Index, InputData.NameId);
			BranchIdsAttribute.ModifyAt(Index, InputData.BranchId);
			PivotPointsAttribute.ModifyAt(Index, InputData.PivotPoint);
			UpVectorsAttribute.ModifyAt(Index, InputData.UpVector);
			NormalVectorsAttribute.ModifyAt(Index, InputData.NormalVector);
			ScalesAttribute.ModifyAt(Index, InputData.Scale);
			LengthFromRootAttribute.ModifyAt(Index, InputData.LengthFromRoot);
			ConditionTipAttribute.ModifyAt(Index, InputData.ConditionTip);
			ConditionScaleAttribute.ModifyAt(Index, InputData.ConditionScale);
			ConditionLightAttribute.ModifyAt(Index, InputData.ConditionLight);
			ConditionUpAlignmentAttribute.ModifyAt(Index, InputData.ConditionUpAlignment);
			ConditionHealthAttribute.ModifyAt(Index, InputData.ConditionHealth);
			ConditionHeightAttribute.ModifyAt(Index, InputData.ConditionHeight);
			ConditionGenerationAttribute.ModifyAt(Index, InputData.ConditionGeneration);
			ParentBoneIdsAttribute.ModifyAt(Index, InputData.ParentBoneID);
		}
	}
 
	void FFoliageFacade::SetFoliageIdsArray(int32 Index, TArray<int32> InputIds)
	{
		check(!IsConst());
		if (IsValid() && FoliageIdsAttribute.IsValidIndex(Index))
		{
			FoliageIdsAttribute.Modify()[Index] = MoveTemp(InputIds);
		}
	}
 
	int32 FFoliageFacade::GetFoliageBranchId(int32 Index) const
	{
		if (BranchIdsAttribute.IsValid() && BranchIdsAttribute.IsValidIndex(Index))
		{
			return BranchIdsAttribute[Index];
		}
		return INDEX_NONE;
	}
 
	void FFoliageFacade::SetFoliageBranchId(int32 Index, int32 InputId)
	{
		check(!IsConst());
		if (IsValid() && BranchIdsAttribute.IsValidIndex(Index))
		{
			BranchIdsAttribute.ModifyAt(Index, InputId);
		}
	}

	bool FFoliageFacade::IsMaskFoliageEntry(const int32 Index) const
	{
		if (FoliageInfoUseAsMaskAttribute.IsValid() && FoliageInfoUseAsMaskAttribute.IsValidIndex(Index))
		{
			return FoliageInfoUseAsMaskAttribute.Get()[Index];
		}
		return false;
	}

	void FFoliageFacade::SetMaskFoliageEntry(const int32 Index, const bool InUseAsMask)
	{
		check(!IsConst());
		if (IsValid() && FoliageInfoUseAsMaskAttribute.IsValidIndex(Index))
		{
			FoliageInfoUseAsMaskAttribute.ModifyAt(Index, InUseAsMask);
		}
	}

	FPVFoliageInfo FFoliageFacade::GetFoliageInfo(const int32 Index) const
	{
		FPVFoliageInfo ReturnData = {};
		
		if (IsValid() && Index > -1)
		{
			if (FoliageInfoUseAsMaskAttribute.IsValidIndex(Index))
			{
				ReturnData.bUseAsMask = FoliageInfoUseAsMaskAttribute.Get()[Index];
			}
			
			if (FoliageInfoNamesAttribute.IsValidIndex(Index))
			{
				ReturnData.Mesh = FoliageInfoNamesAttribute.Get()[Index];
			}
 
			if (FoliageInfoLightAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.Light = FoliageInfoLightAttribute.Get()[Index];
			}
 
			if (FoliageInfoScaleAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.Scale = FoliageInfoScaleAttribute.Get()[Index];
			}
 
			if (FoliageInfoTipAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.Tip = (bool)FoliageInfoTipAttribute.Get()[Index];
			}
 
			if (FoliageInfoHealthAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.Health = FoliageInfoHealthAttribute.Get()[Index];
			}
 
			if (FoliageInfoUpAlignmentAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.UpAlignment = FoliageInfoUpAlignmentAttribute.Get()[Index];
			}

			if (FoliageInfoHeightAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.Height = FoliageInfoHeightAttribute.Get()[Index];
			}

			if (FoliageInfoGenerationAttribute.IsValidIndex(Index))
			{
				ReturnData.Attributes.Generation = FoliageInfoGenerationAttribute.Get()[Index];
			}
		}
		return ReturnData;
	}
 
	TArray<FPVFoliageInfo> FFoliageFacade::GetFoliageInfos() const
	{
		TArray<FPVFoliageInfo> FoliageInfos;
		
		if (FoliageInfoNamesAttribute.IsValid())
		{
			for (int32 i = 0; i < FoliageInfoNamesAttribute.Num(); i++)
			{
				FoliageInfos.Add(GetFoliageInfo(i));
			}
		}
		return FoliageInfos;
	}
 
	const FVector3f& FFoliageFacade::GetPivotPoint(int32 Index) const
	{
		if (PivotPointsAttribute.IsValid() && PivotPointsAttribute.IsValidIndex(Index))
		{
			return PivotPointsAttribute[Index];
		}
 
		static const FVector3f DefaultPivotPoint = FVector3f::Zero();
		return DefaultPivotPoint;
	}
 
	const FVector3f& FFoliageFacade::GetUpVector(const int32 Index) const
	{
		if (UpVectorsAttribute.IsValid() && UpVectorsAttribute.IsValidIndex(Index))
		{
			return UpVectorsAttribute[Index];
		}
 
		static const FVector3f DefaultUpVector = FVector3f::Zero();
		return DefaultUpVector;
	}
 
	const FVector3f& FFoliageFacade::GetNormalVector(const int32 Index) const
	{
		if (NormalVectorsAttribute.IsValid() && NormalVectorsAttribute.IsValidIndex(Index))
		{
			return NormalVectorsAttribute[Index];
		}
 
		static const FVector3f DefaultNormalVector = FVector3f::Zero();
		return DefaultNormalVector;
	}
 
	void FFoliageFacade::SetPivotPoint(const int32 Index, const FVector3f& Input)
	{
		check(!IsConst());
		if (IsValid() && PivotPointsAttribute.IsValidIndex(Index))
		{
			PivotPointsAttribute.ModifyAt(Index, Input);
		}
	}
 
	void FFoliageFacade::SetUpVector(int32 Index, const FVector3f& Input)
	{
		check(!IsConst());
		if (IsValid() && UpVectorsAttribute.IsValidIndex(Index))
		{
			UpVectorsAttribute.ModifyAt(Index, Input);
		}
	}
 
	void FFoliageFacade::SetNormalVector(int32 Index, const FVector3f& Input)
	{
		check(!IsConst());
		if (IsValid() && NormalVectorsAttribute.IsValidIndex(Index))
		{
			NormalVectorsAttribute.ModifyAt(Index, Input);
		}
	}
 
	void FFoliageFacade::SetScale(int32 Index, float Input)
	{
		check(!IsConst());
		if (IsValid() && ScalesAttribute.IsValidIndex(Index))
		{
			ScalesAttribute.ModifyAt(Index, Input);
		}
	}
 
	void FFoliageFacade::SetFoliageName(int32 Index, const FString& Input)
	{
		check(!IsConst());
		if (IsValid() && FoliageInfoNamesAttribute.IsValidIndex(Index))
		{
			FoliageInfoNamesAttribute.ModifyAt(Index, Input);
		}
	}
 
	const FString FFoliageFacade::GetFoliageName(const int32 Index) const
	{
		if (FoliageInfoNamesAttribute.IsValid() && FoliageInfoNamesAttribute.IsValidIndex(Index))
		{
			return FoliageInfoNamesAttribute[Index];
		}
 
		return FString();
	}
 
	TArray<FString> FFoliageFacade::GetFoliageNames() const
	{
		TArray<FString> FoliageNames;
		
		if (FoliageInfoNamesAttribute.IsValid())
		{
			return FoliageInfoNamesAttribute.Get().GetConstArray();
		}
 
		return FoliageNames;
	}

	void FFoliageFacade::SetFoliageNamesFromIndex(const TArray<FString>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(FoliageInfoNamesAttribute, InputArray, StartIndex);
	}

	void FFoliageFacade::SetFoliageNames(const TArray<FString>& InputNames)
	{
		check(!IsConst());
		Collection->EmptyGroup(GroupNames::FoliageNamesGroup);
		Collection->AddElements(InputNames.Num(), GroupNames::FoliageNamesGroup);
		for (int32 Index = 0; Index < InputNames.Num(); ++Index)
		{
			FoliageInfoNamesAttribute.Modify()[Index] = InputNames[Index];
		}
	}
 
	void FFoliageFacade::SetFoliageInfo(const int32 Index, const FString& Input)
	{
		check(!IsConst());
		if (IsValid() && FoliageInfoNamesAttribute.IsValidIndex(Index))
		{
			FoliageInfoNamesAttribute.ModifyAt(Index, Input);
		}
	}
 
	void FFoliageFacade::SetFoliageInfos(const TArray<FPVFoliageInfo>& Infos)
	{
		check(!IsConst());
		Collection->EmptyGroup(GroupNames::FoliageNamesGroup);
		Collection->AddElements(Infos.Num(), GroupNames::FoliageNamesGroup);
 
		for (int32 Index = 0; Index < Infos.Num(); ++Index)
		{
			const FPVFoliageInfo& Info = Infos[Index];
			
			FoliageInfoUseAsMaskAttribute.Modify()[Index] = Info.bUseAsMask;
			FoliageInfoNamesAttribute.Modify()[Index] = Info.Mesh.ToString();
			FoliageInfoScaleAttribute.Modify()[Index] = Info.Attributes.Scale;
			FoliageInfoLightAttribute.Modify()[Index] = Info.Attributes.Light;
			FoliageInfoTipAttribute.Modify()[Index] = Info.Attributes.Tip;
			FoliageInfoUpAlignmentAttribute.Modify()[Index] = Info.Attributes.UpAlignment;
			FoliageInfoHealthAttribute.Modify()[Index] = Info.Attributes.Health;
			FoliageInfoHeightAttribute.Modify()[Index] = Info.Attributes.Height;
			FoliageInfoGenerationAttribute.Modify()[Index] = Info.Attributes.Generation;
		}
	}

	int32 FFoliageFacade::AppendFoliageInfos(const TArray<FPVFoliageInfo>& Infos)
	{
		check(!IsConst());
		const int32 PrevSize = Collection->AddElements(Infos.Num(), GroupNames::FoliageNamesGroup);

		const int32 NewSize = PrevSize + Infos.Num();
		for (int32 Index = PrevSize; Index < NewSize; ++Index)
		{
			const FPVFoliageInfo& Info = Infos[Index - PrevSize];

			FoliageInfoUseAsMaskAttribute.Modify()[Index] = Info.bUseAsMask;
			FoliageInfoNamesAttribute.Modify()[Index] = Info.Mesh.ToString();
			FoliageInfoScaleAttribute.Modify()[Index] = Info.Attributes.Scale;
			FoliageInfoLightAttribute.Modify()[Index] = Info.Attributes.Light;
			FoliageInfoTipAttribute.Modify()[Index] = Info.Attributes.Tip;
			FoliageInfoUpAlignmentAttribute.Modify()[Index] = Info.Attributes.UpAlignment;
			FoliageInfoHealthAttribute.Modify()[Index] = Info.Attributes.Health;
			FoliageInfoHeightAttribute.Modify()[Index] = Info.Attributes.Height;
			FoliageInfoGenerationAttribute.Modify()[Index] = Info.Attributes.Generation;
		}

		return PrevSize;
	}
	
	void FFoliageFacade::SetFoliagePath(const FString InPath)
	{
		check(!IsConst());
 
		int32 NumElements = Collection->NumElements(GroupNames::DetailsGroup);
 
		if (NumElements == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
 
		FoliagePathAttribute.Modify()[0] = InPath;
	}
 
	FString FFoliageFacade::GetFoliagePath() const
	{
		if (FoliagePathAttribute.IsValid() && FoliagePathAttribute.IsValidIndex(0))
		{
			return FoliagePathAttribute[0];
		}
 
		return FString();
	}
 
	int32 FFoliageFacade::GetElementCount() const
	{
		return NameIdsAttribute.Num();
	}
 
	void FFoliageFacade::CopyEntry(int32 FromIndex, int32 ToIndex)
	{
		if (IsValid() && NameIdsAttribute.IsValidIndex(FromIndex) && NameIdsAttribute.IsValidIndex(ToIndex))
		{
			NameIdsAttribute.ModifyAt(ToIndex, NameIdsAttribute[FromIndex]);
			BranchIdsAttribute.ModifyAt(ToIndex, BranchIdsAttribute[FromIndex]);
			PivotPointsAttribute.ModifyAt(ToIndex, PivotPointsAttribute[FromIndex]);
			UpVectorsAttribute.ModifyAt(ToIndex, UpVectorsAttribute[FromIndex]);
			NormalVectorsAttribute.ModifyAt(ToIndex, NormalVectorsAttribute[FromIndex]);
			ScalesAttribute.ModifyAt(ToIndex, ScalesAttribute[FromIndex]);
			LengthFromRootAttribute.ModifyAt(ToIndex, LengthFromRootAttribute[FromIndex]);
			ConditionTipAttribute.ModifyAt(ToIndex, ConditionTipAttribute[FromIndex]);
			ConditionLightAttribute.ModifyAt(ToIndex, ConditionLightAttribute[FromIndex]);
			ConditionScaleAttribute.ModifyAt(ToIndex, ConditionScaleAttribute[FromIndex]);
			ConditionUpAlignmentAttribute.ModifyAt(ToIndex, ConditionUpAlignmentAttribute[FromIndex]);
			ConditionHealthAttribute.ModifyAt(ToIndex, ConditionHealthAttribute[FromIndex]);
			ConditionHeightAttribute.ModifyAt(ToIndex, ConditionHeightAttribute[FromIndex]);
			ConditionGenerationAttribute.ModifyAt(ToIndex, ConditionGenerationAttribute[FromIndex]);
			ParentBoneIdsAttribute.ModifyAt(ToIndex, ParentBoneIdsAttribute[FromIndex]);
		}
	}
 
	void FFoliageFacade::RemoveEntries(int32 NumEntries, int32 StartIndex)
	{
		if (IsValid() && NameIdsAttribute.IsValidIndex(StartIndex) && StartIndex + NumEntries <= NameIdsAttribute.Num())
		{
			NameIdsAttribute.RemoveElements(NumEntries, StartIndex);
		}
	}
 
	FTransform FFoliageFacade::GetFoliageTransform(int32 Id) const
	{
		const FFoliageEntryData Data = GetFoliageEntry(Id);
 
		const FVector UpVectorN = FVector(Data.UpVector).GetSafeNormal();
		if (UpVectorN.IsNearlyZero())
		{
			return FTransform(FQuat::Identity, FVector(Data.PivotPoint), FVector(Data.Scale));
		}
 
		const FVector NormalVectorN = FVector(Data.NormalVector).GetSafeNormal();
 
		FVector OrthogonalizedNormalVectorN =
			(NormalVectorN - FVector::DotProduct(NormalVectorN, UpVectorN) * UpVectorN).GetSafeNormal();
		if (OrthogonalizedNormalVectorN.IsNearlyZero())
		{
			FVector TempVector;
			UpVectorN.FindBestAxisVectors(TempVector, OrthogonalizedNormalVectorN);
		}
 
		const FQuat RotationQuat = FQuat(FRotationMatrix::MakeFromYZ(OrthogonalizedNormalVectorN, UpVectorN));
		return FTransform(RotationQuat, FVector(Data.PivotPoint), FVector(Data.Scale));
	}
 
	const TManagedArray<FVector3f>& FFoliageFacade::GetPivotPositions() const
	{
		return PivotPointsAttribute.Get();
	}

	void FFoliageFacade::SetPivotPositionsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(PivotPointsAttribute, InputArray, StartIndex);
	}

	TManagedArray<FVector3f>& FFoliageFacade::ModifyPivotPositions()
	{
		return PivotPointsAttribute.Modify();
	}

	const TManagedArray<int32>& FFoliageFacade::GetFoliageNameIDs() const
	{
		return NameIdsAttribute.Get();
	}

	void FFoliageFacade::SetFoliageNameIDsFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(NameIdsAttribute, InputArray, StartIndex);
	}

	const TManagedArray<int32>& FFoliageFacade::GetFoliageBranchIDs() const
	{
		return BranchIdsAttribute.Get();
	}

	void FFoliageFacade::SetFoliageBranchIDsFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BranchIdsAttribute, InputArray, StartIndex);
	}

	const TManagedArray<FVector3f>& FFoliageFacade::GetUpVectors() const
	{
		return UpVectorsAttribute.Get();
	}

	void FFoliageFacade::SetUpVectorsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(UpVectorsAttribute, InputArray, StartIndex);
	}

	const TManagedArray<FVector3f>& FFoliageFacade::GetNormalVectors() const
	{
		return NormalVectorsAttribute.Get();
	}

	void FFoliageFacade::SetNormalVectorsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(NormalVectorsAttribute, InputArray, StartIndex);
	}

	const TManagedArray<float>& FFoliageFacade::GetScales() const
	{
		return ScalesAttribute.Get();
	}

	void FFoliageFacade::SetScalesFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(ScalesAttribute, InputArray, StartIndex);
	}

	const TManagedArray<float>& FFoliageFacade::GetLFRs() const
	{
		return LengthFromRootAttribute.Get();
	}

	void FFoliageFacade::SetLFRsFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(LengthFromRootAttribute, InputArray, StartIndex);
	}
}
