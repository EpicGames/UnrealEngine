// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVMetaInfoFacade.h"

#include "DataTypes/PVGraftInfo.h"

#include "Facades/PVAttributesNames.h"

namespace PV::Facades
{
	FMetaInfoFacade::FMetaInfoFacade(FManagedArrayCollection& InCollection, const int32 InitialSize)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, GuidAttribute(InCollection, AttributeNames::Guid, GroupNames::DetailsGroup)
		, LeafGrowthAttribute(InCollection, AttributeNames::LeafGrowth, GroupNames::DetailsGroup)
		, AbscissionSenescenseAttribute(InCollection, AttributeNames::AbscissionSenescense, GroupNames::DetailsGroup)
		, LateralElongationAttribute(InCollection, AttributeNames::LateralElongation, GroupNames::DetailsGroup)
		, GraftUseAsMask(InCollection, AttributeNames::GraftUseAsMask, GroupNames::DetailsGroup)
		, GraftLightAttribute(InCollection, AttributeNames::GraftLight, GroupNames::DetailsGroup)
		, GraftUpAlignmentAttribute(InCollection, AttributeNames::GraftUpAlignment, GroupNames::DetailsGroup)
		, GraftHealthAttribute(InCollection, AttributeNames::GraftHealth, GroupNames::DetailsGroup)
		, GraftTipAttribute(InCollection, AttributeNames::GraftTip, GroupNames::DetailsGroup)
		, GraftHeightAttribute(InCollection, AttributeNames::GraftHeight, GroupNames::DetailsGroup)
		, GraftGenerationAttribute(InCollection, AttributeNames::GraftGeneration, GroupNames::DetailsGroup)
		, GraftScaleAttribute(InCollection, AttributeNames::GraftScale, GroupNames::DetailsGroup)
	{
		DefineSchema(InitialSize);
	}

	FMetaInfoFacade::FMetaInfoFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, GuidAttribute(InCollection, AttributeNames::Guid, GroupNames::DetailsGroup)
		, LeafGrowthAttribute(InCollection, AttributeNames::LeafGrowth, GroupNames::DetailsGroup)
		, AbscissionSenescenseAttribute(InCollection, AttributeNames::AbscissionSenescense, GroupNames::DetailsGroup)
		, LateralElongationAttribute(InCollection, AttributeNames::LateralElongation, GroupNames::DetailsGroup)
		, GraftUseAsMask(InCollection, AttributeNames::GraftUseAsMask, GroupNames::DetailsGroup)
		, GraftLightAttribute(InCollection, AttributeNames::GraftLight, GroupNames::DetailsGroup)
		, GraftUpAlignmentAttribute(InCollection, AttributeNames::GraftUpAlignment, GroupNames::DetailsGroup)
		, GraftHealthAttribute(InCollection, AttributeNames::GraftHealth, GroupNames::DetailsGroup)
		, GraftTipAttribute(InCollection, AttributeNames::GraftTip, GroupNames::DetailsGroup)
		, GraftHeightAttribute(InCollection, AttributeNames::GraftHeight, GroupNames::DetailsGroup)
		, GraftGenerationAttribute(InCollection, AttributeNames::GraftGeneration, GroupNames::DetailsGroup)
		, GraftScaleAttribute(InCollection, AttributeNames::GraftScale, GroupNames::DetailsGroup)
	{}

	void FMetaInfoFacade::DefineSchema(const int32 InitialSize)
	{
		check(!IsConst());

		if (!Collection->HasGroup(GroupNames::DetailsGroup))
		{
			Collection->AddGroup(GroupNames::DetailsGroup);
		}

		GraftUseAsMask.Add();
		GuidAttribute.Add();
		GraftLightAttribute.Add();
		GraftUpAlignmentAttribute.Add();
		GraftHealthAttribute.Add();
		GraftTipAttribute.Add();
		GraftHeightAttribute.Add();
		GraftGenerationAttribute.Add();
		GraftScaleAttribute.Add();
		LeafGrowthAttribute.Add();
		AbscissionSenescenseAttribute.Add();
		LateralElongationAttribute.Add();
	}

	bool FMetaInfoFacade::IsValid() const
	{
		return GuidAttribute.IsValid();
	}

	void FMetaInfoFacade::CreateGuid(const FString& InPath)
	{
		check(!IsConst());

		int32 NumElements = Collection->NumElements(GroupNames::DetailsGroup);

		if(NumElements == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		
		GuidAttribute.Modify()[0] = FGuid::NewDeterministicGuid(InPath);
	}

	FGuid FMetaInfoFacade::GetGuid() const
	{
		if(GuidAttribute.IsValid() && GuidAttribute.IsValidIndex(0))
		{
			return GuidAttribute[0];
		}

		return FGuid::NewGuid();
	}

	void FMetaInfoFacade::SetGraftAttributes(const FPVGraftInfo& InGraftData)
	{
		check(!IsConst());

		int32 NumElements = Collection->NumElements(GroupNames::DetailsGroup);

		if(NumElements == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		
		GraftUseAsMask.Modify()[0] = InGraftData.bUseAsMask;
		GraftLightAttribute.Modify()[0] = InGraftData.Attributes.Light;
		GraftUpAlignmentAttribute.Modify()[0] = InGraftData.Attributes.UpAlignment;
		GraftHealthAttribute.Modify()[0] = InGraftData.Attributes.Health;
		GraftTipAttribute.Modify()[0] = InGraftData.Attributes.Tip ? 1 : 0;
		GraftHeightAttribute.Modify()[0] = InGraftData.Attributes.Height;
		GraftGenerationAttribute.Modify()[0] = InGraftData.Attributes.Generation;
		GraftScaleAttribute.Modify()[0] = InGraftData.Attributes.Scale;
	}

	FPVGraftInfo FMetaInfoFacade::GetGraftAttributes() const
	{
		FPVGraftInfo GraftData = {};
		
		if (GraftUseAsMask.IsValidIndex(0))
			GraftData.bUseAsMask = GraftUseAsMask.Get()[0];
		
		if (GraftLightAttribute.IsValidIndex(0))
			GraftData.Attributes.Light = GraftLightAttribute[0];
		
		if (GraftUpAlignmentAttribute.IsValidIndex(0))
			GraftData.Attributes.UpAlignment = GraftUpAlignmentAttribute[0];
		
		if (GraftHealthAttribute.IsValidIndex(0))
			GraftData.Attributes.Health = GraftHealthAttribute[0];
			
		if (GraftTipAttribute.IsValidIndex(0))
			GraftData.Attributes.Tip = (GraftTipAttribute[0] > 0);
		
		if (GraftHeightAttribute.IsValidIndex(0))
			GraftData.Attributes.Height = GraftHeightAttribute[0];
		
		if (GraftGenerationAttribute.IsValidIndex(0))
			GraftData.Attributes.Generation = GraftGenerationAttribute[0];
		
		if (GraftScaleAttribute.IsValidIndex(0))
			GraftData.Attributes.Scale = GraftScaleAttribute[0];
		
		return GraftData;
	}

	bool FMetaInfoFacade::GraftEntryUseAsMask() const
	{
		if (GraftUseAsMask.IsValid() && GraftUseAsMask.IsValidIndex(0))
		{
			return GraftUseAsMask.Get()[0];
		}
		return false;
	}

	void FMetaInfoFacade::SetGraftUseAsMask(const bool InUseAsMask)
	{
		if (GraftUseAsMask.IsValid() && GraftUseAsMask.IsValidIndex(0))
		{
			GraftUseAsMask.Modify()[0] = InUseAsMask;
		}
	}

	void FMetaInfoFacade::SetLeafGrowth(const TArray<float>& InLeafGrowth)
	{
		check(!IsConst());
		if (Collection->NumElements(GroupNames::DetailsGroup) == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		LeafGrowthAttribute.Modify()[0] = InLeafGrowth;
	}

	void FMetaInfoFacade::SetAbscissionSenescense(const TArray<float>& InAbscissionSenescense)
	{
		check(!IsConst());
		if (Collection->NumElements(GroupNames::DetailsGroup) == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		AbscissionSenescenseAttribute.Modify()[0] = InAbscissionSenescense;
	}

	void FMetaInfoFacade::SetLateralElongation(const TArray<float>& InLateralElongation)
	{
		check(!IsConst());
		if (Collection->NumElements(GroupNames::DetailsGroup) == 0)
		{
			Collection->AddElements(1, GroupNames::DetailsGroup);
		}
		LateralElongationAttribute.Modify()[0] = InLateralElongation;
	}

	const TArray<float>& FMetaInfoFacade::GetLeafGrowth() const
	{
		if(LeafGrowthAttribute.IsValid() && LeafGrowthAttribute.IsValidIndex(0))
		{
			return LeafGrowthAttribute[0];
		}

		static TArray<float> DefaultLeafGrowthArray;
		return DefaultLeafGrowthArray;
	}

	const TArray<float>& FMetaInfoFacade::GetAbscissionSenescense() const
	{
		if(AbscissionSenescenseAttribute.IsValid() && AbscissionSenescenseAttribute.IsValidIndex(0))
		{
			return AbscissionSenescenseAttribute[0];
		}

		static TArray<float> DefaultAbscissionSenescenseArray;
		return DefaultAbscissionSenescenseArray;
	}
	
	const TArray<float>& FMetaInfoFacade::GetLateralElongation() const
	{
		if(LateralElongationAttribute.IsValid() && LateralElongationAttribute.IsValidIndex(0))
		{
			return LateralElongationAttribute[0];
		}

		static TArray<float> DefaultLateralElongationArray;
		return DefaultLateralElongationArray;
	}
}
