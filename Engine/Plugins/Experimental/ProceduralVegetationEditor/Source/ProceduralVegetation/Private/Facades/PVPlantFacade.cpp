// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVPlantFacade.h"
#include "Facades/PVAttributesNames.h"

namespace PV::Facades
{
	FPlantFacade::FPlantFacade(FManagedArrayCollection& InCollection)
		: Collection(&InCollection)
		, PlantNumbers(InCollection, AttributeNames::PlantNumber, GroupNames::BranchGroup)
		, BranchNumbers(InCollection, AttributeNames::BranchNumber, GroupNames::BranchGroup)
		, BranchParentNumbers(InCollection, AttributeNames::BranchParentNumber, GroupNames::BranchGroup)
		, BranchHierarchyNumbers(InCollection, AttributeNames::BranchHierarchyNumber, GroupNames::BranchGroup)
	{
	}

	FPlantFacade::FPlantFacade(const FManagedArrayCollection& InCollection)
		: Collection(nullptr)
		, PlantNumbers(InCollection, AttributeNames::PlantNumber, GroupNames::BranchGroup)
		, BranchNumbers(InCollection, AttributeNames::BranchNumber, GroupNames::BranchGroup)
		, BranchParentNumbers(InCollection, AttributeNames::BranchParentNumber, GroupNames::BranchGroup)
		, BranchHierarchyNumbers(InCollection, AttributeNames::BranchHierarchyNumber, GroupNames::BranchGroup)
	{
	}

	bool FPlantFacade::IsValid() const
	{
		return PlantNumbers.IsValid()
			&& BranchNumbers.IsValid()
			&& BranchParentNumbers.IsValid()
			&& BranchHierarchyNumbers.IsValid();
	}

	int32 FPlantFacade::GetElementCount() const
	{
		TSet<int32> PlantNumberElements;
		for (int32 i = 0; i < PlantNumbers.Num(); i++)
		{
			PlantNumberElements.Add(PlantNumbers[i]);
		}

		return PlantNumberElements.Num();
	}

	TArray<int32> FPlantFacade::GetBranchNumbers(const int32 PlantNumber) const
	{
		TArray<int32> BranchNumbersResult;

		for (int32 i = 0; i < BranchNumbers.Num(); i++)
		{
			if (PlantNumbers[i] == PlantNumber)
			{
				BranchNumbersResult.Add(BranchNumbers[i]);
			}
		}

		return BranchNumbersResult;
	}

	TArray<int32> FPlantFacade::GetBranchIndices(const int32 PlantNumber) const
	{
		TArray<int32> BranchIndicesResult;

		for (int32 i = 0; i < BranchNumbers.Num(); i++)
		{
			if (PlantNumbers[i] == PlantNumber)
			{
				BranchIndicesResult.Add(i);
			}
		}

		return BranchIndicesResult;
	}

	TArray<int32> FPlantFacade::GetBranchIndicesSortedByHierarchyNumber(const int32 PlantNumber) const
	{
		TArray<int32> BranchIndices = GetBranchIndices(PlantNumber);
		BranchIndices.Sort([&](const int32 Index1, const int32 Index2)
			{
				const int32 Index1Hierarchy = BranchHierarchyNumbers[Index1];
				const int32 Index2Hierarchy = BranchHierarchyNumbers[Index2];

				if (Index1Hierarchy == Index2Hierarchy)
				{
					return Index1 < Index2;
				}

				return Index1Hierarchy < Index2Hierarchy;
			});

		return BranchIndices;
	}

	TArray<int32> FPlantFacade::GetPlantNumbers() const
	{
		TSet<int32> PlantNumberElements;
		for (int32 i = 0; i < PlantNumbers.Num(); i++)
		{
			PlantNumberElements.Add(PlantNumbers[i]);
		}

		return PlantNumberElements.Array();
	}

	TArray<int32> FPlantFacade::GetTrunkIndices() const
	{
		TArray<int32> TrunkIndicesResult;

		for (int32 i = 0; i < BranchNumbers.Num(); i++)
		{
			if (BranchParentNumbers[i] == 0)
			{
				TrunkIndicesResult.Add(i);
			}
		}

		return TrunkIndicesResult;
	}

	TMap<int32, int32> FPlantFacade::GetPlantNumbersToTrunkIndicesMap() const
	{
		TMap<int32, int32> PlantNumberToTrunkIndicesMap;
		for (int32 i = 0; i < BranchNumbers.Num(); i++)
		{
			if (BranchParentNumbers[i] == 0)
			{
				PlantNumberToTrunkIndicesMap.Add(PlantNumbers[i], i);
			}
		}

		return PlantNumberToTrunkIndicesMap;
	}

	int32 FPlantFacade::GetTrunkIndex(const int32 PlantNumber) const
	{
		for (int32 i = 0; i < BranchParentNumbers.Num(); i++)
		{
			if (PlantNumbers[i] == PlantNumber && BranchParentNumbers[i] == 0)
			{
				return i;
			}
		}

		return INDEX_NONE;
	}

	bool FPlantFacade::IsTrunkIndex(const int32 BranchIndex) const
	{
		if (BranchParentNumbers.IsValidIndex(BranchIndex))
		{
			return BranchParentNumbers[BranchIndex] == 0;
		}

		return false;
	}
}
