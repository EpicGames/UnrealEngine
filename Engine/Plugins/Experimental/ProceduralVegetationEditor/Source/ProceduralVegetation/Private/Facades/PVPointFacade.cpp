// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVPointFacade.h"
#include "PVFacadeCommon.h"
#include "Facades/PVAttributesNames.h"

namespace PV::Facades
{
	FPointFacade::FPointFacade(FManagedArrayCollection& InCollection)
		: Collection(&InCollection)
		, BranchPoints(InCollection, AttributeNames::BranchPoints, GroupNames::BranchGroup)
		, Positions(InCollection, AttributeNames::PointPosition, GroupNames::PointGroup)
		, LengthFromRoot(InCollection, AttributeNames::LengthFromRoot, GroupNames::PointGroup)
		, PointScaleGradient(InCollection, AttributeNames::PointScaleGradient, GroupNames::PointGroup)
		, HullGradient(InCollection, AttributeNames::HullGradient, GroupNames::PointGroup)
		, MainTrunkGradient(InCollection, AttributeNames::MainTrunkGradient, GroupNames::PointGroup)
		, GroundGradient(InCollection, AttributeNames::GroundGradient, GroupNames::PointGroup)
		, PointScale(InCollection, AttributeNames::PointScale, GroupNames::PointGroup)
		, BudDirections(InCollection, AttributeNames::BudDirection, GroupNames::PointGroup)
		, BudStatus(InCollection, AttributeNames::BudStatus, GroupNames::PointGroup)
		, BudLateralMeristem(InCollection, AttributeNames::BudLateralMeristem, GroupNames::PointGroup)
		, BudHormoneLevels(InCollection, AttributeNames::BudHormoneLevels, GroupNames::PointGroup)
		, PlantGradients(InCollection, AttributeNames::PlantGradient, GroupNames::PointGroup)
		, LengthFromSeed(InCollection, AttributeNames::LengthFromSeed, GroupNames::PointGroup)
		, SeedPScale(InCollection, AttributeNames::SeedPScale, GroupNames::PointGroup)
		, SeedPScaleRatio(InCollection, AttributeNames::SeedPScaleRatio, GroupNames::PointGroup)
		, BudNumber(InCollection, AttributeNames::BudNumber, GroupNames::PointGroup)
		, NjordPixelIndex(InCollection, AttributeNames::NjordPixelIndex, GroupNames::PointGroup)
		, BudLightDetected(InCollection, AttributeNames::BudLightDetected, GroupNames::PointGroup)
		, BudDevelopment(InCollection, AttributeNames::BudDevelopment, GroupNames::PointGroup)
		, TextureCoordV(InCollection, AttributeNames::TextureCoordV, GroupNames::PointGroup)
		, TextureCoordUOffset(InCollection, AttributeNames::TextureCoordUOffset, GroupNames::PointGroup)
		, URange(InCollection, AttributeNames::URange, GroupNames::PointGroup)
	{
	}

	FPointFacade::FPointFacade(const FManagedArrayCollection& InCollection)
		: Collection(nullptr)
		, BranchPoints(InCollection, AttributeNames::BranchPoints, GroupNames::BranchGroup)
		, Positions(InCollection, AttributeNames::PointPosition, GroupNames::PointGroup)
		, LengthFromRoot(InCollection, AttributeNames::LengthFromRoot, GroupNames::PointGroup)
		, PointScaleGradient(InCollection, AttributeNames::PointScaleGradient, GroupNames::PointGroup)
		, HullGradient(InCollection, AttributeNames::HullGradient, GroupNames::PointGroup)
		, MainTrunkGradient(InCollection, AttributeNames::MainTrunkGradient, GroupNames::PointGroup)
		, GroundGradient(InCollection, AttributeNames::GroundGradient, GroupNames::PointGroup)
		, PointScale(InCollection, AttributeNames::PointScale, GroupNames::PointGroup)
		, BudDirections(InCollection, AttributeNames::BudDirection, GroupNames::PointGroup)
		, BudStatus(InCollection, AttributeNames::BudStatus, GroupNames::PointGroup)
		, BudLateralMeristem(InCollection, AttributeNames::BudLateralMeristem, GroupNames::PointGroup)
		, BudHormoneLevels(InCollection, AttributeNames::BudHormoneLevels, GroupNames::PointGroup)
		, PlantGradients(InCollection, AttributeNames::PlantGradient, GroupNames::PointGroup)
		, LengthFromSeed(InCollection, AttributeNames::LengthFromSeed, GroupNames::PointGroup)
		, SeedPScale(InCollection, AttributeNames::SeedPScale, GroupNames::PointGroup)
		, SeedPScaleRatio(InCollection, AttributeNames::SeedPScaleRatio, GroupNames::PointGroup)
		, BudNumber(InCollection, AttributeNames::BudNumber, GroupNames::PointGroup)
		, NjordPixelIndex(InCollection, AttributeNames::NjordPixelIndex, GroupNames::PointGroup)
		, BudLightDetected(InCollection, AttributeNames::BudLightDetected, GroupNames::PointGroup)
		, BudDevelopment(InCollection, AttributeNames::BudDevelopment, GroupNames::PointGroup)
		, TextureCoordV(InCollection, AttributeNames::TextureCoordV, GroupNames::PointGroup)
		, TextureCoordUOffset(InCollection, AttributeNames::TextureCoordUOffset, GroupNames::PointGroup)
		, URange(InCollection, AttributeNames::URange, GroupNames::PointGroup)
	{
	}

	bool FPointFacade::IsValid() const
	{
		return Positions.IsValid()
			&& LengthFromRoot.IsValid()
			&& LengthFromSeed.IsValid()
			&& PointScale.IsValid()
			&& BudDirections.IsValid()
			&& BudHormoneLevels.IsValid()
			&& BudNumber.IsValid()
			&& BudLightDetected.IsValid()
			&& BudDevelopment.IsValid()
			&& SeedPScale.IsValid()
			&& SeedPScaleRatio.IsValid();
	}

	int32 FPointFacade::GetElementCount() const
	{
		return Positions.Num();
	}

	const FVector3f& FPointFacade::GetPosition(int32 Index) const
	{
		if (Positions.IsValid() && Positions.IsValidIndex(Index))
		{
			return Positions[Index];
		}

		static const FVector3f DefaultPosition = FVector3f::ZeroVector;
		return DefaultPosition;
	}

	void FPointFacade::SetPosition(int32 Index, const FVector3f& InPosition)
	{
		if (Positions.IsValid() && Positions.IsValidIndex(Index))
		{
			Positions.ModifyAt(Index, InPosition);
		}
	}

	float FPointFacade::GetLengthFromRoot(int32 Index) const
	{
		if (LengthFromRoot.IsValid() && LengthFromRoot.IsValidIndex(Index))
		{
			return LengthFromRoot[Index];
		}

		return 0;
	}

	void FPointFacade::SetLengthFromRoot(int32 Index, float InLength)
	{
		if (LengthFromRoot.IsValid() && LengthFromRoot.IsValidIndex(Index))
		{
			LengthFromRoot.ModifyAt(Index, InLength);
		}
	}

	float FPointFacade::GetLengthFromSeed(int32 Index) const
	{
		if (LengthFromSeed.IsValid() && LengthFromSeed.IsValidIndex(Index))
		{
			return LengthFromSeed[Index];
		}
		
		return 0;
	}

	void FPointFacade::SetLengthFromSeed(int32 Index, float InLength)
	{
		if (LengthFromSeed.IsValid() && LengthFromSeed.IsValidIndex(Index))
		{
			LengthFromSeed.ModifyAt(Index, InLength);
		}
	}

	float FPointFacade::GetPointScale(int32 Index) const
	{
		if (PointScale.IsValid() && PointScale.IsValidIndex(Index))
		{
			return PointScale[Index];
		}

		return 0;
	}

	void FPointFacade::SetPointScale(int32 Index, float InPointScale)
	{
		if (PointScale.IsValid() && PointScale.IsValidIndex(Index))
		{
			PointScale.ModifyAt(Index, InPointScale);
		}
	}

	void FPointFacade::SetSeedPScale(int32 Index, float InSeedPScale)
	{
		if (SeedPScale.IsValid() && SeedPScale.IsValidIndex(Index))
		{
			SeedPScale.ModifyAt(Index, InSeedPScale);
		}
	}

	void FPointFacade::SetSeedPScaleRatio(int32 Index, float InSeedPScaleRatio)
	{
		if (SeedPScaleRatio.IsValid() && SeedPScaleRatio.IsValidIndex(Index))
		{
			SeedPScaleRatio.ModifyAt(Index, InSeedPScaleRatio);
		}
	}

	int32 FPointFacade::GetBudNumber(int32 Index) const
	{
		if (BudNumber.IsValid() && BudNumber.IsValidIndex(Index))
		{
			return BudNumber[Index];
		}

		return INDEX_NONE;
	}

	void FPointFacade::SetBudNumber(int32 Index, int32 InBudNumber)
	{
		if (BudNumber.IsValid() && BudNumber.IsValidIndex(Index))
		{
			BudNumber.ModifyAt(Index, InBudNumber);
		}
	}
	
	const TArray<float>& FPointFacade::GetBudLightDetected(int32 Index) const
	{
		if (BudLightDetected.IsValid() && BudLightDetected.IsValidIndex(Index))
		{
			return BudLightDetected[Index];
		}

		static const TArray<float> EmptyArray;
		return EmptyArray;
	}

	void FPointFacade::SetBudLightDetected(int32 Index, const TArray<float>& InBudLightDetected)
	{
		if (BudLightDetected.IsValid() && BudLightDetected.IsValidIndex(Index))
		{
			BudLightDetected.Modify()[Index] = InBudLightDetected;
		}
	}

	const TArray<int>& FPointFacade::GetBudDevelopment(int32 Index) const
	{
		if (BudDevelopment.IsValid() && BudDevelopment.IsValidIndex(Index))
		{
			return BudDevelopment[Index];
		}

		static const TArray<int> EmptyArray;
		return EmptyArray;
	}
	
	void FPointFacade::SetBudDevelopment(int32 Index, const TArray<int>& InBudDevelopment)
	{
		if (BudDevelopment.IsValid() && BudDevelopment.IsValidIndex(Index))
		{
			BudDevelopment.Modify()[Index] = InBudDevelopment;
		}
	}

	int32 FPointFacade::GetBudGeneratation(int32 Index) const
	{
		const TArray<int>& PointBudDevelopment = GetBudDevelopment(Index);
		if (PointBudDevelopment.IsValidIndex(0))
		{
			return PointBudDevelopment[0];
		}

		return INDEX_NONE;
	}

	int32 FPointFacade::GetBudAge(int32 Index) const
	{
		const TArray<int>& PointBudDevelopment = GetBudDevelopment(Index);
		if (PointBudDevelopment.IsValidIndex(1))
		{
			return PointBudDevelopment[1];
		}

		return INDEX_NONE;
	}

	int32 FPointFacade::GetMaxBudAge() const
	{
		int32 MaxBudAge = -1;
		if (BudDevelopment.IsValid())
		{
			for (int32 i = 0; i < BudDevelopment.Num(); ++i)
			{
				MaxBudAge = FMath::Max(MaxBudAge, BudDevelopment[i][1]);
			}
		}

		return MaxBudAge;
	}

	void FPointFacade::AddToBudAgeForAllPoints(const int32 Age)
	{
		if (BudDevelopment.IsValid())
		{
			for (int32 i = 0; i < BudDevelopment.Num(); ++i)
			{
				BudDevelopment.Modify()[i][1] += Age;
			}
		}
	}

	int32 FPointFacade::GetBudBranchAge(int32 Index) const
	{
		const TArray<int>& PointBudDevelopment = GetBudDevelopment(Index);
		if (PointBudDevelopment.IsValidIndex(2))
		{
			return PointBudDevelopment[2];
		}

		return INDEX_NONE;
	}

	const TArray<FVector3f>& FPointFacade::GetBudDirection(int32 Index) const
	{
		if (BudDirections.IsValid() && BudDirections.IsValidIndex(Index))
		{
			return BudDirections[Index];
		}

		static const TArray<FVector3f> EmptyArray;
		return EmptyArray;
	}

	void FPointFacade::SetBudDirections(int32 Index, const TArray<FVector3f>& InBudDirections)
	{
		if (BudDirections.IsValid() && BudDirections.IsValidIndex(Index))
		{
			BudDirections.Modify()[Index] = InBudDirections;
		}
	}

	const TArray<float>& FPointFacade::GetBudHormoneLevels(int32 Index) const
	{
		if (BudHormoneLevels.IsValid() && BudHormoneLevels.IsValidIndex(Index))
		{
			return BudHormoneLevels[Index];
		}

		static const TArray<float> EmptyArray;
		return EmptyArray;
	}

	void FPointFacade::SetBudHormoneLevels(int32 Index, const TArray<float>& InBudHormoneLevels)
	{
		if (BudHormoneLevels.IsValid() && BudHormoneLevels.IsValidIndex(Index))
		{
			BudHormoneLevels.Modify()[Index] = InBudHormoneLevels;
		}
	}

	const TArray<float>& FPointFacade::GetBudLateralMeristem(int32 Index) const
	{
		if (BudLateralMeristem.IsValid() && BudLateralMeristem.IsValidIndex(Index))
		{
			return BudLateralMeristem[Index];
		}

		static const TArray<float> EmptyArray;
		return EmptyArray;
	}

	void FPointFacade::SetBudLateralMeristem(int32 Index, const TArray<float>& InBudLateralMeristem)
	{
		if (BudLateralMeristem.IsValid() && BudLateralMeristem.IsValidIndex(Index))
		{
			BudLateralMeristem.ModifyAt(Index, InBudLateralMeristem);
		}
	}

	float FPointFacade::GetTextureCoordV(int32 Index) const
	{
		if (TextureCoordV.IsValid() && TextureCoordV.IsValidIndex(Index))
		{
			return TextureCoordV[Index];
		}

		return float(INDEX_NONE);
	}

	void FPointFacade::SetTextureCoordV(int32 Index, float InTextureCoordV)
	{
		if (!TextureCoordV.IsValid())
		{
			TextureCoordV.Add();
		}
		
		if (TextureCoordV.IsValid() && TextureCoordV.IsValidIndex(Index))
		{
			TextureCoordV.ModifyAt(Index, InTextureCoordV);
		}
	}

	float FPointFacade::GetTextureCoordUOffset(int32 Index) const
	{
		if (TextureCoordUOffset.IsValid() && TextureCoordUOffset.IsValidIndex(Index))
		{
			return TextureCoordUOffset[Index];
		}

		return float(INDEX_NONE);
	}

	void FPointFacade::SetTextureCoordUOffset(int32 Index, float InTextureCoordUOffset)
	{
		if (!TextureCoordUOffset.IsValid())
		{
			TextureCoordUOffset.Add();
		}
		
		if (TextureCoordUOffset.IsValid() && TextureCoordUOffset.IsValidIndex(Index))
		{
			TextureCoordUOffset.ModifyAt(Index, InTextureCoordUOffset);
		}
	}

	const FVector2f& FPointFacade::GetURange(int32 Index) const
	{
		if (URange.IsValid() && URange.IsValidIndex(Index))
		{
			return URange[Index];
		}

		static const FVector2f Range(0,1);
		return Range;
	}

	void FPointFacade::SetURange(int32 Index, const FVector2f InURange)
	{
		if (!URange.IsValid())
		{
			URange.Add();
		}
		
		if (URange.IsValid() && URange.IsValidIndex(Index))
		{
			URange.ModifyAt(Index, InURange);
		}
	}

	bool FPointFacade::IsFusedPoint(const int Generation, const int SourceBudNumber, const int PointIndex) const
	{
		const int Number = GetBudNumber(PointIndex);
		
		return Generation > 1 && Number == SourceBudNumber;
	}

	TManagedArray<FVector3f>& FPointFacade::ModifyPositions()
	{
		return Positions.Modify();
	}

	TManagedArray<float>& FPointFacade::ModifyPointScales()
	{
		return PointScale.Modify();
	}

	TManagedArray<float>& FPointFacade::ModifyLengthFromRoots()
	{
		return LengthFromRoot.Modify();
	}

	TManagedArray<float>& FPointFacade::ModifyLengthFromSeeds()
	{
		return LengthFromSeed.Modify();
	}

	const TManagedArray<FVector3f>& FPointFacade::GetPositions() const
	{
		return Positions.Get();
	}

	const TManagedArray<float>& FPointFacade::GetPointScales() const
	{
		return PointScale.Get();
	}

	const TManagedArray<float>& FPointFacade::GetLengthFromRootsArray() const
	{
		return LengthFromRoot.Get();
	}

	void FPointFacade::CopyEntry(int32 FromIndex, int32 ToIndex)
	{
		if (IsValid() && Positions.IsValidIndex(FromIndex) && Positions.IsValidIndex(ToIndex))
		{
			Positions.ModifyAt(ToIndex, Positions[FromIndex]);
			LengthFromRoot.ModifyAt(ToIndex, LengthFromRoot[FromIndex]);
			
			if (PointScaleGradient.IsValid())
			{
				PointScaleGradient.ModifyAt(ToIndex, PointScaleGradient[FromIndex]);
			}
			
			if (HullGradient.IsValid())
			{
				HullGradient.ModifyAt(ToIndex, HullGradient[FromIndex]);
			}
			
			if (MainTrunkGradient.IsValid())
			{
				MainTrunkGradient.ModifyAt(ToIndex, MainTrunkGradient[FromIndex]);
			}
			
			if (GroundGradient.IsValid())
			{
				GroundGradient.ModifyAt(ToIndex, GroundGradient[FromIndex]);
			}
			
			PointScale.ModifyAt(ToIndex, PointScale[FromIndex]);
			BudDirections.ModifyAt(ToIndex, BudDirections[FromIndex]);
			BudHormoneLevels.ModifyAt(ToIndex, BudHormoneLevels[FromIndex]);
			
			if (PlantGradients.IsValid())
			{
				PlantGradients.ModifyAt(ToIndex, PlantGradients[FromIndex]);
			}
			
			LengthFromSeed.ModifyAt(ToIndex, LengthFromSeed[FromIndex]);
			BudNumber.ModifyAt(ToIndex, BudNumber[FromIndex]);
			
			if (NjordPixelIndex.IsValid())
			{
				NjordPixelIndex.ModifyAt(ToIndex, NjordPixelIndex[FromIndex]);
			}
			
			BudLightDetected.ModifyAt(ToIndex, BudLightDetected[FromIndex]);
			BudDevelopment.ModifyAt(ToIndex, BudDevelopment[FromIndex]);
			
			if(TextureCoordV.IsValid() && TextureCoordV.IsValidIndex(ToIndex) && TextureCoordV.IsValidIndex(FromIndex))
			{
				TextureCoordV.ModifyAt(ToIndex, TextureCoordV[FromIndex]);
			}
			if(TextureCoordUOffset.IsValid() && TextureCoordUOffset.IsValidIndex(ToIndex) && TextureCoordUOffset.IsValidIndex(FromIndex))
			{
				TextureCoordUOffset.ModifyAt(ToIndex, TextureCoordUOffset[FromIndex]);
			}
			if(URange.IsValid() && URange.IsValidIndex(ToIndex) && URange.IsValidIndex(FromIndex))
			{
				URange.ModifyAt(ToIndex, URange[FromIndex]);
			}
			if (BudStatus.IsValid() && BudStatus.IsValidIndex(ToIndex) && BudStatus.IsValidIndex(FromIndex))
			{
				BudStatus.ModifyAt(ToIndex, BudStatus[FromIndex]);
			}

			if (BudLateralMeristem.IsValid() && BudLateralMeristem.IsValidIndex(ToIndex) && BudLateralMeristem.IsValidIndex(FromIndex))
			{
				BudLateralMeristem.ModifyAt(ToIndex, BudLateralMeristem[FromIndex]);
			}

			if (SeedPScale.IsValid() && SeedPScale.IsValidIndex(ToIndex) && SeedPScale.IsValidIndex(FromIndex))
			{
				SeedPScale.ModifyAt(ToIndex, SeedPScale[FromIndex]);
			}

			if (SeedPScaleRatio.IsValid() && SeedPScaleRatio.IsValidIndex(ToIndex) && SeedPScaleRatio.IsValidIndex(FromIndex))
			{
				SeedPScaleRatio.ModifyAt(ToIndex, SeedPScaleRatio[FromIndex]);
			}
		}
	}

	void FPointFacade::RemoveEntries(int32 NumEntries, int32 StartIndex)
	{
		if (IsValid() && Positions.IsValidIndex(StartIndex) && StartIndex + NumEntries <= Positions.Num())
		{
			Positions.RemoveElements(NumEntries, StartIndex);
		}
	}

	int32 FPointFacade::AddElements(int NumElements)
	{
		if (!Positions.IsValid())
		{
			Positions.Add();
		}
		
		return Positions.AddElements(NumElements);
	}

	void FPointFacade::FillPositions(const TArray<FVector3f>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(Positions, InputArray);
	}

	void FPointFacade::FillLFR(const TArray<float>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(LengthFromRoot, InputArray);
	}

	void FPointFacade::FillPointScales(const TArray<float>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(PointScale, InputArray);
	}

	void FPointFacade::FillBudDirections(const TArray<TArray<FVector3f>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudDirections, InputArray);
	}

	void FPointFacade::FillBudHormoneLevels(const  TArray<TArray<float>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudHormoneLevels, InputArray);
	}

	void FPointFacade::FillLengthFromSeeds(const TArray<float>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(LengthFromSeed, InputArray);
	}

	void FPointFacade::FillBudNumbers(const TArray<int32>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudNumber, InputArray);
	}

	void FPointFacade::FillBudLightDetected(const TArray<TArray<float>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudLightDetected, InputArray);
	}

	void FPointFacade::FillBudDevelopment(const TArray<TArray<int>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudDevelopment, InputArray);
	}
	
	void FPointFacade::FillBudStatus(const TArray<TArray<int>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudStatus, InputArray);
	}

	void FPointFacade::FillBudLateralMeristem(const TArray<TArray<float>>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(BudLateralMeristem, InputArray);
	}

	void FPointFacade::FillSeedPScale(const TArray<float>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(SeedPScale, InputArray);
	}

	void FPointFacade::FillSeedPScaleRatio(const TArray<float>& InputArray)
	{
		FILL_COLLECTION_ATTRIBUTE(SeedPScaleRatio, InputArray);
	}

	void FPointFacade::GetLFR(int32 Index, float& OutValue) const
	{
		GetValueFromAccessor<float>(LengthFromRoot, OutValue, Index, -1);
	}

	void FPointFacade::GetPointScales(int32 Index, float& OutValue) const
	{
		GetValueFromAccessor<float>(PointScale, OutValue, Index , -1);
	}

	void FPointFacade::GetBudDirections(int32 Index, TArray<FVector3f>& OutValue) const
	{
		GetValueFromAccessor<TArray<FVector3f>>(BudDirections, OutValue, Index, TArray<FVector3f>());
	}

	void FPointFacade::GetBudHormoneLevels(int32 Index, TArray<float>& OutValue) const
	{
		GetValueFromAccessor<TArray<float>>(BudHormoneLevels, OutValue, Index, TArray<float>());
	}

	void FPointFacade::GetBudNumbers(int32 Index, int32& OutValue) const
	{
		GetValueFromAccessor<int32>(BudNumber, OutValue, Index, -1);
	}

	const TManagedArray<int32>& FPointFacade::GetBudNumbersAttribute() const
	{
		return BudNumber.Get();
	}

	void FPointFacade::SetBudNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BudNumber, InputArray, StartIndex);
	}

	void FPointFacade::GetBudLightDetected(int32 Index, TArray<float>& OutValue) const
	{
		GetValueFromAccessor<TArray<float>>(BudLightDetected, OutValue, Index, TArray<float>());
	}

	void FPointFacade::GetBudDevelopment(int32 Index, TArray<int>& OutValue) const
	{
		GetValueFromAccessor<TArray<int>>(BudDevelopment, OutValue, Index, TArray<int>());
	}

	void FPointFacade::GetBudStatus(int32 Index, TArray<int>& OutValue) const
	{
		GetValueFromAccessor<TArray<int>>(BudStatus, OutValue, Index, TArray<int>());
	}

	void FPointFacade::SetBudStatus(int32 Index, const TArray<int>& InValue)
	{
		if (BudStatus.IsValid() && BudStatus.IsValidIndex(Index))
		{
			BudStatus.ModifyAt(Index, InValue);
		}
	}

	void FPointFacade::GetBudLateralMeristem(int32 Index, TArray<float>& OutValue) const
	{
		GetValueFromAccessor<TArray<float>>(BudLateralMeristem, OutValue, Index, TArray<float>());
	}

	void FPointFacade::GetSeedPScale(int32 Index, float& OutValue) const
	{
		GetValueFromAccessor<float>(SeedPScale, OutValue, Index, 0.1f);
	}

	void FPointFacade::GetSeedPScaleRatio(int32 Index, float& OutValue) const
	{
		GetValueFromAccessor<float>(SeedPScaleRatio, OutValue, Index, 1);
	}
	
	const TManagedArray<TArray<float>>& FPointFacade::GetBudLightDetectedArrays() const
	{
		return BudLightDetected.Get();
	}

	const TManagedArray<TArray<float>>& FPointFacade::GetBudLateralMeristemArrays() const
	{
		return BudLateralMeristem.Get();
	}

	void FPointFacade::SetBudLightDetectedArraysFromIndex(const TArray<TArray<float>>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(BudLightDetected, InputArray, StartIndex);
	}
	
	void FPointFacade::SetPositionsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(Positions, InputArray, StartIndex);
	}

	void FPointFacade::SetPointScalesFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(PointScale, InputArray, StartIndex);
	}

	void FPointFacade::SetSeedPScalesFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(SeedPScale, InputArray, StartIndex);
	}

	void FPointFacade::SetSeedPScaleRatiosFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(SeedPScaleRatio, InputArray, StartIndex);
	}

	void FPointFacade::SetLFRsFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(LengthFromRoot, InputArray, StartIndex);
	}

	const TManagedArray<float>& FPointFacade::GetLengthFromSeedsArray() const
	{
		return LengthFromSeed.Get();
	}

	void FPointFacade::SetLengthFromSeedsArrayFromIndex(const TArray<float>& InputArray, const int32 StartIndex)
	{
		FILL_COLLECTION_ATTRIBUTE_FROM_INDEX(LengthFromSeed, InputArray, StartIndex);
	}
}
