// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVProfileFacade.h"
#include "Facades/PVAttributesNames.h"

namespace PV::Facades
{
	FPlantProfileFacade::FPlantProfileFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, ProfilePointsAttribute(InCollection, AttributeNames::ProfilePoints, GroupNames::PlantProfilesGroup)
	{
		DefineSchema();
	}

	FPlantProfileFacade::FPlantProfileFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, ProfilePointsAttribute(InCollection, AttributeNames::ProfilePoints, GroupNames::PlantProfilesGroup)
	{
	}

	int32 FPlantProfileFacade::AddProfileEntry(const TArray<float>& InProfilePoints)
	{
		const int32 Index = ProfilePointsAttribute.AddElements(1);
		ProfilePointsAttribute.ModifyAt(Index, InProfilePoints);

		return Index;
	}

	const TArray<float>& FPlantProfileFacade::GetProfilePoints(const int32 Index) const
	{
		if (ProfilePointsAttribute.IsValid() && ProfilePointsAttribute.IsValidIndex(Index))
		{
			return ProfilePointsAttribute[Index];
		}

		static const TArray<float> EmptyArray;
		return EmptyArray;
	}

	void FPlantProfileFacade::DefineSchema()
	{
		check(!IsConst());

		if (!Collection->HasGroup(GroupNames::PlantProfilesGroup))
		{
			Collection->AddGroup(GroupNames::PlantProfilesGroup);
		}

		ProfilePointsAttribute.Add();
	}

	bool FPlantProfileFacade::IsValid() const
	{
		return ProfilePointsAttribute.IsValid();
	}
}
