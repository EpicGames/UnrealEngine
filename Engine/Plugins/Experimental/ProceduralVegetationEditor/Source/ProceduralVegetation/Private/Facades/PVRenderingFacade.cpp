// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVRenderingFacade.h"

namespace PV::Facades
{
	FPVRenderingFacade::FPVRenderingFacade(FManagedArrayCollection& InSelf)
		: FRenderingFacade(InSelf)
		,ConstProceduralVegetationCollection(InSelf)
		,ProceduralVegetationCollection(&InSelf)
		,FoliageFacade(InSelf)
	{}

	FPVRenderingFacade::FPVRenderingFacade(const FManagedArrayCollection& InSelf)
		: FRenderingFacade(InSelf)
		,ConstProceduralVegetationCollection(InSelf)
		,ProceduralVegetationCollection(nullptr)
		,FoliageFacade(InSelf)
	{}

	void FPVRenderingFacade::DefineSchema()
	{
		FRenderingFacade::DefineSchema();
	}

	const FFoliageFacade& FPVRenderingFacade::GetFoliageFacade() const
	{
		return FoliageFacade;
	}

	void FPVRenderingFacade::CopyFrom(const FManagedArrayCollection& Other)
	{
		Other.CopyTo(ProceduralVegetationCollection);
	}
}

