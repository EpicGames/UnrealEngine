// Copyright Epic Games, Inc. All Rights Reserved.

#include "Facades/PVTreeFacade.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVMetaInfoFacade.h"
#include "Facades/PVPointFacade.h"
#include "Helpers/PVAttributesHelper.h"
#include "Helpers/PVPlantTraversalHelper.h"

#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
	FRemoveEntriesResult FTreeFacade::RemoveEntriesAndReIndexAttributes(FManagedArrayCollection& OutCollection, TArray<bool>& PointsToRemove,
		TArray<bool>& BranchesToRemove, TArray<bool>& FoliageInstancesToRemove)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Facades::FTreeFacade::RemoveEntriesAndReIndexAttributes);

		FBranchFacade BranchFacadeOut(OutCollection);
		FPointFacade PointFacadeOut(OutCollection);
		FFoliageFacade FoliageFacadeOut(OutCollection);

		TMap<int32, int32> PointsOldIDsToNewIDs = IShrinkable::RemoveEntries(PointFacadeOut, PointsToRemove);
		TMap<int32, int32> BranchesOldIDsToNewIDs = IShrinkable::RemoveEntries(BranchFacadeOut, BranchesToRemove);
		TMap<int32, int32> FoliageInstancesOldIDsToNewIDs = IShrinkable::RemoveEntries(FoliageFacadeOut, FoliageInstancesToRemove);

		const int32 NumOfBranches = BranchFacadeOut.GetElementCount();
		TSet<int32> CurrentBranchNumbers;
		CurrentBranchNumbers.Reserve(NumOfBranches);
		for (int32 BranchIndex = 0; BranchIndex < NumOfBranches; ++BranchIndex)
		{
			CurrentBranchNumbers.Add(BranchFacadeOut.GetBranchNumber(BranchIndex));
		}

		for (int32 BranchIndex = 0; BranchIndex < NumOfBranches; ++BranchIndex)
		{
			TArray<int32> UpdatedPoints;
			const TArray<int32>& BranchPoints = BranchFacadeOut.GetPoints(BranchIndex);
			for (const int32& PointIndex : BranchPoints)
			{
				if (PointsOldIDsToNewIDs.Contains(PointIndex))
				{
					UpdatedPoints.Add(PointsOldIDsToNewIDs[PointIndex]);
				}
				else if (!PointsToRemove[PointIndex])
				{
					UpdatedPoints.Add(PointIndex);
				}
			}
			BranchFacadeOut.SetPoints(BranchIndex, UpdatedPoints);

			TArray<int32> UpdatedParents;
			const TArray<int32>& BranchParents = BranchFacadeOut.GetParents(BranchIndex);
			for (const int32& BranchNumber : BranchParents)
			{
				if (BranchNumber == 0 || CurrentBranchNumbers.Contains(BranchNumber))
				{
					UpdatedParents.Add(BranchNumber);
				}
			}
			BranchFacadeOut.SetParents(BranchIndex, UpdatedParents);

			TArray<int32> UpdatedChildren;
			const TArray<int32>& BranchChildren = BranchFacadeOut.GetChildren(BranchIndex);
			for (const int32& BranchNumber : BranchChildren)
			{
				if (CurrentBranchNumbers.Contains(BranchNumber))
				{
					UpdatedChildren.Add(BranchNumber);
				}
			}
			BranchFacadeOut.SetChildren(BranchIndex, UpdatedChildren);

			TArray<int32> UpdatedFoliageInstances;
			const TArray<int32>& FoliageInstances = FoliageFacadeOut.GetFoliageEntryIdsForBranch(BranchIndex);
			for (const int32& FoliageInstanceIndex : FoliageInstances)
			{
				if (FoliageInstancesOldIDsToNewIDs.Contains(FoliageInstanceIndex))
				{
					const int32 NewFoliageInstanceID = FoliageInstancesOldIDsToNewIDs[FoliageInstanceIndex];
					UpdatedFoliageInstances.Add(NewFoliageInstanceID);
					FoliageFacadeOut.SetFoliageBranchId(NewFoliageInstanceID, BranchIndex);
				}
				else if (!FoliageInstancesToRemove[FoliageInstanceIndex])
				{
					UpdatedFoliageInstances.Add(FoliageInstanceIndex);
				}
			}
			FoliageFacadeOut.SetFoliageIdsArray(BranchIndex, UpdatedFoliageInstances);
		}

		FRemoveEntriesResult Result = {
			.PointsOldIDsToNewIDs = MoveTemp(PointsOldIDsToNewIDs),
			.BranchesOldIDsToNewIDs = MoveTemp(BranchesOldIDsToNewIDs),
			.FoliageInstancesOldIDsToNewIDs = MoveTemp(FoliageInstancesOldIDsToNewIDs)
		};

		return Result;
	}

	void FTreeFacade::RemoveBranches(const FBranchFacade& InBranchFacade, TArray<int>& InBranchesToRemove, FManagedArrayCollection& OutCollection)
	{
		FBranchFacade BranchFacadeOut(OutCollection);
		FPointFacade PointFacadeOut(OutCollection);
		FFoliageFacade FoliageFacadeOut(OutCollection);

		for (int32 i = InBranchesToRemove.Num() - 1; i >= 0 ; i--)
		{
			GatherChildBranches(InBranchFacade, InBranchesToRemove[i], InBranchesToRemove);
		}
			
		TArray<bool> BranchesToRemove;
		BranchesToRemove.Init(false, BranchFacadeOut.GetElementCount());

		TArray<bool> PointsToRemove;
		PointsToRemove.Init(false, PointFacadeOut.GetElementCount());
	
		TArray<bool> FoliageInstancesToRemove;
		FoliageInstancesToRemove.Init(false, FoliageFacadeOut.GetElementCount());
	
		for (int32 i = 0; i < InBranchesToRemove.Num(); i++)
		{
			const int BranchIndex = InBranchesToRemove[i];
			
			BranchesToRemove[InBranchesToRemove[i]] = true;

			const int Generation = BranchFacadeOut.GetHierarchyGenerationNumber(BranchIndex);
			const int SourceBudNumber = BranchFacadeOut.GetBranchSourceBudNumber(BranchIndex);
			
			for (const int32& P : BranchFacadeOut.GetPoints(InBranchesToRemove[i]))
			{
				if (!PointFacadeOut.IsFusedPoint(Generation, SourceBudNumber, P))
				{
					PointsToRemove[P] = true;
				}
			}
			for (const int32& FId : FoliageFacadeOut.GetFoliageEntryIdsForBranch(InBranchesToRemove[i]))
			{
				FoliageInstancesToRemove[FId] = true;
			}	
		}

		FRemoveEntriesResult RemoveEntriesResult = RemoveEntriesAndReIndexAttributes(
			OutCollection, PointsToRemove, BranchesToRemove, FoliageInstancesToRemove);
	}

	int32 FTreeFacade::GetBranchGenerationNumber(const FManagedArrayCollection& Collection, int32 BranchIndex)
	{
		const auto BudDevelopmentAttribute = PV::FBudDevelopmentAttribute::GetAttribute(Collection);
		const auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(Collection);
		const auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(Collection);
		return PV::AttributesHelper::GetBranchGeneration(
			BudDevelopmentAttribute,
			BranchPointsAttribute,
			BranchParentNumberAttribute,
			BranchIndex
		);
	}

	TArray<float> FTreeFacade::GetVisualizationValues(const FManagedArrayCollection& Collection, ESkeletonVisualizationModes VisualizationMode)
	{
		TArray<float> VisualizationValues;

		switch (VisualizationMode)
		{
		case ESkeletonVisualizationModes::None:
			{
				break;
			}
		case ESkeletonVisualizationModes::GravitationalStress:
			{
				const PV::FBranchPointsAttributeConstView               BranchPointsAttribute         = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBudDirectionAttributeConstView               BudDirectionAttribute         = PV::FBudDirectionAttribute::FindAttribute(Collection);
				const PV::FBudLateralMeristemAttributeConstView         BudLateralMeristemAttribute   = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
				const PV::FDetailAbscissionSenescenseAttributeConstView AbscissionSenescenseAttribute = PV::FDetailAbscissionSenescenseAttribute::FindAttribute(Collection);
				const PV::FBranchParentNumberAttributeConstView         BranchParentNumberAttribute   = PV::FBranchParentNumberAttribute::FindAttribute(Collection);
				const PV::FBranchNumberAttributeConstView               BranchNumberAttribute         = PV::FBranchNumberAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BudDirectionAttribute, BudLateralMeristemAttribute, AbscissionSenescenseAttribute, BranchParentNumberAttribute, BranchNumberAttribute))
				{
					VisualizationValues.SetNum(BudDirectionAttribute.Num());
					const float WeightAbscissionThreshold = 1 - AbscissionSenescenseAttribute[0].GravitationalAbscissionFactor;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints  = BranchPointsAttribute[BranchIndex];
						if (BranchPoints.Num() < 2)
						{
							continue;
						}
						
						const FVector3f FirstBudDirection  = BudDirectionAttribute[BranchPoints[0]].Apical;
						const FVector3f SecondBudDirection = BudDirectionAttribute[BranchPoints[1]].Apical;
						const float BirthParentChildDot    = BudLateralMeristemAttribute[BranchPoints[1]].ParentDot;

						const float GravityParentChildDot = FVector3f::DotProduct(SecondBudDirection, FirstBudDirection);
						const float GravitationalPressure = FMath::GetMappedRangeValueUnclamped(
							FVector2f{0.0f, WeightAbscissionThreshold},
							FVector2f{0.0f, 1.0f},
							BirthParentChildDot - GravityParentChildDot
						);
						const bool bIsTrunk = PV::PlantTraversalHelper::IsTrunk(BranchParentNumberAttribute, BranchIndex);
						const int32 StartIndex = bIsTrunk ? 0 : 1;
						for (int32 BranchPointIndex = StartIndex; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
						{
							VisualizationValues[BranchPoints[BranchPointIndex]] = GravitationalPressure;
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::CellDensity:
			{
				const PV::FBranchPointsAttributeConstView            BranchPointsAttribute       = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBudLateralMeristemAttributeConstView      BudLateralMeristemAttribute = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
				const PV::FDetailLateralElongationAttributeConstView LateralElongationAttribute  = PV::FDetailLateralElongationAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BudLateralMeristemAttribute, LateralElongationAttribute))
				{
					VisualizationValues.SetNum(BudLateralMeristemAttribute.Num());
					const float CellDensityMax = LateralElongationAttribute[0].Elongation * 100.0f;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						for (const int32 BranchPointIndex : BranchPoints)
						{
							const float CurrentLateralMeristem = BudLateralMeristemAttribute[BranchPointIndex].LateralMeristem;
							VisualizationValues[BranchPointIndex] = FMath::GetMappedRangeValueClamped(
								FVector2f{0.0f, CellDensityMax},
								FVector2f{0.0f, 1.0f},
								CurrentLateralMeristem
							);
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::CellWeight:
			{
				const PV::FBranchPointsAttributeConstView            BranchPointsAttribute       = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBudLateralMeristemAttributeConstView      BudLateralMeristemAttribute = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
				const PV::FDetailLateralElongationAttributeConstView LateralElongationAttribute  = PV::FDetailLateralElongationAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BudLateralMeristemAttribute, LateralElongationAttribute))
				{
					VisualizationValues.SetNum(BudLateralMeristemAttribute.Num());
					const float CellWeight = LateralElongationAttribute[0].WhorledImpact;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						for (const int32 BranchPointIndex : BranchPoints)
						{
							const float CurrentLateralWeight = BudLateralMeristemAttribute[BranchPointIndex].Davinci * CellWeight;
							VisualizationValues[BranchPointIndex] = FMath::Clamp(CurrentLateralWeight, 0.0f, 1.0f);
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::LightSenescence:
			{
				const PV::FBranchPointsAttributeConstView               BranchPointsAttribute         = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBranchNumberAttributeConstView               BranchNumberAttribute         = PV::FBranchNumberAttribute::FindAttribute(Collection);
				const PV::FBudDevelopmentAttributeConstView             BudDevelopmentAttribute       = PV::FBudDevelopmentAttribute::FindAttribute(Collection);
				const PV::FDetailAbscissionSenescenseAttributeConstView AbscissionSenescenseAttribute = PV::FDetailAbscissionSenescenseAttribute::FindAttribute(Collection);
				const PV::FBranchParentNumberAttributeConstView         BranchParentNumberAttribute   = PV::FBranchParentNumberAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BranchNumberAttribute, BudDevelopmentAttribute, AbscissionSenescenseAttribute, BranchParentNumberAttribute))
				{
					VisualizationValues.SetNum(BudDevelopmentAttribute.Num());
					const float LightAbscissionMin = AbscissionSenescenseAttribute[0].LightAbscissionMin;
					const float LightAbscissionMax = AbscissionSenescenseAttribute[0].LightAbscissionMax;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						if (BranchPoints.Num() < 2)
						{
							continue;
						}

						const int32 BranchNumber = BranchNumberAttribute[BranchIndex];
						FRandomStream RandomStream(BranchNumber);

						const int32 LightSenescence = BudDevelopmentAttribute[BranchPoints[1]].LightSenescense;
						const float LightAbscissionThreshold = FMath::RoundToInt32(RandomStream.FRandRange(LightAbscissionMin, LightAbscissionMax));
						const float LightSenescenceValue = FMath::GetMappedRangeValueUnclamped(
							FVector2f{0.0f, LightAbscissionThreshold},
							FVector2f{0.0f, 1.0f},
							LightSenescence
						);
						const bool bIsTrunk = PV::PlantTraversalHelper::IsTrunk(BranchParentNumberAttribute, BranchIndex);
						const int32 StartIndex = bIsTrunk ? 0 : 1;
						for (int32 BranchPointIndex = StartIndex; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
						{
							VisualizationValues[BranchPoints[BranchPointIndex]] = LightSenescenceValue;
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::BranchLight:
			{
				const PV::FBranchPointsAttributeConstView       BranchPointsAttribute       = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBudLightDetectedAttributeConstView   BudLightDetectedAttribute   = PV::FBudLightDetectedAttribute::FindAttribute(Collection);
				const PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(Collection);
				const PV::FBranchNumberAttributeConstView       BranchNumberAttribute       = PV::FBranchNumberAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BudLightDetectedAttribute, BranchParentNumberAttribute, BranchNumberAttribute))
				{
					VisualizationValues.SetNum(BudLightDetectedAttribute.Num());
					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						if (BranchPoints.Num() < 2)
						{
							continue;
						}

						const float LightDetected = BudLightDetectedAttribute[BranchPoints[1]].Branch;
						const bool bIsTrunk = PV::PlantTraversalHelper::IsTrunk(BranchParentNumberAttribute, BranchIndex);
						const int32 StartIndex = bIsTrunk ? 0 : 1;
						for (int32 BranchPointIndex = StartIndex; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
						{
							VisualizationValues[BranchPoints[BranchPointIndex]] = LightDetected;
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::LightAbscissionRetention:
			{
				const PV::FBranchPointsAttributeConstView               BranchPointsAttribute         = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBudLateralMeristemAttributeConstView         BudLateralMeristemAttribute   = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
				const PV::FDetailAbscissionSenescenseAttributeConstView AbscissionSenescenseAttribute = PV::FDetailAbscissionSenescenseAttribute::FindAttribute(Collection);
				const PV::FDetailLateralElongationAttributeConstView    LateralElongationAttribute    = PV::FDetailLateralElongationAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BudLateralMeristemAttribute, AbscissionSenescenseAttribute, LateralElongationAttribute))
				{
					VisualizationValues.SetNum(BudLateralMeristemAttribute.Num());
					const float LightRetention    = AbscissionSenescenseAttribute[0].LightScaleRetention;
					const float LateralElongation = LateralElongationAttribute[0].Elongation;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						for (const int32 BranchPointIndex : BranchPoints)
						{
							const float LateralMeristem = BudLateralMeristemAttribute[BranchPointIndex].LateralMeristem;
							const float LightRetentionThreshold = (LateralElongation * 100.0f) * (1 - LightRetention);
							const float LightRetentionValue = FMath::GetMappedRangeValueUnclamped(
								FVector2f{0.0f, LightRetentionThreshold},
								FVector2f{0.0f, 1.0f},
								LateralMeristem
							);
							VisualizationValues[BranchPointIndex] = LightRetentionValue;
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::AgeSenescence:
			{
				const PV::FBranchPointsAttributeConstView               BranchPointsAttribute         = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBranchNumberAttributeConstView               BranchNumberAttribute         = PV::FBranchNumberAttribute::FindAttribute(Collection);
				const PV::FBudDevelopmentAttributeConstView             BudDevelopmentAttribute       = PV::FBudDevelopmentAttribute::FindAttribute(Collection);
				const PV::FDetailAbscissionSenescenseAttributeConstView AbscissionSenescenseAttribute = PV::FDetailAbscissionSenescenseAttribute::FindAttribute(Collection);
				const PV::FBranchParentNumberAttributeConstView         BranchParentNumberAttribute   = PV::FBranchParentNumberAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BranchNumberAttribute, BudDevelopmentAttribute, AbscissionSenescenseAttribute, BranchParentNumberAttribute))
				{
					VisualizationValues.SetNum(BudDevelopmentAttribute.Num());
					const float AgeAbscissionMin = AbscissionSenescenseAttribute[0].AgeAbscissionMin;
					const float AgeAbscissionMax = AbscissionSenescenseAttribute[0].AgeAbscissionMax;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						if (BranchPoints.Num() < 2)
						{
							continue;
						}

						const int32 BranchNumber = BranchNumberAttribute[BranchIndex];
						FRandomStream RandomStream(BranchNumber);

						const int32 AgeSenescence = BudDevelopmentAttribute[BranchPoints[1]].AgeSenescense;
						const float AgeAbscissionThreshold = FMath::RoundToInt32(RandomStream.FRandRange(AgeAbscissionMin, AgeAbscissionMax));
						const float AgeSenescenceValue = FMath::GetMappedRangeValueUnclamped(
							FVector2f{0.0f, AgeAbscissionThreshold},
							FVector2f{0.0f, 1.0f},
							AgeSenescence
						);
						const bool bIsTrunk = PV::PlantTraversalHelper::IsTrunk(BranchParentNumberAttribute, BranchIndex);
						const int32 StartIndex = bIsTrunk ? 0 : 1;
						for (int32 BranchPointIndex = StartIndex; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
						{
							VisualizationValues[BranchPoints[BranchPointIndex]] = AgeSenescenceValue;
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::AgeAbscissionRetention:
			{
				const PV::FBranchPointsAttributeConstView               BranchPointsAttribute         = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBudLateralMeristemAttributeConstView         BudLateralMeristemAttribute   = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
				const PV::FDetailAbscissionSenescenseAttributeConstView AbscissionSenescenseAttribute = PV::FDetailAbscissionSenescenseAttribute::FindAttribute(Collection);
				const PV::FDetailLateralElongationAttributeConstView    LateralElongationAttribute    = PV::FDetailLateralElongationAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BranchPointsAttribute, BudLateralMeristemAttribute, AbscissionSenescenseAttribute, LateralElongationAttribute))
				{
					VisualizationValues.SetNum(BudLateralMeristemAttribute.Num());
					const float AgeRetention    = AbscissionSenescenseAttribute[0].AgeScaleRetention;
					const float LateralElongation = LateralElongationAttribute[0].Elongation;

					for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
					{
						const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
						for (const int32 BranchPointIndex : BranchPoints)
						{
							const float LateralMeristem = BudLateralMeristemAttribute[BranchPointIndex].LateralMeristem;
							const float AgeRetentionThreshold = (LateralElongation * 100.0f) * (1 - AgeRetention);
							const float AgeRetentionValue = FMath::GetMappedRangeValueUnclamped(
								FVector2f{0.0f, AgeRetentionThreshold},
								FVector2f{0.0f, 1.0f},
								LateralMeristem
							);
							VisualizationValues[BranchPointIndex] = AgeRetentionValue;
						}
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::Apical:
			{
				const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BudHormoneLevelsAttribute))
				{
					VisualizationValues.SetNum(BudHormoneLevelsAttribute.Num());
					for (int32 Index = 0; Index < BudHormoneLevelsAttribute.Num(); ++Index)
					{
						VisualizationValues[Index] = BudHormoneLevelsAttribute[Index].Apical;
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::Auxin:
			{
				const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BudHormoneLevelsAttribute))
				{
					VisualizationValues.SetNum(BudHormoneLevelsAttribute.Num());
					for (int32 Index = 0; Index < BudHormoneLevelsAttribute.Num(); ++Index)
					{
						VisualizationValues[Index] = BudHormoneLevelsAttribute[Index].Axillary;
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::Radical:
			{
				const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BudHormoneLevelsAttribute))
				{
					VisualizationValues.SetNum(BudHormoneLevelsAttribute.Num());
					for (int32 Index = 0; Index < BudHormoneLevelsAttribute.Num(); ++Index)
					{
						VisualizationValues[Index] = BudHormoneLevelsAttribute[Index].Radical;
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::Ethylene:
			{
				const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BudHormoneLevelsAttribute))
				{
					VisualizationValues.SetNum(BudHormoneLevelsAttribute.Num());
					for (int32 Index = 0; Index < BudHormoneLevelsAttribute.Num(); ++Index)
					{
						VisualizationValues[Index] = BudHormoneLevelsAttribute[Index].Ethylene;
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::AuxinRetention:
			{
				const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(Collection);
				const PV::FDetailLeafGrowthAttributeConstView LeafGrowthAttribute       = PV::FDetailLeafGrowthAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BudHormoneLevelsAttribute, LeafGrowthAttribute))
				{
					VisualizationValues.SetNum(BudHormoneLevelsAttribute.Num());
					const float AuxinRetention = LeafGrowthAttribute[0].EthyleneThreshold;

					for (int32 Index = 0; Index < BudHormoneLevelsAttribute.Num(); ++Index)
					{
						const PV::FBudHormoneLevelsConstView HormoneLevels = BudHormoneLevelsAttribute[Index];
						const float EthyleneLevel = FMath::Clamp(HormoneLevels.Ethylene, 0.0f, 1.0f);
						const float ApicalAuxin = 1.0f - FMath::Lerp(EthyleneLevel, 0, HormoneLevels.Apical);
						VisualizationValues[Index] = FMath::Lerp(0, ApicalAuxin, AuxinRetention);
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::Cytokinin:
			{
				const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(BudHormoneLevelsAttribute))
				{
					VisualizationValues.SetNum(BudHormoneLevelsAttribute.Num());
					for (int32 Index = 0; Index < BudHormoneLevelsAttribute.Num(); ++Index)
					{
						VisualizationValues[Index] = BudHormoneLevelsAttribute[Index].Cytokinin;
					}
				}
				break;
			}
		case ESkeletonVisualizationModes::BranchGeneration:
			{
				const PV::FPointPositionAttributeConstView      PointPositionAttribute      = PV::FPointPositionAttribute::FindAttribute(Collection);
				const PV::FBranchPointsAttributeConstView       BranchPointsAttribute       = PV::FBranchPointsAttribute::FindAttribute(Collection);
				const PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(Collection);
				const PV::FBranchNumberAttributeConstView       BranchNumberAttribute       = PV::FBranchNumberAttribute::FindAttribute(Collection);
				const PV::FBudDevelopmentAttributeConstView     BudDevelopmentAttribute     = PV::FBudDevelopmentAttribute::FindAttribute(Collection);
				if (PV::ValidateAttributeCollection(
					PointPositionAttribute, 
					BranchPointsAttribute, 
					BranchParentNumberAttribute, 
					BranchNumberAttribute, 
					BudDevelopmentAttribute
				))
				{
					int32 MaxGeneration = 1;
					for (int32 i = 0; i < BudDevelopmentAttribute.Num(); ++i)
					{
						MaxGeneration = FMath::Max(MaxGeneration, BudDevelopmentAttribute[i].Generation);
					}

					VisualizationValues.SetNum(PointPositionAttribute.Num());

					for (int32 PointIndex = 0; PointIndex < BudDevelopmentAttribute.Num(); ++PointIndex)
					{
						const int32 GenerationNumber = BudDevelopmentAttribute[PointIndex].Generation;
						VisualizationValues[PointIndex] = (float)GenerationNumber / MaxGeneration;
					}
				}
				break;
			}
		default:
			{
				break;
			}
		}
		
		return VisualizationValues;
	}

	void FTreeFacade::GatherChildBranches(const FBranchFacade& InBranchFacade, const int InParentIndex, TArray<int32>& OutBranchesToRemove)
	{
		const TArray<int32>& BranchChildren = InBranchFacade.GetChildren(InParentIndex);
		const TManagedArray<int32>& BranchNumbers = InBranchFacade.GetBranchNumbers();
		
		for (const int32& BranchNumber : BranchChildren)
		{
			const int32 BranchIndex = BranchNumbers.Find(BranchNumber);
			
			if (BranchIndex != INDEX_NONE && !OutBranchesToRemove.Contains(BranchIndex))
			{
				OutBranchesToRemove.Add(BranchIndex);
			}

			GatherChildBranches(InBranchFacade, BranchIndex, OutBranchesToRemove);
		}
	}
}
