// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Helpers/PVAnalyticsHelper.h"
 
#include "EngineAnalytics.h"
#include "ProceduralVegetationModule.h"
#include "PVDistributionHelper.h"
#include "Params/PVExportParams.h"
#include "Engine/RendererSettings.h"
#include "Facades/PVFoliageFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PVUtilities.h"
#include "Implementations/PVFoliage.h"
#include "UObject/UnrealTypePrivate.h"
 
namespace PV::Analytics
{
	void LogEvent(const FString& InEventName, const TArray<FAnalyticsEventAttribute>& InAttributes = TArray<FAnalyticsEventAttribute>())
	{
		FString AttributesData;
		for (int32 i = 0; i < InAttributes.Num(); i++)
		{
			AttributesData += FString::Printf(TEXT("(%s,%s),"), *InAttributes[i].GetName(), *InAttributes[i].GetValue());
		}
			
		UE_LOGF(LogProceduralVegetation, Display, "Event: %ls : Attributes: {%ls}", *InEventName, *AttributesData);
	}
	
	void SendEvent(const FString& InEventName, const TArray<FAnalyticsEventAttribute>& InAttributes = TArray<FAnalyticsEventAttribute>())
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FString EventName = FString::Printf(TEXT("Editor.Usage.PVE.%s"), *InEventName);
			FEngineAnalytics::GetProvider().RecordEvent(EventName, InAttributes);
		}
	}
 
	void SendSessionStartedEvent()
	{
		if (FEngineAnalytics::IsAvailable())
		{
			SendEvent(TEXT("SessionStarted"));
		}
	}
 
	void SendSessionEndedEvent(const double InTimeInSeconds)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("TimeActiveSeconds"),  InTimeInSeconds));
			
			SendEvent(TEXT("SessionEnded"), Attributes);
		}
	}
 
	void SendNodeAddedEvent(const FString& InNodeType)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Name"),  InNodeType));
		
			SendEvent(TEXT("NodeAdded"), Attributes);
			LogEvent(TEXT("NodeAdded"), Attributes);
		}
	}
 
	void SendNodeTweakedEvent(const FString& InNodeType, const FProperty* InProperty)
	{
		if (FEngineAnalytics::IsAvailable() && InProperty)
		{
			FString PropertyName = InProperty->GetName();
 
			FString PropertyCategory = "N/A";
#if WITH_METADATA
			PropertyCategory = InProperty->GetMetaData(TEXT("Category"));
#endif
			
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Name"),  InNodeType));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("PropertyCategory"),  PropertyCategory));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("PropertyName"),  PropertyName));
		
			SendEvent(TEXT("NodeTweaked"), Attributes);
			LogEvent(TEXT("NodeTweaked"), Attributes);
		}
	}
 
	void SendMaterialChangeEvent(FString InMatPath)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Path"),  InMatPath));
			
			SendEvent(TEXT("MeshMaterialChanged"), Attributes);
			LogEvent(TEXT("MeshMaterialChanged"), Attributes);
		}
	}
 
	void SendFoliageMeshChangeEvent(FString InFoliageMeshPath)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Path"),  InFoliageMeshPath));
			
			SendEvent(TEXT("FoliageMeshChanged"), Attributes);
			LogEvent(TEXT("FoliageMeshChanged"), Attributes);
		}
	}
 
	void SendWindSettingsChangeEvent(FString InPath)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Path"),  InPath));
			
			SendEvent(TEXT("WindPresetChanged"), Attributes);
			LogEvent(TEXT("WindPresetChanged"), Attributes);
		}
	}
 
	void GatherFoliageMeshesTriangles(const Facades::FFoliageFacade& InFoliageFacade, uint32& OutUniqueFoliageTriangles, uint32& OutTotalFoliageTriangles)
	{
		OutUniqueFoliageTriangles = 0;
		OutTotalFoliageTriangles = 0;
		
		for (int32 FoliageIndex = 0; FoliageIndex < InFoliageFacade.NumFoliageInfo(); ++FoliageIndex)
		{
			FPVFoliageInfo Foliage = InFoliageFacade.GetFoliageInfo(FoliageIndex);
			
			OutUniqueFoliageTriangles += PV::Utilities::GetMeshTriangles(Foliage.Mesh.ToString());
		}
 
		int32 NumInstances = InFoliageFacade.NumFoliageEntries();
		
		for (int32 Id = 0; Id < NumInstances; Id++)
		{
			Facades::FFoliageEntryData Data = InFoliageFacade.GetFoliageEntry(Id);
			FPVFoliageInfo FoliageInfo = InFoliageFacade.GetFoliageInfo(Data.NameId);
 
			OutTotalFoliageTriangles += PV::Utilities::GetMeshTriangles(FoliageInfo.Mesh.ToString());
		}
	}
 
	TArray<FAnalyticsEventAttribute> GatherExportCommonAttributes(const FManagedArrayCollection& InCollection, const FPVExportParams& InExportParam)
	{
		TArray<FAnalyticsEventAttribute> ExportAttributes;
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("ExportSettings.MeshType"),  StaticEnum<EPVExportMeshType>()->GetNameStringByValue((uint8)InExportParam.ExportMeshType)));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("ExportSettings.CreateNaniteFoliage"),  InExportParam.bCreateNaniteFoliage ? TEXT("True") : TEXT("False")));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("ExportSettings.ReplacePolicy"), StaticEnum<EPVAssetReplacePolicy>()->GetNameStringByValue((uint8)InExportParam.ReplacePolicy)));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("ExportSettings.NaniteShapePreservation"),  StaticEnum<ENaniteShapePreservation>()->GetNameStringByValue((uint8)InExportParam.NaniteShapePreservation)));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("NaniteSettings.NaniteFoliageEnabled"),  GetDefault<URendererSettings>()->bEnableNaniteFoliage));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("NaniteSettings.Enabled"),  GetDefault<URendererSettings>()->bNanite));
 
		Facades::FFoliageFacade FoliageFacade(InCollection);
		
		const int FoliageInstances = FoliageFacade.GetElementCount();
		const int TrunkFaces = InCollection.NumElements(FGeometryCollection::FacesGroup);
 
		uint32 UniqueFoliageTriangles = 0;
		uint32 TotalFoliageTriangles = 0;
		GatherFoliageMeshesTriangles(FoliageFacade, UniqueFoliageTriangles, TotalFoliageTriangles);
		
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("FoliageInstances"),  FoliageInstances));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("TrunkTriangles"),  TrunkFaces));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("FoliageTrianglesUnique"),  UniqueFoliageTriangles));
		ExportAttributes.Add(FAnalyticsEventAttribute(TEXT("FoliageTrianglesTotal"),  TotalFoliageTriangles));
 
		return ExportAttributes;
	}
 
	void SendExportStartEvent(const TArray<FAnalyticsEventAttribute>& InCommonAttributes)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			SendEvent(TEXT("ExportStarted"), InCommonAttributes);
			LogEvent(TEXT("ExportStarted"), InCommonAttributes);
		}
	}
 
	void SendExportFinishedEvent(TArray<FAnalyticsEventAttribute> InCommonAttributes, float InTotalTimeInSeconds, EExportResult ExportResult)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			InCommonAttributes.Add(FAnalyticsEventAttribute(TEXT("TotalTimeSeconds"), InTotalTimeInSeconds));
 
			const static auto ExportResultEnumToString = [](EExportResult InExportResult)->FString {
				switch (InExportResult)
				{
				case EExportResult::Failed:
					return TEXT("Failed");
				case EExportResult::Success:
					return TEXT("Success");
				case EExportResult::Skipped:
					return TEXT("Skipped");
				}
 
				return TEXT("Unknown");
			};
			
			InCommonAttributes.Add(FAnalyticsEventAttribute(TEXT("ExportResult"), ExportResultEnumToString(ExportResult)));
 
			SendEvent(TEXT("ExportFinished"), InCommonAttributes);
			LogEvent(TEXT("ExportFinished"), InCommonAttributes);
		}
	}
}