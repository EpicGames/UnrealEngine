// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PVAttributesHelper.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "Implementations/PVRecomputePointScale.h"
#include "Chaos/Convex.h"

float PV::AttributesHelper::GetBranchPointBranchGradient(
	PV::FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
	PV::FBranchPointsAttributeConstView BranchPointsAttribute,
	int32 BranchIndex, 
	int32 BranchPointIndex
)
{
	if (!BranchPointsAttribute.IsValidIndex(BranchIndex) || !BranchPointsAttribute[BranchIndex].IsValidIndex(BranchPointIndex))
	{
		return 0;
	}

	const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
	if (BranchPoints.Num() < 2)
	{
		return 0;
	}

	const int32 PointIndex = BranchPoints[BranchPointIndex];

	if (!PointLengthFromRootAttribute.IsValidIndex(BranchPoints[0])
		|| !PointLengthFromRootAttribute.IsValidIndex(BranchPoints.Last())
		|| !PointLengthFromRootAttribute.IsValidIndex(PointIndex))
	{
		return 0;
	}

	const float BaseLengthFromRoot = PointLengthFromRootAttribute[BranchPoints[0]];
	const float TipLengthFromRoot = PointLengthFromRootAttribute[BranchPoints.Last()];
	const float LengthFromRoot = PointLengthFromRootAttribute[PointIndex];

	return FMath::GetMappedRangeValueClamped(
		FVector2f(BaseLengthFromRoot, TipLengthFromRoot),
		FVector2f(0.0f, 1.0f),
		LengthFromRoot
	);
}

float PV::AttributesHelper::GetBranchPointScale(
	PV::FPointScaleAttributeConstView PointScaleAttribute,
	PV::FBranchPointsAttributeConstView BranchPointsAttribute,
	PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
	int32 BranchIndex,
	int32 BranchPointIndex
)
{
	if (!BranchPointsAttribute.IsValidIndex(BranchIndex) || !BranchPointsAttribute[BranchIndex].IsValidIndex(BranchPointIndex))
	{
		return 0.0f;
	}

	const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
	const int32 PointIndex = BranchPoints[BranchPointIndex];
	const bool bIsRootFusedPoint = BranchPointIndex == 0
		&& !PV::PlantTraversalHelper::IsTrunk(BranchParentNumberAttribute, BranchIndex);

	if (bIsRootFusedPoint && BranchPoints.IsValidIndex(1))
	{
		const float S1 = PointScaleAttribute[BranchPoints[1]];
		const float ParentScale = PointScaleAttribute[PointIndex];
		const float Elongation = BranchPoints.IsValidIndex(2) ? S1 - PointScaleAttribute[BranchPoints[2]] : 0.0f;
		const float NewScale = S1 + Elongation / UE_PI;

		if (NewScale <= 0.0f)
		{
			// Find the parent branch's scale at the point just before the junction so the
			// fallback uses the parent's own taper step instead of an arbitrary 0.8 factor.
			const int32 JunctionPointIdx = BranchPoints[0];
			for (int32 i = 0; i < BranchPointsAttribute.Num(); ++i)
			{
				if (i == BranchIndex)
				{
					continue;
				}
				const TArray<int32>& OtherPts = BranchPointsAttribute[i];
				const int32 JunctionPosInParent = OtherPts.Find(JunctionPointIdx);
				if (JunctionPosInParent > 0)
				{
					return PointScaleAttribute[OtherPts[JunctionPosInParent - 1]] - ParentScale;
				}
			}
			return ParentScale * 0.8f;
		}

		return NewScale;
	}

	return PointScaleAttribute[PointIndex];
}

TArray<FVector3f> PV::AttributesHelper::GetBudDirection(
	PV::FBudDirectionAttributeConstView BudDirectionAttribute,
	PV::FBranchPointsAttributeConstView BranchPointsAttribute,
	PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
	PV::FPointPositionAttributeConstView PointPositionAttribute,
	const int32 BranchIndex,
	const int32 BranchPointIndex)
{
	if (!BranchPointsAttribute.IsValidIndex(BranchIndex))
	{
		return TBudDirectionView<FVector3f>::DefaultData;
	}

	const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
	const bool bIsTrunk = BranchParentNumberAttribute.IsValidIndex(BranchIndex) && BranchParentNumberAttribute[BranchIndex] == 0;
	const bool bBranchFusedPoint = !bIsTrunk && BranchPointIndex == 0;

	if (bBranchFusedPoint && BranchPoints.IsValidIndex(1))
	{
		const int32 CurrentPointIndex = BranchPoints[0];
		const int32 NextPointIndex = BranchPoints[1];
		
		if (ensure(PointPositionAttribute.IsValidIndex(CurrentPointIndex) && PointPositionAttribute.IsValidIndex(NextPointIndex) && BudDirectionAttribute.IsValidIndex(NextPointIndex)))
		{
			const FVector3f Dir = (PointPositionAttribute[NextPointIndex] - PointPositionAttribute[CurrentPointIndex]).GetSafeNormal();
			const FBudDirectionConstView NextBudDirection = BudDirectionAttribute[NextPointIndex];
			const FQuat4f Rotation = FQuat4f::FindBetweenNormals(NextBudDirection.Apical, Dir);

			TArray<FVector3f> Result = NextBudDirection.Array;
			FBudDirectionView ResultView(Result);
			ResultView.Apical = Dir;
			ResultView.Axillary = Rotation.RotateVector(NextBudDirection.Axillary);
			ResultView.UpVector = Rotation.RotateVector(NextBudDirection.UpVector);
			return Result;
		}
		else
		{
			return TBudDirectionView<FVector3f>::DefaultData;
		}
	}
	
	if (!BranchPoints.IsValidIndex(BranchPointIndex))
	{
		return TBudDirectionView<FVector3f>::DefaultData;
	}

	const int32 PointIndex = BranchPoints[BranchPointIndex];
	return BudDirectionAttribute.IsValidIndex(PointIndex) ? BudDirectionAttribute[PointIndex].Array : TBudDirectionView<FVector3f>::DefaultData;
}

TArray<int32> PV::AttributesHelper::GetBudDevelopment(
	PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute,
	PV::FBranchPointsAttributeConstView BranchPointsAttribute,
	PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
	const int32 BranchIndex,
	const int32 BranchPointIndex)
{
	if (!BranchPointsAttribute.IsValidIndex(BranchIndex))
		return FBudDevelopmentView::DefaultData;

	const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
	const bool bIsTrunk = BranchParentNumberAttribute.IsValidIndex(BranchIndex) && BranchParentNumberAttribute[BranchIndex] == 0;
	const bool bBranchFusedPoint = !bIsTrunk && BranchPointIndex == 0;

	if (bBranchFusedPoint)
	{
		// Some operations can leave behind branches with only 1 point.
		// This is technically not a valid state and should be resolved,
		// but for now we make sure the [1] index is valid.
		if (BranchPoints.IsValidIndex(1))
		{
			const int32 FirstOwnedPointIndex = BranchPoints[1];
			
			return BudDevelopmentAttribute.IsValidIndex(FirstOwnedPointIndex)
				? BudDevelopmentAttribute[FirstOwnedPointIndex].Array
				: FBudDevelopmentView::DefaultData;
		}
	}

	if (!BranchPoints.IsValidIndex(BranchPointIndex))
		return FBudDevelopmentView::DefaultData;

	const int32 PointIndex = BranchPoints[BranchPointIndex];
	
	return BudDevelopmentAttribute.IsValidIndex(PointIndex) ? BudDevelopmentAttribute[PointIndex].Array : FBudDevelopmentView::DefaultData;
}

int32 PV::AttributesHelper::GetBranchGeneration(
	PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute,
	PV::FBranchPointsAttributeConstView BranchPointsAttribute,
	PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
	int32 BranchIndex)
{
	if (!ValidateAttributeCollection(
		BudDevelopmentAttribute,
		BranchPointsAttribute,
		BranchParentNumberAttribute
	))
	{
		return INDEX_NONE;
	}

	if (!BranchPointsAttribute.IsValidIndex(BranchIndex))
	{
		return INDEX_NONE;
	}

	const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
	if (BranchPoints.Num() == 0)
	{
		return INDEX_NONE;
	}

	const int32 BranchPointIndex = BranchPoints.Last();

	if (!BudDevelopmentAttribute.IsValidIndex(BranchPointIndex))
	{
		return INDEX_NONE;
	}

	if (BranchPoints.Num() == 1) // Handle edge-case where we have branches with only 1 point
	{
		const bool bIsTrunk = PlantTraversalHelper::IsTrunk(BranchParentNumberAttribute, BranchIndex);
		if (!bIsTrunk)
		{
			return BudDevelopmentAttribute[BranchPointIndex].Generation + 1;
		}
	}

	return BudDevelopmentAttribute[BranchPointIndex].Generation;
}

void PV::AttributesHelper::ComputeBudNumbers(const FComputeBudNumbersAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBudNumbers);

	for (int32 PointIndex = 0; PointIndex < InAttributes.PointBudNumber.Num(); ++PointIndex)
	{
		InAttributes.PointBudNumber[PointIndex] = PointIndex + 1;
	}

	for (int32 BranchIndex = 0; BranchIndex < InAttributes.BranchSourceBudNumber.Num(); ++BranchIndex)
	{
		if (!ensure(InAttributes.BranchPoints[BranchIndex].Num() > 0))
		{
			continue;
		}

		const int32 BranchRootPointIndex = InAttributes.BranchPoints[BranchIndex][0];
		InAttributes.BranchSourceBudNumber[BranchIndex] = InAttributes.PointBudNumber[BranchRootPointIndex];
	}
}

void PV::AttributesHelper::ComputeNjordPixelIndex(const FComputeNjordPixelIndexAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeNjordPixelIndex);

	if (!InAttributes.IsValid())
	{
		return;
	}

	TMap<int32, TArray<int32>> BudNumberPointIndicesMap;
	for (int32 PointIndex = 0; PointIndex < InAttributes.PointBudNumber.Num(); ++PointIndex)
	{
		const int32 BudNumber = InAttributes.PointBudNumber[PointIndex];
		BudNumberPointIndicesMap.FindOrAdd(BudNumber).Add(PointIndex);
	}

	for (const auto& [BudNumber, PointIndices] : BudNumberPointIndicesMap)
	{
		const int32 Count = PointIndices.Num();
		for (int32 i = 0; i < Count; ++i)
		{
			InAttributes.PointNjordPixelIndex[PointIndices[i]] = BudNumber + static_cast<float>(i) / Count;
		}
	}
}

void PV::AttributesHelper::ComputeLengthFromRoot(const FComputeLengthFromRootAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeLengthFromRoot);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	const FBranchNumberToIndexTable BranchNumberToIndex(InAttributes.BranchNumber);
	RecursiveWalkBranches(InAttributes.BranchParentNumber, InAttributes.BranchChildren, BranchNumberToIndex, [&](int32 BranchIndex)->EForEachResult
	{
		const TArray<int32>& CurrentBranchPoints = InAttributes.BranchPoints[BranchIndex];
		if (!ensure(CurrentBranchPoints.Num() > 0))
		{
			return EForEachResult::Continue;
		}

		const int32 FirstPointIndex = CurrentBranchPoints[0];
		const int32 ParentBranchIndex = GetBranchParentIndex(InAttributes.BranchParentNumber, BranchNumberToIndex, BranchIndex);

		if (ParentBranchIndex == INDEX_NONE)
		{
			InAttributes.PointLengthFromRoot[FirstPointIndex] = 0.f; // No parent, length to root is 0
			InAttributes.PointLengthFromSeed[FirstPointIndex] = 0.f;
		}

		for (int32 i = 1; i < CurrentBranchPoints.Num(); ++i)
		{
			const int32 PointIndex = CurrentBranchPoints[i];
			const int32 PrevPointIndex = CurrentBranchPoints[i - 1];

			const FVector3f& Position = InAttributes.PointPosition[PointIndex];
			const FVector3f& PrevPosition = InAttributes.PointPosition[PrevPointIndex];

			InAttributes.PointLengthFromRoot[PointIndex] = InAttributes.PointLengthFromRoot[PrevPointIndex] + (Position - PrevPosition).Size() * 0.01 /*cm->m*/;
			InAttributes.PointLengthFromSeed[PointIndex] = InAttributes.PointLengthFromRoot[PointIndex];
		}

		return EForEachResult::Continue;
	});
}

void PV::AttributesHelper::ComputeLengthFromTrunk(const FComputeLengthFromTrunkAttributes& InAttributes, TArrayView<float> OutLengthFromTunk)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeLengthFromTrunk);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid() || OutLengthFromTunk.Num() != InAttributes.PointLengthFromRoot.Num())
	{
		return;
	}

	const FBranchNumberToIndexTable BranchNumberToIndex(InAttributes.BranchNumber);
	RecursiveWalkBranches(InAttributes.BranchParentNumber, InAttributes.BranchChildren, BranchNumberToIndex, [&](int32 BranchIndex) -> EForEachResult
	{
		const TArray<int32>& CurrentBranchPoints = InAttributes.BranchPoints[BranchIndex];
		if (!ensure(CurrentBranchPoints.Num() > 0))
		{
			return EForEachResult::Continue;
		}

		const bool bIsTrunk = InAttributes.BudDevelopment[CurrentBranchPoints.Last()].Generation == 1;
		if (IsTrunk(InAttributes.BranchParentNumber, BranchIndex))
		{
			for (const int32 PointIndex : CurrentBranchPoints)
			{
				OutLengthFromTunk[PointIndex] = 0.f;
			}
		}
		else
		{
			// CurrentBranchPoints[0] is the fused point shared with the parent branch;
			// its LengthFromTrunk is already set from the parent traversal.
			for (int32 i = 1; i < CurrentBranchPoints.Num(); ++i)
			{
				const int32 PointIndex = CurrentBranchPoints[i];
				const int32 PrevPointIndex = CurrentBranchPoints[i - 1];

				OutLengthFromTunk[PointIndex] = OutLengthFromTunk[PrevPointIndex] + (InAttributes.PointLengthFromRoot[PointIndex] - InAttributes.PointLengthFromRoot[PrevPointIndex]);
			}
		}

		return EForEachResult::Continue;
	});
}

void PV::AttributesHelper::ComputeBudDevelopment(const FComputeBudDevelopmentAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBudDevelopment);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	int32 MaxBudAge = 0;
	for (int32 BranchIndex = 0; BranchIndex < InAttributes.BranchPoints.Num(); ++BranchIndex)
	{
		MaxBudAge = FMath::Max(MaxBudAge, InAttributes.BranchPoints[BranchIndex].Num());
	}

	ForEachUniquePointOnBranches(
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		[&](int32 BranchIndex, int32 BranchPointIndex, int32 PointIndex)
		{
			const float Gradient = InAttributes.PointPlantGradient[PointIndex];
			const int32 BudAge = FMath::RoundToInt(FMath::GetMappedRangeValueClamped(FVector2f(0, 1), FVector2f(1, float(MaxBudAge)), Gradient));

			InAttributes.BudDevelopment[PointIndex].BudAge = BudAge;
			InAttributes.BudDevelopment[PointIndex].RelativeBudAge = BudAge;
			InAttributes.BudDevelopment[PointIndex].Generation = InAttributes.BranchParents[BranchIndex].Num();

			return EForEachResult::Continue;
		}
	);

	const FBranchNumberToIndexTable BranchNumberToIndex(InAttributes.BranchNumber);
	ForEachUniquePointOnBranches(
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		[&](int32 BranchIndex, int32 BranchPointIndex, int32 PointIndex)
		{
			const bool bIsRootBranch = GetBranchParentIndex(InAttributes.BranchParentNumber, BranchNumberToIndex, BranchIndex) == INDEX_NONE;
			const int32 RootPointIndex = bIsRootBranch ? InAttributes.BranchPoints[BranchIndex][0] : InAttributes.BranchPoints[BranchIndex][1];
			const int32 BranchAge = InAttributes.BudDevelopment[RootPointIndex].BudAge;
			InAttributes.BudDevelopment[PointIndex].BranchAge = BranchAge;

			return EForEachResult::Continue;
		}
	);
}

void PV::AttributesHelper::ComputeBudDirections(const FComputeBudDirectionsAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBudDirections);

	using namespace PV::PlantTraversalHelper;

	ForEachUniquePointOnBranches(
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		[&](int32 BranchIndex, int32 BranchPointIndex, int32 PointIndex)
		{
			const TArray<int32>& BranchPoints = InAttributes.BranchPoints[BranchIndex];

			const FVector3f Apical = Invoke([&]()
			{
				if (BranchPointIndex == 0)
				{
					const int32 NextPointIndex = BranchPoints[BranchPointIndex + 1];
					return (InAttributes.PointPosition[NextPointIndex] - InAttributes.PointPosition[PointIndex]).GetSafeNormal();
				}
				else if (ensure(BranchPointIndex < BranchPoints.Num()))
				{
					const int32 PrevPointIndex = BranchPoints[BranchPointIndex - 1];
					return (InAttributes.PointPosition[PointIndex] - InAttributes.PointPosition[PrevPointIndex]).GetSafeNormal();
				}

				return FVector3f::ZeroVector;
			});

			InAttributes.BudDirection[PointIndex].Apical = Apical;
			InAttributes.BudDirection[PointIndex].LightOptimal = Apical;
			InAttributes.BudDirection[PointIndex].LightSubOptimal = Apical;
			InAttributes.BudDirection[PointIndex].GuideCurve = Apical;

			return EForEachResult::Continue;
		}
	);

	struct FWalkPlantPointsState
	{
		int32 AxillaryCount = 0;
		int32 PrevPointIndex = INDEX_NONE;
	};
	RecursiveWalkPlantPoints<FWalkPlantPointsState>(
		InAttributes.BranchChildren,
		InAttributes.BranchParentNumber,
		InAttributes.BranchNumber,
		InAttributes.BranchPoints,
		FWalkPlantPointsState(),
		[&](const FRecursiveWalkBranchPointsParams& Params, FWalkPlantPointsState& State)->EForEachResult
		{
			const TArray<int32>& BranchPoints = InAttributes.BranchPoints[Params.BranchIndex];

			const FVector3f Apical = InAttributes.BudDirection[Params.PointIndex].Apical;
			const FVector3f UpVector = FRotationMatrix44f::MakeFromZX(Apical, FVector3f::ForwardVector).ToQuat().GetRightVector();
			const FVector3f Axillary = Invoke([&]()
			{
				const bool bIsRootPoint = State.PrevPointIndex == INDEX_NONE;
				if (bIsRootPoint)
				{
					State.AxillaryCount = 0;
					return FVector3f::ForwardVector;
				}

				if (Params.ChildPointIndices.Num() > 0)
				{
					State.AxillaryCount = 0;

					const FVector3f& ChildApical = InAttributes.BudDirection[Params.ChildPointIndices[0]].Apical;
					return FRotationMatrix44f::MakeFromZX(Apical, ChildApical).ToQuat().GetForwardVector();
				}

				State.AxillaryCount += 1;

				return FQuat4f(Apical, FMath::DegreesToRadians(137.5f * State.AxillaryCount)).RotateVector(UpVector);
			});

			InAttributes.BudDirection[Params.PointIndex].Axillary = Axillary;
			InAttributes.BudDirection[Params.PointIndex].UpVector = UpVector;
			
			State.PrevPointIndex = Params.PointIndex;

			return EForEachResult::Continue;
		}
	);
}

void PV::AttributesHelper::ComputeBudHormoneLevels(const FComputeBudHormoneLevelsAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBudHormoneLevels);

	using namespace PV::PlantTraversalHelper;
		
	for (int32 PointIndex = 0; PointIndex < InAttributes.BudHormoneLevels.Num(); PointIndex++)
	{
		const float Gradient = InAttributes.PointPlantGradient[PointIndex];
		const float InverseGradient = 1.f - Gradient;

		InAttributes.BudHormoneLevels[PointIndex].Apical = 1;
		InAttributes.BudHormoneLevels[PointIndex].Axillary = FMath::GetMappedRangeValueClamped(FVector2f(0, 1), FVector2f(0.35f, 1.0f), InverseGradient);
		InAttributes.BudHormoneLevels[PointIndex].AxillaryInhibition = 0;
		InAttributes.BudHormoneLevels[PointIndex].Radical = 0;
		InAttributes.BudHormoneLevels[PointIndex].Ethylene = Gradient;
		InAttributes.BudHormoneLevels[PointIndex].Cytokinin = 0;
	}
}

void PV::AttributesHelper::ComputePointGroundGradient(const FComputePointGroundGradientAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputePointGroundGradient);

	if (!InAttributes.IsValid())
	{
		return;
	}

	const FVector3f* MaxZVertex = Algo::MaxElementBy(InAttributes.PointPosition, [&](const auto& A) { return A.Z; });
	if (MaxZVertex == nullptr || MaxZVertex->Z == 0)
	{
		return;
	}

	const float MaxZ = MaxZVertex->Z;
	for (int32 PointIndex = 0; PointIndex < InAttributes.PointGroundGradient.Num(); PointIndex++)
	{
		InAttributes.PointGroundGradient[PointIndex] = 1.f - (InAttributes.PointPosition[PointIndex].Z / MaxZ);
	}
}

void PV::AttributesHelper::ComputePointScaleGradient(const FComputePointScaleGradientAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputePointScaleGradient);

	if (!InAttributes.IsValid())
	{
		return;
	}

	const float* MaxPointScale = Algo::MaxElement(InAttributes.PointScale);
	if (MaxPointScale == nullptr || *MaxPointScale == 0)
	{
		return;
	}

	for (int32 PointIndex = 0; PointIndex < InAttributes.PointScaleGradient.Num(); PointIndex++)
	{
		const float PointScale = InAttributes.PointScale[PointIndex];
		const float PointScaleAlpha = PointScale / *MaxPointScale;
		InAttributes.PointScaleGradient[PointIndex] = PointScaleAlpha;
	}
}

void PV::AttributesHelper::ComputePointPlantGradient(const FComputePointPlantGradientAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputePointPlantGradient);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	// PlantGradient - A gradient from 1-0 where the tip of each branch is 0, and the base of each branch is whatever the gradient of the parent branch is at that point.
	RecursiveWalkBranches(
		InAttributes.BranchParentNumber,
		InAttributes.BranchChildren,
		InAttributes.BranchNumber,
		[&](int32 BranchIndex)->EForEachResult
		{
			const TArray<int32>& BranchPoints = InAttributes.BranchPoints[BranchIndex];
			if (!ensure(BranchPoints.Num() > 0))
			{
				return EForEachResult::Continue;
			}

			const bool bIsTrunk = IsTrunk(InAttributes.BranchParentNumber, BranchIndex);
			const float BasePointPlantGradient = bIsTrunk ? 1.0 : InAttributes.PointPlantGradient[BranchPoints[0]];

			for (int32 i = bIsTrunk ? 0 : 1; i < BranchPoints.Num(); ++i)
			{
				const float BranchAlpha = BranchPoints.Num() > 1 
					? i / float(BranchPoints.Num() - 1)
					: 0.f;
				const float PointPlantGradient = FMath::Lerp(BasePointPlantGradient, 0, BranchAlpha);
				InAttributes.PointPlantGradient[BranchPoints[i]] = PointPlantGradient;
			}

			return EForEachResult::Continue;
		}
	);
}

void PV::AttributesHelper::ComputePointHullGradient(const FComputePointHullGradientAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputePointHullGradient);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	for (int32 i = 0; i < InAttributes.PointHullGradient.Num(); ++i)
	{
		InAttributes.PointHullGradient[i] = 1.0f;
	}

	TArray<float> HullDistanceArray;
	HullDistanceArray.SetNumZeroed(InAttributes.PointHullGradient.Num());

	TArray<Chaos::FConvex::FVec3Type> HullPoints;
	for (int32 BranchIndex = 0; BranchIndex < InAttributes.BranchPoints.Num(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = InAttributes.BranchPoints[BranchIndex];
		if (BranchPoints.IsEmpty())
		{
			continue;
		}

		// Tip of each branch is a hull candidate
		const FVector3f& TipPosition = InAttributes.PointPosition[BranchPoints.Last()];
		HullPoints.Add(Chaos::FConvex::FVec3Type(TipPosition.X, TipPosition.Y, TipPosition.Z));

		// Root of the trunk is also a hull candidate
		const bool bIsTrunk = IsTrunk(InAttributes.BranchParentNumber, BranchIndex);
		if (bIsTrunk)
		{
			const FVector3f& BasePosition = InAttributes.PointPosition[BranchPoints[0]];
			HullPoints.Add(Chaos::FConvex::FVec3Type(BasePosition.X, BasePosition.Y, BasePosition.Z));
		}
	}

	const Chaos::FConvex ConvexShape(HullPoints, 0.f);
	if (ConvexShape.NumPlanes() == 0)
	{
		return;
	}

	// For each plant point compute the distance to the hull surface
	for (int32 PointIndex = 0; PointIndex < InAttributes.PointHullGradient.Num(); ++PointIndex)
	{
		const FVector3f& PointPos = InAttributes.PointPosition[PointIndex];
		Chaos::FVec3 Normal;
		const double Dist = FMath::Abs(ConvexShape.PhiWithNormal(Chaos::FVec3(PointPos.X, PointPos.Y, PointPos.Z), Normal));
		HullDistanceArray[PointIndex] = static_cast<float>(Dist);
	}

	// Normalize globally: map [MinDist, MaxDist] -> [1, 0]
	// Points near the hull surface (small distance) get gradient 1; deep interior points get 0.
	const float MinDist = *Algo::MinElement(HullDistanceArray);
	const float MaxDist = *Algo::MaxElement(HullDistanceArray);

	if (FMath::IsNearlyEqual(MinDist, MaxDist))
	{
		return;
	}

	for (int32 i = 0; i < InAttributes.PointHullGradient.Num(); ++i)
	{
		InAttributes.PointHullGradient[i] = FMath::GetMappedRangeValueClamped(
			FVector2f(MinDist, MaxDist),
			FVector2f(1.0f, 0.0f),
			HullDistanceArray[i]
		);
	}
}

void PV::AttributesHelper::ComputePointMainTrunkGradient(const FComputePointMainTrunkGradientAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputePointMainTrunkGradient);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	TArray<float> DistanceArray;
	DistanceArray.SetNumZeroed(InAttributes.PointMainTrunkGradient.Num());

	ForEachPlant(InAttributes.BranchPlantNumber, [&](int32 PlantNumber)
	{
		TArray<int32> TrunkPoints;
		TArray<int32> NonTrunkPoints;

		ForEachPlantPoint(
			InAttributes.BranchPlantNumber, 
			InAttributes.BranchPoints, 
			InAttributes.BranchParentNumber, 
			PlantNumber,
			[&](int32 BranchIndex, int32 PointIndex)
			{
				// Note that we check the Generation instead of using IsTrunk to also include bifurcated branches in the TrunkPoints
				const bool bIsTrunk = InAttributes.BudDevelopment[PointIndex].Generation == 1;
				if (bIsTrunk)
				{
					TrunkPoints.Add(PointIndex);
				}
				else
				{
					NonTrunkPoints.Add(PointIndex);
				}

				return EForEachResult::Continue;
			}
		);

		if (TrunkPoints.IsEmpty() || NonTrunkPoints.IsEmpty())
		{
			return EForEachResult::Continue;
		}

		for (int32 i = 0; i < NonTrunkPoints.Num(); ++i)
		{
			const int32 PointIndex = NonTrunkPoints[i];

			const FVector3f& PointPos = InAttributes.PointPosition[PointIndex];
			float MinDist = TNumericLimits<float>::Max();

			if (TrunkPoints.Num() == 1)
			{
				MinDist = FMath::Min(MinDist, (PointPos - InAttributes.PointPosition[TrunkPoints[0]]).Size());
			}
			else
			{
				for (int32 j = 0; j < TrunkPoints.Num() - 1; ++j)
				{
					const FVector3f& P0 = InAttributes.PointPosition[TrunkPoints[j]];
					const FVector3f& P1 = InAttributes.PointPosition[TrunkPoints[j + 1]];

					// We're currently using the physical distance to the trunk branch. It may suffice to use
					// the difference in LengthFromRoot from the connection point to the trunk to the current
					// point.
					const FVector3f ClosestPoint = FMath::ClosestPointOnSegment(PointPos, P0, P1);
					MinDist = FMath::Min(MinDist, (PointPos - ClosestPoint).Size());
				}
			}

			DistanceArray[PointIndex] = MinDist;
		}

		return EForEachResult::Continue;
	});

	const float MinDist = *Algo::MinElement(DistanceArray);
	const float MaxDist = *Algo::MaxElement(DistanceArray);

	if (FMath::IsNearlyEqual(MinDist, MaxDist))
	{
		for (int32 i = 0; i < InAttributes.PointMainTrunkGradient.Num(); ++i)
		{
			InAttributes.PointMainTrunkGradient[i] = 1.0f;
		}
	}
	else
	{
		for (int32 PointIndex = 0; PointIndex < InAttributes.PointMainTrunkGradient.Num(); ++PointIndex)
		{
			InAttributes.PointMainTrunkGradient[PointIndex] = FMath::GetMappedRangeValueClamped(
				FVector2f(MinDist, MaxDist),
				FVector2f(1.0f, 0.0f),
				DistanceArray[PointIndex]
			);
		}
	}
}

void PV::AttributesHelper::ComputeBudStatus(const FComputeBudStatusAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBudStatus);

	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	const FBranchNumberToIndexTable BranchNumberToIndex(InAttributes.BranchNumber);
	ForEachUniquePointOnBranches(
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		[&](int32 BranchIndex, int32 BranchPointIndex, int32 PointIndex)->EForEachResult
		{
			PV::FBudStatusView BudStatusView = InAttributes.BudStatus[PointIndex];
			BudStatusView.ApicalMeristem = 0;
			BudStatusView.Codominant = 0;
			BudStatusView.Axillary = 0;
			BudStatusView.Seed = 0;
			BudStatusView.Dormant = 0;
			BudStatusView.Triggered = 1;
			BudStatusView.NumTriggered = 1;
			BudStatusView.Inactive = 1;
			BudStatusView.BrokenTip = 0;
			BudStatusView.Broken = 0;

			if (InAttributes.BranchPoints[BranchIndex].Last() == PointIndex) // IsTip
			{
				BudStatusView.ApicalMeristem = 1;
				BudStatusView.Dormant = 1;
				BudStatusView.Triggered = 0;
				BudStatusView.NumTriggered = 0;
				BudStatusView.Inactive = 0;
			}
			else if (InAttributes.BranchPoints[BranchIndex][0] == PointIndex
				&& GetBranchParentIndex(InAttributes.BranchParentNumber, BranchNumberToIndex, BranchIndex) == INDEX_NONE) // Is plant root point
			{
				BudStatusView.Axillary = 1;
				BudStatusView.Seed = 1;
				BudStatusView.Dormant = 1;
				BudStatusView.Triggered = 0;
				BudStatusView.NumTriggered = 0;
			}
			else
			{
				const int32 NumChildren = GetBranchPointChildBranchIndices(
					InAttributes.BranchChildren,
					InAttributes.BranchParentNumber,
					BranchNumberToIndex,
					InAttributes.BranchPoints,
					BranchIndex, 
					BranchPointIndex
				).Num();

				if (NumChildren > 0) 
				{
					BudStatusView.NumTriggered = NumChildren;
					BudStatusView.Axillary = 1;
				}
			}

			return EForEachResult::Continue;
		}
	);
}

void PV::AttributesHelper::ComputeBudLateralMeristem(const FComputeBudLateralMeristemAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBudLateralMeristem);

	using namespace PV::PlantTraversalHelper;

	for (int32 i = 0; i < InAttributes.BudLateralMeristem.Num(); ++i)
	{
		InAttributes.BudLateralMeristem[i].Davinci = 0;
	}

	const float EstimatedLateralElongation = Invoke([&]() -> float
	{
		// The LateralElongation is a user parameter but it can be fairly accuratly estimated by taking the scale of the
		// second to last point on the tunk (multiplied by 0.01).
		// NOTE: This may not be accurate for multiplant growth data as the different plants may have different LateralElongation.
		for (const int32 TrunkIndex : GetTrunkIndices(InAttributes.BranchParentNumber))
		{
			const TArray<int32>& BranchPoints = InAttributes.BranchPoints[TrunkIndex];
			if (BranchPoints.Num() > 1)
			{
				const int32 SecondToLastPointIndex = BranchPoints[BranchPoints.Num() - 2];
				return InAttributes.PointScale[SecondToLastPointIndex] * 0.01f;
			}
		}
		return 0.033f;
	});

	RecursiveWalkBranches_Reversed(
		InAttributes.BranchParentNumber,
		InAttributes.BranchChildren,
		InAttributes.BranchNumber,
		[&](int32 BranchIndex)->EForEachResult
		{
			const TArray<int32>& BranchPointIndices = InAttributes.BranchPoints[BranchIndex];
			if (!ensure(BranchPointIndices.Num() > 1))
			{
				return EForEachResult::Continue;
			}

			const bool bIsTrunk = IsTrunk(InAttributes.BranchParentNumber, BranchIndex);

			float PreviousPointDavinciValue = 0;
			for (int32 i = BranchPointIndices.Num() - 1; i>= 0; --i)
			{
				const int32 PointIndex = BranchPointIndices[i];
				const float ExisitingDavinciValue = InAttributes.BudLateralMeristem[PointIndex].Davinci;
				const float AccumulatedDavinciValue = PreviousPointDavinciValue + ExisitingDavinciValue + EstimatedLateralElongation;
				PreviousPointDavinciValue = AccumulatedDavinciValue;

				InAttributes.BudLateralMeristem[PointIndex].LateralMeristem = InAttributes.PointScale[PointIndex] * 0.01f;
				InAttributes.BudLateralMeristem[PointIndex].Multiplier = 1;
				InAttributes.BudLateralMeristem[PointIndex].Inactive = 0.0;
				InAttributes.BudLateralMeristem[PointIndex].Davinci = AccumulatedDavinciValue;

				if (!bIsTrunk && i == 1)
				{
					InAttributes.BudLateralMeristem[PointIndex].ParentDot = FVector3f::DotProduct(
						InAttributes.BudDirection[BranchPointIndices[0]].Apical, 
						InAttributes.BudDirection[BranchPointIndices[1]].Apical
					);
				}
				else
				{
					InAttributes.BudLateralMeristem[PointIndex].ParentDot = 0;
				}
				
				InAttributes.BudLateralMeristem[PointIndex].RootDistance = InAttributes.PointLengthFromRoot[PointIndex];
				InAttributes.BudLateralMeristem[PointIndex].Degredation = 0.0;
			}

			return EForEachResult::Continue;
		}
	);
}

void PV::AttributesHelper::EstimateBudLightDetected(const FEstimateBudLightDetectedAttributes& InAttributes)
{
	using namespace PV::PlantTraversalHelper;

	if (!InAttributes.IsValid())
	{
		return;
	}

	const int32 NumBranches = InAttributes.BranchPoints.Num();

	for (int32 PointIndex = 0; PointIndex < InAttributes.BudLightDetected.Num(); ++PointIndex)
	{
		const float LightDetectedEstimate = InAttributes.PointHullGradient[PointIndex] * (1.f - InAttributes.PointGroundGradient[PointIndex]);
		InAttributes.BudLightDetected[PointIndex].Availible = LightDetectedEstimate;
		InAttributes.BudLightDetected[PointIndex].Collision = 0; // Assume no collision with external geometry
	}

	TArray<float> BranchAvailibleLight;
	BranchAvailibleLight.SetNumZeroed(NumBranches);

	for (int32 BranchIndex = 0; BranchIndex < InAttributes.BranchPoints.Num(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = InAttributes.BranchPoints[BranchIndex];

		int32 NumUntriggeredPoints = 0;
		for (int32 PointIndex : BranchPoints)
		{
			if (InAttributes.BudStatus[PointIndex].Triggered == 0)
			{
				BranchAvailibleLight[BranchIndex] += InAttributes.BudLightDetected[PointIndex].Availible;
				NumUntriggeredPoints += 1;
			}
		}

		BranchAvailibleLight[BranchIndex] = NumUntriggeredPoints > 0
			? BranchAvailibleLight[BranchIndex] / NumUntriggeredPoints
			: 0.f;
	}

	const FBranchNumberToIndexTable BranchNumberToIndex(InAttributes.BranchNumber);
	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		float BranchLight = BranchAvailibleLight[BranchIndex];

		const TArray<int32>& BranchChildren = InAttributes.BranchChildren[BranchIndex];
		if (BranchChildren.Num() > 0)
		{
			float MinBranchLight = BranchLight;

			for (int32 ChildBranchNumber : BranchChildren)
			{
				const int32 ChildBranchIndex = BranchNumberToIndex.Find(ChildBranchNumber);
				if (ensure(ChildBranchIndex != INDEX_NONE))
				{
					BranchLight += BranchAvailibleLight[ChildBranchIndex];
					MinBranchLight = FMath::Min(MinBranchLight, BranchAvailibleLight[ChildBranchIndex]);
				}
			}

			BranchLight -= MinBranchLight;
			BranchLight = BranchLight / BranchChildren.Num();
		}

		const TArray<int32>& BranchPoints = InAttributes.BranchPoints[BranchIndex];
		const bool bIsTrunk = IsTrunk(InAttributes.BranchParentNumber, BranchIndex);
		const int32 StartIndex = bIsTrunk ? 0 : 1;
		for (int32 i = StartIndex; i < BranchPoints.Num(); ++i)
		{
			const int32 PointIndex = BranchPoints[i];
			InAttributes.BudLightDetected[PointIndex].Branch = BranchLight;
		}
	}
}

void PV::AttributesHelper::ComputeBranchHierarchyNumbers(const FComputeBranchHierarchyNumberAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeBranchHierarchyNumbers);

	using namespace PV::PlantTraversalHelper;

	RecursiveWalkBranches<int32>(
		InAttributes.BranchParentNumber,
		InAttributes.BranchChildren,
		InAttributes.BranchNumber,
		1,
		[&](int32 BranchIndex, int32& BranchHierarchyNumber)->EForEachResult
		{
			InAttributes.BranchHierarchyNumber[BranchIndex] = BranchHierarchyNumber;
			BranchHierarchyNumber += 1;
			return EForEachResult::Continue;
		}
	);
}

bool PV::AttributesHelper::RecomputeAllGrowthDataAttributes(const FRecomputeGrowthDataAttributes& InAttributes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RecomputeAllGrowthDataAttributes);

	if (!InAttributes.IsValid())
	{
		return false;
	}

	ComputeBudNumbers({
		InAttributes.PointBudNumber,
		InAttributes.BranchSourceBudNumber,
		InAttributes.BranchPoints
	});

	ComputeLengthFromRoot({
		InAttributes.PointLengthFromRoot,
		InAttributes.PointLengthFromSeed,
		InAttributes.PointPosition,
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		InAttributes.BranchNumber,
		InAttributes.BranchChildren
	});

	ComputeBudDirections({
		InAttributes.BudDirection,
		InAttributes.PointPosition,
		InAttributes.BranchChildren,
		InAttributes.BranchParentNumber,
		InAttributes.BranchNumber,
		InAttributes.BranchPoints
	});

	ComputePointScales_SmoothTaper({
		InAttributes.PointScale,
		InAttributes.PointLengthFromRoot,
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		InAttributes.BranchChildren,
		InAttributes.BranchNumber
	});

	ComputeBudLateralMeristem({
		InAttributes.BudLateralMeristem,
		InAttributes.PointPosition,
		InAttributes.PointScale,
		InAttributes.PointLengthFromRoot,
		InAttributes.BudDirection,
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		InAttributes.BranchChildren,
		InAttributes.BranchNumber
	});

	ComputePointPlantGradient({
		InAttributes.PointPlantGradient,
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		InAttributes.BranchNumber,
		InAttributes.BranchChildren
	});

	if (InAttributes.PointGroundGradient.IsValid() 
		&& InAttributes.PointGroundGradient.Num() == InAttributes.PointPosition.Num())
	{
		ComputePointGroundGradient({ 
			InAttributes.PointGroundGradient,
			InAttributes.PointPosition
		});
	}

	if (InAttributes.PointScaleGradient.IsValid()
		&& InAttributes.PointScaleGradient.Num() == InAttributes.PointScale.Num())
	{
		ComputePointScaleGradient({ 
			InAttributes.PointScaleGradient,
			InAttributes.PointScale
		});
	}
		
	if (InAttributes.PointHullGradient.IsValid()
		&& InAttributes.PointHullGradient.Num() == InAttributes.PointPosition.Num())
	{
		ComputePointHullGradient({
			InAttributes.PointHullGradient,
			InAttributes.PointPosition,
			InAttributes.BranchPoints,
			InAttributes.BranchParentNumber
		});
	}

	if (ValidateAttributeCollection(
		InAttributes.PointHullGradient,
		InAttributes.PointGroundGradient,
		InAttributes.BudStatus,
		InAttributes.BudLightDetected,
		InAttributes.BranchPoints,
		InAttributes.BranchChildren,
		InAttributes.BranchParentNumber
	))
	{
		EstimateBudLightDetected({
			InAttributes.PointHullGradient,
			InAttributes.PointGroundGradient,
			InAttributes.BudStatus,
			InAttributes.BudLightDetected,
			InAttributes.BranchPoints,
			InAttributes.BranchChildren,
			InAttributes.BranchParentNumber,
			InAttributes.BranchNumber
		});
	}

	if (InAttributes.PointMainTrunkGradient.IsValid()
		&& InAttributes.PointMainTrunkGradient.Num() == InAttributes.PointPosition.Num())
	{
		ComputePointMainTrunkGradient({
			InAttributes.PointMainTrunkGradient,
			InAttributes.PointPosition,
			InAttributes.BranchPoints,
			InAttributes.BranchParentNumber,
			InAttributes.BranchPlantNumber,
			InAttributes.BudDevelopment
		});
	}

	ComputeBudDevelopment({
		InAttributes.BudDevelopment,
		InAttributes.PointPlantGradient,
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		InAttributes.BranchParents,
		InAttributes.BranchNumber
	});

	ComputeBudHormoneLevels({
		InAttributes.BudHormoneLevels,
		InAttributes.PointPlantGradient
	});

	ComputeBudStatus({
		InAttributes.BudStatus,
		InAttributes.BranchPoints,
		InAttributes.BranchParentNumber,
		InAttributes.BranchNumber,
		InAttributes.BranchChildren
	});

	return true;
}

int32 PV::AttributesHelper::GetMaxBudAge(PV::FBudDevelopmentAttributeConstView BudDevelopment)
{
	int32 MaxBudAge = -1;
	if (BudDevelopment.IsValid())
	{
		for (int32 i = 0; i < BudDevelopment.Num(); ++i)
		{
			MaxBudAge = FMath::Max(MaxBudAge, BudDevelopment[i].BudAge);
		}
	}
	return MaxBudAge;
}
