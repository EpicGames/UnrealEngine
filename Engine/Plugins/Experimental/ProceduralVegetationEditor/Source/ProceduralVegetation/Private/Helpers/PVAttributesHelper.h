// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Utils/PVAttributes.h"

namespace PV::AttributesHelper
{
	float GetBranchPointBranchGradient(
		PV::FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		int32 BranchIndex,
		int32 BranchPointIndex
	);

	float GetBranchPointScale(
		PV::FPointScaleAttributeConstView PointScaleAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		int32 BranchIndex,
		int32 BranchPointIndex
	);

	// Returns the bud directions for the given branch point. For non-trunk branches, the 0th
	// branch ring is the fused point shared with the parent; the frame is recomputed by rotating
	// the first child point's axillary and up vectors through the dihedral from apical_next to dir,
	// so callers always receive the correct child-branch frame without extra fused-point handling.
	TArray<FVector3f> GetBudDirection(
		PV::FBudDirectionAttributeConstView BudDirectionAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		PV::FPointPositionAttributeConstView PointPositionAttribute,
		const int32 BranchIndex,
		const int32 BranchPointIndex
	);

	// Returns the BudDevelopment data for a branch point, correctly handling the fused root point.
	// For non-trunk branches, BranchPointIndex 0 is shared with the parent and retains the parent's
	// BudDevelopment. This helper redirects to BranchPoints[1] (the first owned child point) instead.
	TArray<int32> GetBudDevelopment(
		PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		int32 BranchIndex,
		int32 BranchPointIndex
	);

	// Returns the Generation of the given branch. Returns INDEX_NONE if the inputs 
	// are invalid or the branch has no points.
	int32 GetBranchGeneration(
		PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		int32 BranchIndex
	);

	struct FComputeBudNumbersAttributes
	{
		FPointBudNumberAttributeView PointBudNumber;
		FBranchSourceBudNumberAttributeView BranchSourceBudNumber;
		FBranchPointsAttributeConstView BranchPoints;

		FComputeBudNumbersAttributes(
			FPointBudNumberAttributeView InPointBudNumber,
			FBranchSourceBudNumberAttributeView InBranchSourceBudNumber,
			FBranchPointsAttributeConstView InBranchPoints)
			: PointBudNumber(InPointBudNumber)
			, BranchSourceBudNumber(InBranchSourceBudNumber)
			, BranchPoints(InBranchPoints)
		{}

		FComputeBudNumbersAttributes(FManagedArrayCollection& InCollection)
			: PointBudNumber(FPointBudNumberAttribute::FindAttribute(InCollection))
			, BranchSourceBudNumber(FBranchSourceBudNumberAttribute::FindAttribute(InCollection))
			, BranchPoints(FBranchPointsAttribute::FindAttribute(InCollection))
		{}

		bool IsValid() const
		{
			return ValidateAttributeCollection(PointBudNumber, BranchSourceBudNumber, BranchPoints);
		}
	};

	// Re-computes the point bud numbers and writes them to the BudNumber and BranchSourceBudNumber attributes.
	void ComputeBudNumbers(const FComputeBudNumbersAttributes& InAttributes);

	struct FComputeNjordPixelIndexAttributes
	{
		PV::FPointNjordPixelIndexAttributeView PointNjordPixelIndex;
		FPointBudNumberAttributeView PointBudNumber;

		FComputeNjordPixelIndexAttributes(
			FPointBudNumberAttributeView InPointBudNumber,
			PV::FPointNjordPixelIndexAttributeView InPointNjordPixelIndex)
			: PointNjordPixelIndex(InPointNjordPixelIndex)
			, PointBudNumber(InPointBudNumber)
		{
		}

		FComputeNjordPixelIndexAttributes(FManagedArrayCollection& InCollection)
			: PointNjordPixelIndex(PV::FPointNjordPixelIndexAttribute::FindAttribute(InCollection))
			, PointBudNumber(FPointBudNumberAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(PointNjordPixelIndex, PointBudNumber);
		}
	};

	// Re-computes all njord pixel indices and writes them to the NjordPixelIndexAttribute.
	void ComputeNjordPixelIndex(const FComputeNjordPixelIndexAttributes& InAttributes);

	struct FComputeLengthFromRootAttributes
	{
		PV::FPointLengthFromRootAttributeView PointLengthFromRoot;
		PV::FPointLengthFromSeedAttributeView PointLengthFromSeed;
		PV::FPointPositionAttributeConstView PointPosition;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;

		FComputeLengthFromRootAttributes(
			PV::FPointLengthFromRootAttributeView InPointLengthFromRoot,
			PV::FPointLengthFromSeedAttributeView InPointLengthFromSeed,
			PV::FPointPositionAttributeConstView InPointPosition,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren)
			: PointLengthFromRoot(InPointLengthFromRoot)
			, PointLengthFromSeed(InPointLengthFromSeed)
			, PointPosition(InPointPosition)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
			, BranchChildren(InBranchChildren)
		{
		}

		FComputeLengthFromRootAttributes(FManagedArrayCollection& Collection)
			: PointLengthFromRoot(PV::FPointLengthFromRootAttribute::FindAttribute(Collection))
			, PointLengthFromSeed(PV::FPointLengthFromSeedAttribute::FindAttribute(Collection))
			, PointPosition(PV::FPointPositionAttribute::FindAttribute(Collection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(Collection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(Collection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(Collection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(Collection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(PointLengthFromRoot, PointLengthFromSeed, PointPosition, BranchPoints, BranchParentNumber, BranchNumber, BranchChildren);
		}
	};

	// Re-computes the length from root and length from seed for all points.
	void ComputeLengthFromRoot(const FComputeLengthFromRootAttributes& InAttributes);

	struct FComputeLengthFromTrunkAttributes
	{
		PV::FPointLengthFromRootAttributeConstView PointLengthFromRoot;
		PV::FBudDevelopmentAttributeConstView BudDevelopment;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;

		FComputeLengthFromTrunkAttributes(
			PV::FPointLengthFromRootAttributeConstView InPointLengthFromRoot,
			PV::FBudDevelopmentAttributeConstView InBudDevelopment,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren)
			: PointLengthFromRoot(InPointLengthFromRoot)
			, BudDevelopment(InBudDevelopment)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
			, BranchChildren(InBranchChildren)
		{}

		FComputeLengthFromTrunkAttributes(const FManagedArrayCollection& Collection)
			: PointLengthFromRoot(PV::FPointLengthFromRootAttribute::FindAttribute(Collection))
			, BudDevelopment(PV::FBudDevelopmentAttribute::FindAttribute(Collection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(Collection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(Collection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(Collection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(Collection))
		{}

		bool IsValid() const
		{
			return ValidateAttributeCollection(
				PointLengthFromRoot, 
				BudDevelopment,
				BranchPoints, 
				BranchParentNumber, 
				BranchNumber, 
				BranchChildren
			);
		}
	};
	void ComputeLengthFromTrunk(const FComputeLengthFromTrunkAttributes& InAttributes, TArrayView<float> OutLengthFromTunk);

	struct FComputeBudDevelopmentAttributes
	{
		PV::FBudDevelopmentAttributeView BudDevelopment;
		PV::FPointPlantGradientAttributeConstView PointPlantGradient;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchParentsAttributeConstView BranchParents;
		PV::FBranchNumberAttributeConstView BranchNumber;

		FComputeBudDevelopmentAttributes(
			PV::FBudDevelopmentAttributeView InBudDevelopment,
			PV::FPointPlantGradientAttributeConstView InPointPlantGradient,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchParentsAttributeConstView InBranchParents,
			PV::FBranchNumberAttributeConstView InBranchNumber)
			: BudDevelopment(InBudDevelopment)
			, PointPlantGradient(InPointPlantGradient)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchParents(InBranchParents)
			, BranchNumber(InBranchNumber)
		{}

		FComputeBudDevelopmentAttributes(FManagedArrayCollection& InCollection)
			: BudDevelopment(PV::FBudDevelopmentAttribute::FindAttribute(InCollection))
			, PointPlantGradient(PV::FPointPlantGradientAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchParents(PV::FBranchParentsAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
		{}

		bool IsValid() const
		{
			return ValidateAttributeCollection(BudDevelopment, PointPlantGradient, BranchPoints, BranchParentNumber, BranchParents, BranchNumber);
		}
	};

	// Re-computes bud development for all points using the existing branch hierarchy and plant gradient. 
	void ComputeBudDevelopment(const FComputeBudDevelopmentAttributes& InAttributes);

	struct FComputeBudDirectionsAttributes
	{
		PV::FBudDirectionAttributeView BudDirection;
		PV::FPointPositionAttributeConstView PointPosition;
		PV::FBranchChildrenAttributeConstView BranchChildren;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;
		PV::FBranchPointsAttributeConstView BranchPoints;

		FComputeBudDirectionsAttributes(
			PV::FBudDirectionAttributeView InBudDirection,
			PV::FPointPositionAttributeConstView InPointPosition,
			PV::FBranchChildrenAttributeConstView InBranchChildren,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber,
			PV::FBranchPointsAttributeConstView InBranchPoints)
			: BudDirection(InBudDirection)
			, PointPosition(InPointPosition)
			, BranchChildren(InBranchChildren)
			, BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
			, BranchPoints(InBranchPoints)
		{}

		FComputeBudDirectionsAttributes(FManagedArrayCollection& InCollection)
			: BudDirection(PV::FBudDirectionAttribute::FindAttribute(InCollection))
			, PointPosition(PV::FPointPositionAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
		{}

		bool IsValid() const
		{
			return ValidateAttributeCollection(BudDirection, PointPosition, BranchChildren, BranchParentNumber, BranchNumber, BranchPoints);
		}
	};

	// Re-computes the bud directions for all points.
	void ComputeBudDirections(const FComputeBudDirectionsAttributes& InAttributes);

	struct FComputeBudHormoneLevelsAttributes
	{
		PV::FBudHormoneLevelsAttributeView BudHormoneLevels;
		PV::FPointPlantGradientAttributeConstView PointPlantGradient;

		bool IsValid() const
		{
			return ValidateAttributeCollection(BudHormoneLevels, PointPlantGradient);
		}
	};

	// Re-computes the bud hormone levels using the plant gradient for all points.
	void ComputeBudHormoneLevels(const FComputeBudHormoneLevelsAttributes& InAttributes);

	struct FComputePointGroundGradientAttributes
	{
		PV::FPointGroundGradientAttributeView PointGroundGradient;
		PV::FPointPositionAttributeConstView PointPosition;

		FComputePointGroundGradientAttributes(
			PV::FPointGroundGradientAttributeView InPointGroundGradient,
			PV::FPointPositionAttributeConstView InPointLengthFromRoot)
			: PointGroundGradient(InPointGroundGradient)
			, PointPosition(InPointLengthFromRoot)
		{
		}

		FComputePointGroundGradientAttributes(FManagedArrayCollection& InCollection)
			: PointGroundGradient(PV::FPointGroundGradientAttribute::FindAttribute(InCollection))
			, PointPosition(PV::FPointPositionAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const { return ValidateAttributeCollection(PointGroundGradient, PointPosition); }
	};

	void ComputePointGroundGradient(const FComputePointGroundGradientAttributes& InAttributes);

	struct FComputePointScaleGradientAttributes
	{
		PV::FPointScaleGradientAttributeView PointScaleGradient;
		PV::FPointScaleAttributeConstView PointScale;

		FComputePointScaleGradientAttributes(
			PV::FPointScaleGradientAttributeView InPointScaleGradient,
			PV::FPointScaleAttributeConstView InPointScale)
			: PointScaleGradient(InPointScaleGradient)
			, PointScale(InPointScale)
		{
		}

		FComputePointScaleGradientAttributes(FManagedArrayCollection& InCollection)
			: PointScaleGradient(PV::FPointScaleGradientAttribute::FindAttribute(InCollection))
			, PointScale(PV::FPointScaleAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const { return ValidateAttributeCollection(PointScaleGradient, PointScale); }
	};

	void ComputePointScaleGradient(const FComputePointScaleGradientAttributes& InAttributes);

	struct FComputePointPlantGradientAttributes
	{
		PV::FPointPlantGradientAttributeView PointPlantGradient;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;

		FComputePointPlantGradientAttributes(
			PV::FPointPlantGradientAttributeView InPointPlantGradient,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren)
			: PointPlantGradient(InPointPlantGradient)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
			, BranchChildren(InBranchChildren)
		{
		}

		FComputePointPlantGradientAttributes(FManagedArrayCollection& InCollection)
			: PointPlantGradient(PV::FPointPlantGradientAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(PointPlantGradient, BranchPoints, BranchParentNumber, BranchNumber, BranchChildren);
		}
	};

	void ComputePointPlantGradient(const FComputePointPlantGradientAttributes& InAttributes);

	struct FComputePointHullGradientAttributes
	{
		PV::FPointHullGradientAttributeView PointHullGradient;
		PV::FPointPositionAttributeConstView PointPosition;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;

		FComputePointHullGradientAttributes(
			PV::FPointHullGradientAttributeView InPointHullGradient,
			PV::FPointPositionAttributeConstView InPointPosition,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber)
			: PointHullGradient(InPointHullGradient)
			, PointPosition(InPointPosition)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
		{
		}

		FComputePointHullGradientAttributes(FManagedArrayCollection& InCollection)
			: PointHullGradient(PV::FPointHullGradientAttribute::FindAttribute(InCollection))
			, PointPosition(PV::FPointPositionAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(PointHullGradient, PointPosition, BranchPoints, BranchParentNumber);
		}
	};

	void ComputePointHullGradient(const FComputePointHullGradientAttributes& InAttributes);

	struct FComputePointMainTrunkGradientAttributes
	{
		PV::FPointMainTrunkGradientAttributeView PointMainTrunkGradient;
		PV::FPointPositionAttributeConstView PointPosition;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchPlantNumberAttributeConstView BranchPlantNumber;
		PV::FBudDevelopmentAttributeConstView BudDevelopment;

		FComputePointMainTrunkGradientAttributes(
			PV::FPointMainTrunkGradientAttributeView InPointMainTrunkGradient,
			PV::FPointPositionAttributeConstView InPointPosition,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchPlantNumberAttributeConstView InBranchPlantNumber,
			PV::FBudDevelopmentAttributeConstView InBudDevelopment)
			: PointMainTrunkGradient(InPointMainTrunkGradient)
			, PointPosition(InPointPosition)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchPlantNumber(InBranchPlantNumber)
			, BudDevelopment(InBudDevelopment)
		{
		}

		FComputePointMainTrunkGradientAttributes(FManagedArrayCollection& InCollection)
			: PointMainTrunkGradient(PV::FPointMainTrunkGradientAttribute::FindAttribute(InCollection))
			, PointPosition(PV::FPointPositionAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchPlantNumber(PV::FBranchPlantNumberAttribute::FindAttribute(InCollection))
			, BudDevelopment(PV::FBudDevelopmentAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(PointMainTrunkGradient, PointPosition, BranchPoints, BranchParentNumber, BranchPlantNumber, BudDevelopment);
		}
	};

	void ComputePointMainTrunkGradient(const FComputePointMainTrunkGradientAttributes& InAttributes);

	struct FComputeBudStatusAttributes
	{ 
		PV::FBudStatusAttributeView BudStatus;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;

		FComputeBudStatusAttributes(PV::FBudStatusAttributeView InBudStatus,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren)
			: BudStatus(InBudStatus)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
			, BranchChildren(InBranchChildren)
		{
		}

		FComputeBudStatusAttributes(FManagedArrayCollection& InCollection)
			: BudStatus(PV::FBudStatusAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))		
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(BudStatus, BranchPoints, BranchParentNumber, BranchNumber, BranchChildren);
		}
	};

	// Re-computes the bud status for all points.
	void ComputeBudStatus(const FComputeBudStatusAttributes& InAttributes);

	struct FComputeBudLateralMeristemAttributes
	{
		PV::FBudLateralMeristemAttributeView BudLateralMeristem;
		PV::FPointPositionAttributeConstView PointPosition;
		PV::FPointScaleAttributeConstView PointScale;
		PV::FPointLengthFromRootAttributeConstView PointLengthFromRoot;
		PV::FBudDirectionAttributeConstView BudDirection;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;
		PV::FBranchNumberAttributeConstView BranchNumber;

		FComputeBudLateralMeristemAttributes(FManagedArrayCollection& InCollection)
			: BudLateralMeristem(PV::FBudLateralMeristemAttribute::FindAttribute(InCollection))
			, PointPosition(PV::FPointPositionAttribute::FindAttribute(InCollection))
			, PointScale(PV::FPointScaleAttribute::FindAttribute(InCollection))
			, PointLengthFromRoot(PV::FPointLengthFromRootAttribute::FindAttribute(InCollection))
			, BudDirection(PV::FBudDirectionAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
		{
		}

		FComputeBudLateralMeristemAttributes(
			PV::FBudLateralMeristemAttributeView InBudLateralMeristem,
			PV::FPointPositionAttributeConstView InPointPosition,
			PV::FPointScaleAttributeConstView InPointScale,
			PV::FPointLengthFromRootAttributeConstView InPointLengthFromRoot,
			PV::FBudDirectionAttributeConstView InBudDirection,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren,
			PV::FBranchNumberAttributeConstView InBranchNumber)
			: BudLateralMeristem(InBudLateralMeristem)
			, PointPosition(InPointPosition)
			, PointScale(InPointScale)
			, PointLengthFromRoot(InPointLengthFromRoot)
			, BudDirection(InBudDirection)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchChildren(InBranchChildren)
			, BranchNumber(InBranchNumber)
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(BudLateralMeristem, PointPosition, PointScale, PointLengthFromRoot, BudDirection, BranchPoints, BranchParentNumber, BranchChildren, BranchNumber);
		}
	};

	// Re-computes the bud lateral meristem for all points.
	void ComputeBudLateralMeristem(const FComputeBudLateralMeristemAttributes& InAttributes);

	struct FEstimateBudLightDetectedAttributes
	{
		PV::FPointHullGradientAttributeConstView PointHullGradient;
		PV::FPointGroundGradientAttributeConstView PointGroundGradient;
		PV::FBudStatusAttributeConstView BudStatus;
		PV::FBudLightDetectedAttributeView BudLightDetected;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchChildrenAttributeConstView BranchChildren;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;

		FEstimateBudLightDetectedAttributes(
			PV::FPointHullGradientAttributeConstView InPointHullGradient,
			PV::FPointGroundGradientAttributeConstView InPointGroundGradient,
			PV::FBudStatusAttributeConstView InBudStatus,
			PV::FBudLightDetectedAttributeView InBudLightDetected,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchChildrenAttributeConstView InBranchChildren,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber)
			: PointHullGradient(InPointHullGradient)
			, PointGroundGradient(InPointGroundGradient)
			, BudStatus(InBudStatus)
			, BudLightDetected(InBudLightDetected)
			, BranchPoints(InBranchPoints)
			, BranchChildren(InBranchChildren)
			, BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
		{
		}

		FEstimateBudLightDetectedAttributes(FManagedArrayCollection& InCollection)
			: PointHullGradient(PV::FPointHullGradientAttribute::FindAttribute(InCollection))
			, PointGroundGradient(PV::FPointGroundGradientAttribute::FindAttribute(InCollection))
			, BudStatus(PV::FBudStatusAttribute::FindAttribute(InCollection))
			, BudLightDetected(PV::FBudLightDetectedAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(
				PointHullGradient,
				PointGroundGradient,
				BudStatus,
				BudLightDetected,
				BranchPoints,
				BranchChildren,
				BranchParentNumber,
				BranchNumber
			);
		}
	};

	// Estimate the bud light detected for all points.
	void EstimateBudLightDetected(const FEstimateBudLightDetectedAttributes& InAttributes);

	struct FComputeBranchHierarchyNumberAttributes
	{
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchNumberAttributeConstView BranchNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;
		PV::FBranchHierarchyNumberAttributeView BranchHierarchyNumber;

		FComputeBranchHierarchyNumberAttributes(
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchNumberAttributeConstView InBranchNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren,
			PV::FBranchHierarchyNumberAttributeView InBranchHierarchyNumber)
			: BranchParentNumber(InBranchParentNumber)
			, BranchNumber(InBranchNumber)
			, BranchChildren(InBranchChildren)
			, BranchHierarchyNumber(InBranchHierarchyNumber)
		{
		}

		FComputeBranchHierarchyNumberAttributes(FManagedArrayCollection& InCollection)
			: BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchHierarchyNumber(PV::FBranchHierarchyNumberAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(BranchParentNumber, BranchNumber, BranchChildren, BranchHierarchyNumber);
		}
	};

	// Re-computes the BranchHierarchyNumber attribute given an already existing branch hierarchy.
	void ComputeBranchHierarchyNumbers(const FComputeBranchHierarchyNumberAttributes& InAttributes);

	struct FRecomputeGrowthDataAttributes
	{
		FPointPositionAttributeConstView PointPosition;
		FBranchPointsAttributeConstView BranchPoints;
		FBranchParentNumberAttributeConstView BranchParentNumber;
		FBranchParentsAttributeConstView BranchParents;
		FBranchNumberAttributeConstView BranchNumber;
		FBranchChildrenAttributeConstView BranchChildren;
		FBranchHierarchyNumberAttributeConstView BranchHierarchyNumber;
		FBranchPlantNumberAttributeConstView BranchPlantNumber;
		FPointScaleAttributeView PointScale;
		FPointBudNumberAttributeView PointBudNumber;
		FBranchSourceBudNumberAttributeView BranchSourceBudNumber;
		FPointLengthFromRootAttributeView PointLengthFromRoot;
		FPointLengthFromSeedAttributeView PointLengthFromSeed;
		FBudDirectionAttributeView BudDirection;
		FBudLightDetectedAttributeView BudLightDetected;
		FBudLateralMeristemAttributeView BudLateralMeristem;
		FPointGroundGradientAttributeView PointGroundGradient;
		FPointScaleGradientAttributeView PointScaleGradient;
		FPointPlantGradientAttributeView PointPlantGradient;
		FPointHullGradientAttributeView PointHullGradient;
		FPointMainTrunkGradientAttributeView PointMainTrunkGradient;
		FBudDevelopmentAttributeView BudDevelopment;
		FBudHormoneLevelsAttributeView BudHormoneLevels;
		FBudStatusAttributeView BudStatus;

		FRecomputeGrowthDataAttributes() = default;
		FRecomputeGrowthDataAttributes(FManagedArrayCollection& InCollection)
			: PointPosition(FPointPositionAttribute::FindAttribute(InCollection))
			, BranchPoints(FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchParents(FBranchParentsAttribute::FindAttribute(InCollection))
			, BranchNumber(FBranchNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchHierarchyNumber(FBranchHierarchyNumberAttribute::FindAttribute(InCollection))
			, BranchPlantNumber(FBranchPlantNumberAttribute::FindAttribute(InCollection))
			, PointScale(FPointScaleAttribute::FindAttribute(InCollection))
			, PointBudNumber(FPointBudNumberAttribute::FindAttribute(InCollection))
			, BranchSourceBudNumber(FBranchSourceBudNumberAttribute::FindAttribute(InCollection))
			, PointLengthFromRoot(FPointLengthFromRootAttribute::FindAttribute(InCollection))
			, PointLengthFromSeed(FPointLengthFromSeedAttribute::FindAttribute(InCollection))
			, BudDirection(FBudDirectionAttribute::FindAttribute(InCollection))
			, BudLightDetected(FBudLightDetectedAttribute::FindAttribute(InCollection))
			, BudLateralMeristem(FBudLateralMeristemAttribute::FindAttribute(InCollection))
			, PointGroundGradient(FPointGroundGradientAttribute::FindAttribute(InCollection))
			, PointScaleGradient(FPointScaleGradientAttribute::FindAttribute(InCollection))
			, PointPlantGradient(FPointPlantGradientAttribute::FindAttribute(InCollection))
			, PointHullGradient(FPointHullGradientAttribute::FindAttribute(InCollection))
			, PointMainTrunkGradient(FPointMainTrunkGradientAttribute::FindAttribute(InCollection))
			, BudDevelopment(FBudDevelopmentAttribute::FindAttribute(InCollection))
			, BudHormoneLevels(FBudHormoneLevelsAttribute::FindAttribute(InCollection))
			, BudStatus(FBudStatusAttribute::FindAttribute(InCollection))
		{}

		bool IsValid() const
		{
			return ValidateAttributeCollection(
				PointPosition,
				BranchPoints,
				BranchParentNumber,
				BranchParents,
				BranchNumber,
				BranchChildren,
				BranchHierarchyNumber,
				BranchPlantNumber,
				PointScale,
				PointBudNumber,
				BranchSourceBudNumber,
				PointLengthFromRoot,
				PointLengthFromSeed,
				BudDirection,
				BudLightDetected,
				BudLateralMeristem,
				PointPlantGradient,
				BudDevelopment,
				BudHormoneLevels,
				BudStatus
			);
		}
	};

	bool RecomputeAllGrowthDataAttributes(const FRecomputeGrowthDataAttributes& InAttributes);

	/** Returns the highest BudAge value across all points, or -1 if the view is invalid or empty. */
	int32 GetMaxBudAge(PV::FBudDevelopmentAttributeConstView BudDevelopment);
}
