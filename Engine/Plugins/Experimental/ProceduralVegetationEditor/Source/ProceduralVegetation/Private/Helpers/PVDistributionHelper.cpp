// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PVDistributionHelper.h"
#include "Utils/PVAttributes.h"

namespace PV::DistributionConditionUtils
{
	float FConditionAttributes::GetConditionValue(EPVDistributionCondition InCondition, const int Index) const
	{
		const TArray<float>* Result = nullptr;
		switch (InCondition)
		{
		case EPVDistributionCondition::Light:
			{
				Result = &Light;
				break;
			}
		case EPVDistributionCondition::Scale:
			{
				Result = &Scale;
				break;;
			}
		case EPVDistributionCondition::Tip:
			{
				Result = &Tip;
				break;
			}
		case EPVDistributionCondition::UpAlignment:
			{
				Result = &UpAlignment;
				break;
			}
		case EPVDistributionCondition::Health:
			{
				Result = &Health;
				break;
			}
		case EPVDistributionCondition::Height:
			{
				Result = &Height;
				break;
			}
		case EPVDistributionCondition::Generation:
			{
				Result = &Generation;
				break;
			}
		default:
			break;
		}
 
		if (Result && Result->IsValidIndex(Index))
			[[likely]]
			{
				return (*Result)[Index];
			}
 
		return 0;
	}
	
	void FNormalizedPointCaches::Reset(const int32 NumPoints)
	{
		PScales.Reset();
		Light.Reset();
		Health.Reset();
		Height.Reset();
		Generation.Reset();

		PScales.SetNumZeroed(NumPoints);
		Light.SetNumZeroed(NumPoints);
		Health.SetNumZeroed(NumPoints);
		Height.SetNumZeroed(NumPoints);
		Generation.SetNumZeroed(NumPoints);

		bHasPScales = false;
		bHasLight = false;
		bHasHealth = false;
		bHasHeight = false;
		bHasGeneration = false;
	}

	void BuildNormalizedPointCaches(
		const FManagedArrayCollection& Collection,
		const FNormalizationInputs& Options,
		FNormalizedPointCaches& OutCaches)
	{
		auto PointBudLightDetectedAttribute = PV::FBudLightDetectedAttribute::GetAttribute(Collection);
		auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(Collection);
		auto PointBudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::GetAttribute(Collection);
		auto PointBudDevelopmentAttribute = PV::FBudDevelopmentAttribute::GetAttribute(Collection);
		auto PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(Collection);

		const int32 NumOfPoints = PointScaleAttribute.Num();
		OutCaches.Reset(NumOfPoints);
		if (NumOfPoints == 0)
		{
			return;
		}

		float MinLight = TNumericLimits<float>::Max();
		float MaxLight = TNumericLimits<float>::Lowest();

		float MinHealth = TNumericLimits<float>::Max();
		float MaxHealth = TNumericLimits<float>::Lowest();

		float MinGeneration = TNumericLimits<float>::Max();
		float MaxGeneration = TNumericLimits<float>::Lowest();

		float MinHeight = TNumericLimits<float>::Max();
		float MaxHeight = TNumericLimits<float>::Lowest();

		for (int32 PointIndex = 0; PointIndex < NumOfPoints; ++PointIndex)
		{
			if (Options.bComputeLight)
			{
				const float Available = PointBudLightDetectedAttribute[PointIndex].Availible;
				MinLight = FMath::Min(MinLight, Available);
				MaxLight = FMath::Max(MaxLight, Available);
			}

			if (Options.bComputeHealth)
			{
				const float Ethylene = PointBudHormoneLevelsAttribute[PointIndex].Ethylene;
				const int32 AgeSen = PointBudDevelopmentAttribute[PointIndex].AgeSenescense;
				const int32 LightSen = PointBudDevelopmentAttribute[PointIndex].LightSenescense;
				const int32 Senescence = FMath::Max(AgeSen, LightSen) + 1;

				const float Health = 1.0f - (Ethylene * (1.0f / FMath::Max(1, Senescence)));
				OutCaches.Health[PointIndex] = Health;

				MinHealth = FMath::Min(MinHealth, Health);
				MaxHealth = FMath::Max(MaxHealth, Health);
			}

			if (Options.bComputeHeight && PointPositionAttribute.IsValid())
			{
				const float Z = PointPositionAttribute[PointIndex].Z;
				OutCaches.Height[PointIndex] = Z;
				MinHeight = FMath::Min(MinHeight, Z);
				MaxHeight = FMath::Max(MaxHeight, Z);
			}

			if (Options.bComputeGeneration)
			{
				const float Gen = static_cast<float>(PointBudDevelopmentAttribute[PointIndex].Generation);
				OutCaches.Generation[PointIndex] = Gen;
				MinGeneration = FMath::Min(MinGeneration, Gen);
				MaxGeneration = FMath::Max(MaxGeneration, Gen);
			}
		}

		if (Options.bComputePScales)
		{
			OutCaches.PScales     = Options.PScalesOverride;
			OutCaches.bHasPScales = true;
		}

		if (Options.bComputeLight)
		{
			OutCaches.bHasLight = (MaxLight > MinLight + KINDA_SMALL_NUMBER);
			for (int32 i = 0; i < NumOfPoints; ++i)
			{
				const float Available = PointBudLightDetectedAttribute[i].Availible;
				OutCaches.Light[i] = OutCaches.bHasLight
					? FMath::GetMappedRangeValueClamped(
						FVector2f(MinLight, MaxLight),
						FVector2f(0.0f, 1.0f),
						Available)
					: 0.0f;
			}
		}

		if (Options.bComputeHealth)
		{
			OutCaches.bHasHealth = (MaxHealth > MinHealth + KINDA_SMALL_NUMBER);

			for (int32 i = 0; i < NumOfPoints; ++i)
			{
				const float Current = OutCaches.Health[i];
				OutCaches.Health[i] = OutCaches.bHasHealth
					? FMath::GetMappedRangeValueClamped(
						FVector2f(MinHealth, MaxHealth),
						FVector2f(0.f, 1.f),
						Current)
					: 0.0f;
			}
		}

		if (Options.bComputeHeight && PointPositionAttribute.IsValid())
		{
			OutCaches.bHasHeight = (MaxHeight > MinHeight + KINDA_SMALL_NUMBER);

			for (int32 i = 0; i < NumOfPoints; ++i)
			{
				OutCaches.Height[i] = OutCaches.bHasHeight
					? FMath::GetMappedRangeValueClamped(
						FVector2f(MinHeight, MaxHeight),
						FVector2f(0.f, 1.f),
						OutCaches.Height[i])
					: 0.0f;
			}
		}

		if (Options.bComputeGeneration)
		{
			OutCaches.bHasGeneration = (MaxGeneration > MinGeneration + KINDA_SMALL_NUMBER);

			for (int32 i = 0; i < NumOfPoints; ++i)
			{
				const float Current = OutCaches.Generation[i];
				OutCaches.Generation[i] = OutCaches.bHasGeneration
					? FMath::GetMappedRangeValueClamped(
						FVector2f(MinGeneration, MaxGeneration),
						FVector2f(0.f, 1.f),
						Current)
					: 0.0f;
			}
		}
	}

	namespace ConditionPicker
	{
		struct FWeightInfo
		{
			int32 CandidateIndex = INDEX_NONE;
			float Weight = 0.0f;
		};

		bool GetActiveConditionSettings(
			const FPVDistributionConditionParams& DistributionConditions,
			const EPVDistributionCondition Condition,
			FPVDistributionConditionInfluence& OutSettings)
		{
			if (!DistributionConditions.IsActiveCondition(Condition))
			{
				return false;
			}

			return DistributionConditions.GetInfluence(Condition, OutSettings);
		}

		static void ShuffleInPlace(TArray<FWeightInfo>& Weights, const FRandomStream& RandomStream)
		{
			for (int32 i = Weights.Num() - 1; i > 0; --i)
			{
				const int32 SwapIndex = RandomStream.RandRange(0, i);
				Weights.Swap(i, SwapIndex);
			}
		}

		int32 PickCandidateIndex(
			const int32 NumCandidates,
			const FPVDistributionConditionParams& DistributionConditions,
			const FPointConditionSample& PointSample,
			const FRandomStream& RandomStream,
			const FCandidateValueGetter& GetCandidateValue,
			const FCandidateValidFn& IsCandidateValid)
		{
			if (NumCandidates < 1)
			{
				return INDEX_NONE;
			}

			TArray<FWeightInfo> Weights;
			Weights.Reserve(NumCandidates);

			for (int32 i = 0; i < NumCandidates; ++i)
			{
				if (IsCandidateValid(i))
				{
					Weights.Add({i, 0.0f});
				}
			}

			if (Weights.Num() == 0)
			{
				return INDEX_NONE;
			}

			if (!DistributionConditions.HasActiveCondition())
			{
				return Weights[RandomStream.RandRange(0, Weights.Num() - 1)].CandidateIndex;
			}

			ShuffleInPlace(Weights, RandomStream);

			for (const EPVDistributionCondition Condition : TEnumRange<EPVDistributionCondition>())
			{
				FPVDistributionConditionInfluence ConditionSettings;
				if (!GetActiveConditionSettings(DistributionConditions, Condition, ConditionSettings))
				{
					continue;
				}

				const float Target = PointSample.Get(Condition) + ConditionSettings.Offset;

				for (auto& [CandidateIndex, Weight] : Weights)
				{
					const float CandidateValue = GetCandidateValue(CandidateIndex, Condition);
					const float Dist = FMath::Abs(CandidateValue - Target);
					Weight += (Dist * ConditionSettings.Weight);
				}
			}

			float MinWeight = TNumericLimits<float>::Max();
			float MaxWeight = TNumericLimits<float>::Lowest();
			for (const auto& [CandidateIndex, Weight] : Weights)
			{
				MinWeight = FMath::Min(MinWeight, Weight);
				MaxWeight = FMath::Max(MaxWeight, Weight);
			}

			const bool bHasRange = (MaxWeight > MinWeight + KINDA_SMALL_NUMBER);
			for (auto& [CandidateIndex, Weight] : Weights)
			{
				Weight = bHasRange
					? FMath::GetMappedRangeValueClamped(
						FVector2f(MinWeight, MaxWeight),
						FVector2f(0.0f, 1.0f),
						Weight)
					: 0.0f;
			}

			Weights.Sort([](const FWeightInfo& A, const FWeightInfo& B)
				{
					return A.Weight < B.Weight;
				});

			const int32 MinCandidates = FMath::Clamp(DistributionConditions.MinimumCandidates, 1, Weights.Num());

			TArray<int32> Picked;
			Picked.Reserve(Weights.Num());

			for (const auto& [CandidateIndex, Weight] : Weights)
			{
				if (Weight < DistributionConditions.CutoffThreshold)
				{
					Picked.Add(CandidateIndex);
				}
				else if (Picked.Num() < MinCandidates)
				{
					Picked.Add(CandidateIndex);
				}
				else
				{
					break;
				}
			}

			if (Picked.Num() == 0)
			{
				return Weights[0].CandidateIndex;
			}

			return Picked[RandomStream.RandRange(0, Picked.Num() - 1)];
		}

		int32 PickCandidateIndex(
			const int32 NumCandidates,
			const FPVDistributionConditionParams& DistributionConditions,
			const FPointConditionSample& PointSample,
			const FRandomStream& RandomStream,
			const FCandidateValueGetter& GetCandidateValue)
		{
			return PickCandidateIndex(
				NumCandidates,
				DistributionConditions,
				PointSample,
				RandomStream,
				GetCandidateValue,
				[](int32)
					{
						return true;
					});
		}

		float GetAttributeValue(const FPVDistributionConditions& FoliageAttributes, const EPVDistributionCondition InCondition)
		{
			switch (InCondition)
			{
			case EPVDistributionCondition::Light:
				return FoliageAttributes.Light;
			case EPVDistributionCondition::Scale:
				return FoliageAttributes.Scale;
			case EPVDistributionCondition::UpAlignment:
				return FoliageAttributes.UpAlignment;
			case EPVDistributionCondition::Tip:
				return FoliageAttributes.Tip ? 1.0f : 0.0f;
			case EPVDistributionCondition::Health:
				return FoliageAttributes.Health;
			case EPVDistributionCondition::Height:
				return FoliageAttributes.Height;
			case EPVDistributionCondition::Generation:
				return FoliageAttributes.Generation;
			default:
				return 0.0f;
			}
		}
	}
}

static FQuat4f AlignVectorToPlane(const FVector3f& InputVec, const FVector3f& FlatAxis)
{
	const FVector3f InputNormal = InputVec.GetSafeNormal();
	const FVector3f PlaneNormal = FlatAxis.GetSafeNormal();

	if (InputNormal.IsNearlyZero() || PlaneNormal.IsNearlyZero())
	{
		return FQuat4f::Identity;
	}

	const FVector3f FlatVec = FVector3f::VectorPlaneProject(InputNormal, PlaneNormal).GetSafeNormal();

	if (FlatVec.IsNearlyZero())
	{
		return FQuat4f::Identity;
	}

	return FQuat4f::FindBetweenNormals(InputNormal, FlatVec);
}

static FQuat4f MakeRotationBetweenVectors(const FVector3f& From, const FVector3f& To)
{
	return FQuat4f::FindBetweenNormals(
		From.GetSafeNormal(),
		To.GetSafeNormal()
	);
}

FQuat4f PV::DistributionVectorUtils::FetchAimTarget(const EPVAimVectorType Type, const FVector3f& InputVector, const FPointVectorData& Point, const FVector3f VectorAxis, const float BlendAmount)
{
	const FQuat4f BaseQuat = FQuat4f::Identity;
	FQuat4f AimQuat = FQuat4f::Identity;
	
	switch (Type)
	{
	case EPVAimVectorType::BranchUpFlatten:
		AimQuat = AlignVectorToPlane(InputVector, Point.UpVector);
		break;
	case EPVAimVectorType::AxisFlatten:
		AimQuat = AlignVectorToPlane(InputVector, VectorAxis);
		break;
	case EPVAimVectorType::AxisAim:
		AimQuat = MakeRotationBetweenVectors(InputVector, VectorAxis);
		break;
	case EPVAimVectorType::LightOptimal:
		AimQuat = MakeRotationBetweenVectors(InputVector, Point.LightOptimal);
		break;
	case EPVAimVectorType::LightAvoid:
		AimQuat = MakeRotationBetweenVectors(InputVector, Point.LightSubOptimal);
		break;
	default:
		AimQuat = FQuat4f::Identity;
	}
	
	return FQuat4f::Slerp(BaseQuat, AimQuat, BlendAmount).GetNormalized();
}

FQuat4f PV::DistributionVectorUtils::FetchFaceTarget(const EPVFaceVectorType Type, const FVector3f& InputVector, const FPointVectorData& Point, const FVector3f VectorAxis, const float BlendAmount)
{
	const FQuat4f BaseQuat = FQuat4f::Identity;
	FQuat4f AimQuat = FQuat4f::Identity;
	
	switch (Type)
	{
	case EPVFaceVectorType::Apical:
		AimQuat = MakeRotationBetweenVectors(InputVector, Point.ApicalDirection);
		break;
	case EPVFaceVectorType::Branch:
		AimQuat = MakeRotationBetweenVectors(InputVector, Point.UpVector);
		break;
	case EPVFaceVectorType::AxisFlatten:
		AimQuat = AlignVectorToPlane(InputVector, VectorAxis);
		break;
	case EPVFaceVectorType::AxisAim:
		AimQuat = MakeRotationBetweenVectors(InputVector, VectorAxis);
		break;
	case EPVFaceVectorType::LightOptimal:
		AimQuat = MakeRotationBetweenVectors(InputVector, Point.LightOptimal);
		break;
	case EPVFaceVectorType::LightAvoid:
		AimQuat = MakeRotationBetweenVectors(InputVector, Point.LightSubOptimal);
		break;
	default:
		AimQuat = FQuat4f::Identity;
	}
	
	return FQuat4f::Slerp(BaseQuat, AimQuat, BlendAmount).GetNormalized();
}

float PV::DistributionVectorUtils::GetBlendPosition(const EPVAimVectorBlendAttribute BlendAttr, const FPointVectorData& Point, const FVector3f InputVector)                                                                                                          
{                                                                                                                                          
	switch (BlendAttr)                                                                                                                      
	{                                                                                                                                       
	case EPVAimVectorBlendAttribute::PlantGradient:                                                                                         
		return Point.PlantGradient;
	case EPVAimVectorBlendAttribute::PlantGradientNormalized:                                                                                         
		return Point.PlantGradientNormalized;                                                                                                         
	case EPVAimVectorBlendAttribute::BranchGradient:                                                                                        
		return Point.BranchGradient;                                                                                                        
	case EPVAimVectorBlendAttribute::BranchGradientNormalized:                                                                              
		return Point.BranchGradientNormalized;                                                                                              
	case EPVAimVectorBlendAttribute::WorldUpDot:                                                                                            
		return FMath::GetMappedRangeValueClamped(FVector2f(-1.f, 1.f), FVector2f(0.f, 1.f),FVector3f::DotProduct(InputVector, FVector3f::UpVector));                                                                   
	default:                                                                                                                                
		return 0.f;                                                                                                                         
	}                                                                                                                                       
}  

void PV::DistributionVectorUtils::AdjustAimVectors(const FPVDistributionAimVectorSettings& InAimVectors, const FPointVectorData& InPointData, const FVector3f InputVector, FVector3f& OutUpVector, FVector3f& OutNormalVector)
{
	for (const FPVAimVectorSettings& Settings : InAimVectors.AimVectors)
	{
		if (InPointData.bIsTip && !Settings.bAffectTip)
		{
			continue;
		}	
		
		FQuat4f Vector1AimQuat = FQuat4f::Identity;
		FQuat4f Vector2AimQuat = FQuat4f::Identity;
		
		if (Settings.bDualVectors)
		{
			Vector1AimQuat = FetchAimTarget(Settings.Vector1, InputVector, InPointData, Settings.Vector1Axis, Settings.Vector1Strength);
		}
		
		Vector2AimQuat = FetchAimTarget(Settings.Vector2, InputVector, InPointData, Settings.Vector2Axis, Settings.Vector2Strength);
		
		const float RampValue = GetBlendPosition(Settings.BlendAttribute, InPointData, InputVector);
		const float BlendValue = Settings.VectorRamp.GetRichCurveConst()->Eval(RampValue);
		auto Rotation = FQuat4f::Slerp(Vector1AimQuat, Vector2AimQuat, BlendValue);
		
		OutUpVector  = Rotation.RotateVector(OutUpVector);                                                                                                             
	 	OutNormalVector = Rotation.RotateVector(OutNormalVector); 
	}
}

void PV::DistributionVectorUtils::AdjustFaceVectors(const FPVDistributionFaceVectorSettings& InFaceVectors, const FPointVectorData& InPointData, const FVector3f InputVector, FVector3f& OutNormalVector)
{
	for (const FPVFaceVectorSettings& Settings : InFaceVectors.FaceVectors)
	{
		if (InPointData.bIsTip && !Settings.bAffectTip)
		{
			continue;
		}	
		
		FQuat4f Vector1AimQuat = FQuat4f::Identity;
		FQuat4f Vector2AimQuat = FQuat4f::Identity;
		
		if (Settings.bDualVectors)
		{
			Vector1AimQuat = FetchFaceTarget(Settings.Vector1, InputVector, InPointData, Settings.Vector1Axis, Settings.Vector1Strength);
		}
		
		Vector2AimQuat = FetchFaceTarget(Settings.Vector2, InputVector, InPointData, Settings.Vector2Axis, Settings.Vector2Strength);
		
		const float RampValue = GetBlendPosition(Settings.BlendAttribute, InPointData, InputVector);
		const float BlendValue = Settings.VectorRamp.GetRichCurveConst()->Eval(RampValue);
		auto Rotation = FQuat4f::Slerp(Vector1AimQuat, Vector2AimQuat, BlendValue);
		
	 	OutNormalVector = Rotation.RotateVector(OutNormalVector); 
	}
}

void PV::DistributionVectorUtils::ApplyRollPitchYaw(const FPVDistributionRollPitchYawSettings& InSettings, const int32 InstanceIndex, FVector3f& InOutUpVector, FVector3f& InOutNormalVector)
{
	for (const FPVRollPitchYawSettings& Entry : InSettings.RollPitchYaw)
	{
		const uint32 SeedHash = HashCombine(GetTypeHash(InstanceIndex), GetTypeHash(Entry.RandomSeed));
		FRandomStream Stream(SeedHash);
		const float Random01 = Stream.FRand();
		
		float RotationAngle = FMath::Lerp(Entry.MinStrength, Entry.MaxStrength, Random01) * UE_PI;
		
		FVector3f RotationAxis;
		switch (Entry.Mode)
		{
			case EPVRollPitchYawMode::Roll:
				RotationAxis = InOutUpVector.GetSafeNormal();
				break;
			case EPVRollPitchYawMode::Pitch:
				RotationAxis = FVector3f::CrossProduct(InOutUpVector, InOutNormalVector).GetSafeNormal();
				break;
			case EPVRollPitchYawMode::Yaw:
			default:
				{
					const FVector3f CrossDir = FVector3f::CrossProduct(InOutUpVector, InOutNormalVector).GetSafeNormal();
					RotationAxis = FVector3f::CrossProduct(CrossDir, InOutUpVector).GetSafeNormal();
				}
				break;
		}

		if (RotationAxis.IsNearlyZero())
		{
			continue;
		}

		const FQuat4f RotQuat(RotationAxis, RotationAngle);
		InOutUpVector  = RotQuat.RotateVector(InOutUpVector);
		InOutNormalVector = RotQuat.RotateVector(InOutNormalVector);
	}
}

void PV::DistributionVectorUtils::ApplyVectorSettings(const FPVDistributionVectorParams& InVectorSettings, const FPointVectorData& InPointData, const int32 InstanceIndex, FVector3f& InOutUpVector, FVector3f& InOutNormalVector)
{
	if (InVectorSettings.AimVectorSettings.bAutoAlignEnd && InPointData.bIsTip)
	{
		InOutUpVector = InPointData.ApicalDirection;	
	}
	
	if (InVectorSettings.FaceVectorSettings.bAutoAlignEnd && InPointData.bIsTip)
	{
		InOutNormalVector = InPointData.UpVector;	
	}
	
	AdjustAimVectors(InVectorSettings.AimVectorSettings, InPointData, InOutUpVector, InOutUpVector, InOutNormalVector);
	AdjustFaceVectors(InVectorSettings.FaceVectorSettings, InPointData, InOutNormalVector, InOutNormalVector);
	ApplyRollPitchYaw(InVectorSettings.RollPitchYawSettings, InstanceIndex, InOutUpVector, InOutNormalVector);
}