// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "DataTypes/PVDistributionParams.h"
#include "DataTypes/PVFoliageInfo.h"

namespace PV::DistributionVectorUtils
{
	struct FPointVectorData                                                                                                                     
  	{                                                                                                                                           
	 	FVector3f ApicalDirection 			= FVector3f::ZeroVector; // TODO : Verify                                                                                  
	 	FVector3f LightOptimal    			= FVector3f::ZeroVector;                                                                                      
	 	FVector3f LightSubOptimal 			= FVector3f::ZeroVector;
	 	FVector3f UpVector 					= FVector3f::ZeroVector;
	 	float PlantGradient               	= 0.f;                                                                                                
	 	float BranchGradient              	= 0.f;                                                                                                
	 	float PlantGradientNormalized    	= 0.f;
	 	float BranchGradientNormalized    	= 0.f;
		bool bIsTip = false;
  	};
	
	FQuat4f FetchAimTarget(const EPVAimVectorType Type, const FVector3f& InputVector, const FPointVectorData& Point, const FVector3f VectorAxis, const float BlendAmount);  
	FQuat4f FetchFaceTarget(const EPVFaceVectorType Type, const FVector3f& InputVector, const FPointVectorData& Point, const FVector3f VectorAxis, const float BlendAmount);  
	
	float GetBlendPosition(const EPVAimVectorBlendAttribute BlendAttr, const FPointVectorData& Point, const FVector3f InputVector); 
	
	void AdjustAimVectors(const FPVDistributionAimVectorSettings& InAimVectors, const FPointVectorData& InPointData, const FVector3f InputVector, FVector3f& OutUpVector, FVector3f& OutNormalVector);
	void AdjustFaceVectors(const FPVDistributionFaceVectorSettings& InFaceVectors, const FPointVectorData& InPointData, const FVector3f InputVector, FVector3f& OutNormalVector);
	void ApplyRollPitchYaw(const FPVDistributionRollPitchYawSettings& InSettings, const int32 InstanceIndex, FVector3f& InOutUpVector, FVector3f& InOutNormalVector);
	void ApplyVectorSettings(const FPVDistributionVectorParams& InVectorSettings, const FPointVectorData& InPointData, const int32 InstanceIndex, FVector3f& InOutUpVector, FVector3f& InOutNormalVector);
};

namespace PV::DistributionConditionUtils
{
	struct FConditionAttributes
	{
		TArray<float> UpAlignment;
		TArray<float> Light;
		TArray<float> Scale;
		TArray<float> Tip;
		TArray<float> Health;
		TArray<float> Height;
		TArray<float> Generation;

		float GetConditionValue(EPVDistributionCondition InCondition, const int Index) const;
	};
	
	struct FNormalizedPointCaches
	{
		TArray<float> PScales;
		TArray<float> Light;
		TArray<float> Health;
		TArray<float> Height;
		TArray<float> Generation;
		bool bHasPScales = false;
		bool bHasLight  = false;
		bool bHasHealth = false;
		bool bHasHeight = false;
		bool bHasGeneration = false;

		void Reset(const int32 NumPoints);
	};

	struct FNormalizationInputs
	{
		bool bComputePScales    = true;
		bool bComputeLight      = true;
		bool bComputeHealth     = true;
		bool bComputeHeight     = true;
		bool bComputeGeneration = true;
		TArray<float> PScalesOverride;
	};

	FORCEINLINE float UpAlignmentNormalized(const FVector3f& ApicalDirection)
	{
		const float Dot = FVector3f::DotProduct(ApicalDirection.GetSafeNormal(), FVector3f::UpVector);
		
		return FMath::GetMappedRangeValueClamped(
			FVector2f(-1.0f, 1.0f), 
			FVector2f(0.0f, 1.0f), 
			Dot);
	}

	FORCEINLINE float Tip(const bool bIsTip)
	{
		return bIsTip ? 1.0f : 0.0f;
	}

	void BuildNormalizedPointCaches(
		const FManagedArrayCollection& Collection,
		const FNormalizationInputs& Options,
		FNormalizedPointCaches& OutCaches);
	
	namespace ConditionPicker
	{
		struct FPointConditionSample
		{
			float Light = 0.0f;
			float Scale = 0.0f;
			float UpAlignment = 0.0f;
			float Tip = 0.0f;
			float Health = 0.0f;
			float Height = 0.0f;
			float Generation = 0.0f;

			FORCEINLINE float Get(EPVDistributionCondition Condition) const
			{
				switch (Condition)
				{
				case EPVDistributionCondition::Light:
					return Light;
				case EPVDistributionCondition::Scale:
					return Scale;
				case EPVDistributionCondition::UpAlignment:
					return UpAlignment;
				case EPVDistributionCondition::Tip:
					return Tip;
				case EPVDistributionCondition::Health:
					return Health;
				case EPVDistributionCondition::Height:
					return Height;
				case EPVDistributionCondition::Generation:
					return Generation;
				default:
					return 0.0f;
				}
			}
		};

		using FCandidateValueGetter = TFunctionRef<float(int32, EPVDistributionCondition)>;

		using FCandidateValidFn = TFunctionRef<bool(int32)>;

		int32 PickCandidateIndex(
			const int32 NumCandidates,
			const FPVDistributionConditionParams& DistributionConditions,
			const FPointConditionSample& PointSample,
			const FRandomStream& RandomStream,
			const FCandidateValueGetter& GetCandidateValue,
			const FCandidateValidFn& IsCandidateValid);

		int32 PickCandidateIndex(
			const int32 NumCandidates,
			const FPVDistributionConditionParams& DistributionConditions,
			const FPointConditionSample& PointSample,
			const FRandomStream& RandomStream,
			const FCandidateValueGetter& GetCandidateValue);

		bool GetActiveConditionSettings(
			const FPVDistributionConditionParams& DistributionConditions,
			const EPVDistributionCondition Condition,
			FPVDistributionConditionInfluence& OutSettings);

		float GetAttributeValue(
			const FPVDistributionConditions& FoliageAttributes, 
			const EPVDistributionCondition InCondition);
	}
}

namespace PV::DistributionHelper
{
	struct FAttachmentPoint
	{
		int32 BranchIndex;
		int32 PointIndexA;
		int32 PointIndexB;
		float PointAlpha;
		FVector3f Position;
		FVector3f UpDirection;
		FVector3f NormalDirection;

		float Scale;
		float LengthFromRoot;
		bool bIsTipInstance;
		int32 TipPointIndex;

		DistributionConditionUtils::ConditionPicker::FPointConditionSample ConditionSample;
	};
}