// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PVDynamicMeshHelpers.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

PV::DynamicMeshHelper::FFindMeshElemsResult PV::DynamicMeshHelper::GetUniqueVertexIndices(const UE::Geometry::FDynamicMesh3& DynamicMesh, const TArray<int32>& TriangleIds)
{
	TSet<int32> UniqueVertexIndices;
	UniqueVertexIndices.Reserve(TriangleIds.Num() * 3);
	for (int32 tid : TriangleIds)
	{
		const UE::Geometry::FIndex3i VertexIndices = DynamicMesh.GetTriangle(tid);
		UniqueVertexIndices.Add(VertexIndices[0]);
		UniqueVertexIndices.Add(VertexIndices[1]);
		UniqueVertexIndices.Add(VertexIndices[2]);
	}

	PV::DynamicMeshHelper::FFindMeshElemsResult OutResult;
	OutResult.Reserve(UniqueVertexIndices.Num());

	for (int32 VID : UniqueVertexIndices)
	{
		OutResult.Add(VID);
	}

	return OutResult;
}

PV::DynamicMeshHelper::FFindMeshElemsResult PV::DynamicMeshHelper::GetVertexNeighbours(const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 VertexID)
{
	FFindMeshElemsResult OutNeighbours;
	OutNeighbours.Reserve(DynamicMesh.GetVtxEdgeCount(VertexID));
	for (int32 EdgeID : DynamicMesh.VtxEdgesItr(VertexID))
	{
		const UE::Geometry::FIndex2i VertexIDs = DynamicMesh.GetEdgeV(EdgeID);
		const int32 OtherVertexID = VertexIDs.A == VertexID ? VertexIDs.B : VertexIDs.A;
		OutNeighbours.Add(OtherVertexID);
	}
	return OutNeighbours;
}

PV::DynamicMeshHelper::FFindMeshElemsResult PV::DynamicMeshHelper::FindVerticesInRadius(const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree, const FVector& Pos, float Radius)
{
	FFindMeshElemsResult OutVertexIndices;

	const UE::Geometry::FDynamicMesh3* DynamicMesh = DynamicMeshAABBTree.GetMesh();
	if (DynamicMesh == nullptr)
	{
		return OutVertexIndices;
	}

	OutVertexIndices.Reserve(DynamicMesh->VertexCount());

	const double RadiusSqr = Radius * Radius;

	UE::Geometry::FDynamicMeshAABBTree3::FTreeTraversal TreeTraversal;

	TreeTraversal.NextBoxF = [&](const UE::Geometry::FAxisAlignedBox3d& Box, int32 Depth)->bool
	{
		return Box.DistanceSquared(Pos) <= RadiusSqr;
	};

	TreeTraversal.NextTriangleF = [&](int32 TID)
	{
		const UE::Geometry::FIndex3i Triangle = DynamicMesh->GetTriangle(TID);
		for (int32 j = 0; j < 3; ++j)
		{
			const double VertexDistSqr = (DynamicMesh->GetVertex(Triangle[j]) - Pos).SizeSquared();
			if (VertexDistSqr <= RadiusSqr)
			{
				OutVertexIndices.AddUnique(Triangle[j]);
			}
		}
	};

	DynamicMeshAABBTree.DoTraversal(TreeTraversal);

	return OutVertexIndices;
}

PV::DynamicMeshHelper::FFindMeshElemsResult PV::DynamicMeshHelper::FindTrianglesInRadius(const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree, const FVector& Pos, float Radius)
{
	FFindMeshElemsResult OutTriangleIDs;

	const UE::Geometry::FDynamicMesh3* DynamicMesh = DynamicMeshAABBTree.GetMesh();
	if (DynamicMesh == nullptr)
	{
		return OutTriangleIDs;
	}

	const double RadiusSqr = Radius * Radius;

	UE::Geometry::FDynamicMeshAABBTree3::FTreeTraversal TreeTraversal;

	TreeTraversal.NextBoxF = [&](const UE::Geometry::FAxisAlignedBox3d& Box, int32 Depth)->bool
	{
		return Box.DistanceSquared(Pos) <= RadiusSqr;
	};

	TreeTraversal.NextTriangleF = [&](int32 TID)
	{
		const UE::Geometry::FIndex3i Triangle = DynamicMesh->GetTriangle(TID);

		using FDynamicMeshQueries3 = UE::Geometry::TMeshQueries<UE::Geometry::FDynamicMesh3>;
		const UE::Geometry::FDistPoint3Triangle3d Query = FDynamicMeshQueries3::TriangleDistance(*DynamicMesh, TID, Pos);

		if (FVector::DistSquared(Pos, Query.ClosestTrianglePoint) < RadiusSqr)
		{
			OutTriangleIDs.Add(TID);
		}
	};

	DynamicMeshAABBTree.DoTraversal(TreeTraversal);

	return OutTriangleIDs;
}

PV::DynamicMeshHelper::FFindMeshElemsResult PV::DynamicMeshHelper::FindTrianglesInRadius(
	const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree,
	const FVector& Pos,
	float Radius,
	TFunctionRef<float(int32)> GetVertexScale)
{
	const UE::Geometry::FDynamicMesh3* DynamicMesh = DynamicMeshAABBTree.GetMesh();
	if (DynamicMesh == nullptr)
	{
		return FFindMeshElemsResult();
	}

	float GlobalMaxVertexScale = 0.f;
	for (int32 VID : DynamicMesh->VertexIndicesItr())
	{
		GlobalMaxVertexScale = FMath::Max(GlobalMaxVertexScale, GetVertexScale(VID));
	}

	return FindTrianglesInRadius(DynamicMeshAABBTree, Pos, Radius, GetVertexScale, GlobalMaxVertexScale);
}

PV::DynamicMeshHelper::FFindMeshElemsResult PV::DynamicMeshHelper::FindTrianglesInRadius(
	const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree,
	const FVector& Pos,
	float Radius,
	TFunctionRef<float(int32)> GetVertexScale,
	float GlobalMaxVertexScale)
{
	FFindMeshElemsResult OutTriangleIDs;

	const UE::Geometry::FDynamicMesh3* DynamicMesh = DynamicMeshAABBTree.GetMesh();
	if (DynamicMesh == nullptr)
	{
		return OutTriangleIDs;
	}

	// Conservative upper bound: a box must be visited if any triangle inside it
	// could pass the per-triangle test. The per-triangle check below still
	// re-evaluates each candidate precisely.
	const double BoxRadiusSqr = FMath::Square((double)(GlobalMaxVertexScale + Radius));

	UE::Geometry::FDynamicMeshAABBTree3::FTreeTraversal TreeTraversal;

	TreeTraversal.NextBoxF = [&](const UE::Geometry::FAxisAlignedBox3d& Box, int32 Depth) -> bool
	{
		return Box.DistanceSquared(Pos) <= BoxRadiusSqr;
	};

	TreeTraversal.NextTriangleF = [&](int32 TID)
	{
		const UE::Geometry::FIndex3i Triangle = DynamicMesh->GetTriangle(TID);

		// Use the maximum vertex scale in the triangle as a conservative
		// upper bound for TargetScale at the closest point on the triangle.
		float MaxVertexScale = 0.f;
		for (int32 j = 0; j < 3; ++j)
		{
			MaxVertexScale = FMath::Max(MaxVertexScale, GetVertexScale(Triangle[j]));
		}

		using FDynamicMeshQueries3 = UE::Geometry::TMeshQueries<UE::Geometry::FDynamicMesh3>;
		const UE::Geometry::FDistPoint3Triangle3d Query = FDynamicMeshQueries3::TriangleDistance(*DynamicMesh, TID, Pos);

		const float AllowedDist = MaxVertexScale + Radius;
		if (FVector::DistSquared(Pos, Query.ClosestTrianglePoint) <= FMath::Square(AllowedDist))
		{
			OutTriangleIDs.Add(TID);
		}
	};

	DynamicMeshAABBTree.DoTraversal(TreeTraversal);

	return OutTriangleIDs;
}