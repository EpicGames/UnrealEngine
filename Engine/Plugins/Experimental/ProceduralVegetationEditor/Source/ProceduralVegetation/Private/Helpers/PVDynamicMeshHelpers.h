// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FManagedArrayCollection;

namespace UE::Geometry
{
	class FDynamicMesh3;
	template<class T> class TMeshAABBTree3;
	using FDynamicMeshAABBTree3 = TMeshAABBTree3<FDynamicMesh3>;
}

namespace PV::DynamicMeshHelper
{
	using FFindMeshElemsResult = TArray<int32, TInlineAllocator<4>>;

	FFindMeshElemsResult GetUniqueVertexIndices(const UE::Geometry::FDynamicMesh3& DynamicMesh, const TArray<int32>& TriangleIds);

	FFindMeshElemsResult GetVertexNeighbours(const UE::Geometry::FDynamicMesh3& DynamicMesh, int32 VertexID);

	FFindMeshElemsResult FindVerticesInRadius(const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree, const FVector& Pos, float Radius);

	FFindMeshElemsResult FindTrianglesInRadius(const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree, const FVector& Pos, float Radius);

	FFindMeshElemsResult FindTrianglesInRadius(
		const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree,
		const FVector& Pos,
		float Radius,
		TFunctionRef<float(int32)> GetVertexScale);

	FFindMeshElemsResult FindTrianglesInRadius(
		const UE::Geometry::FDynamicMeshAABBTree3& DynamicMeshAABBTree,
		const FVector& Pos,
		float Radius,
		TFunctionRef<float(int32)> GetVertexScale,
		float GlobalMaxVertexScale);
};