// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PVFoliageJSONHelper.h"

#include "ProceduralVegetationModule.h"
#include "Implementations/PVFoliage.h"
#include "Facades/PVAttributesNames.h"
#include "Facades/PVFoliageFacade.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Facades/PVFoliageConditionFacade.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"

namespace PV
{
	bool PVFoliageJSONHelper::LoadJSON(const FString& InPath, TSharedPtr<FJsonObject>& OutData, FString& OutErrorMessage)
	{
		FString JsonString;
		if (!FFileHelper::LoadFileToString(JsonString, *InPath))
		{
			OutErrorMessage = TEXT("Failed to open file");
			return false;
		}

		OutData = MakeShareable(new FJsonObject);
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonString), OutData))
		{
			OutErrorMessage = TEXT("Failed to deserialize JSON file");
			return false;
		}

		return true;
	}

	bool GetConditionFromString(const FString& InString, EPVDistributionCondition& OutCondition)
	{
		if (InString.Equals("light", ESearchCase::IgnoreCase))
		{
			OutCondition = EPVDistributionCondition::Light;
			return true;
		}
		else if (InString.Equals("scale", ESearchCase::IgnoreCase))
		{
			OutCondition = EPVDistributionCondition::Scale;
			return true;
		}
		else if (InString.Equals("upAlignment", ESearchCase::IgnoreCase))
		{
			OutCondition = EPVDistributionCondition::UpAlignment;
			return true;
		}
		else if (InString.Equals("health", ESearchCase::IgnoreCase))
		{
			OutCondition = EPVDistributionCondition::Health;
			return true;
		}
		else if (InString.Equals("tip", ESearchCase::IgnoreCase))
		{
			OutCondition = EPVDistributionCondition::Tip;
			return true;
		}
			
		return false;
	}

	bool PVFoliageJSONHelper::LoadFoliageData(const FString& InPath, const FString& InPackagePath, FoliageVariationsMap& OutFoliageVariationsMap, FString& OutErrorMessage)
	{
		TSharedPtr<FJsonObject> OutData = MakeShareable(new FJsonObject);
		
		OutFoliageVariationsMap.Empty();

		if (LoadJSON(InPath, OutData, OutErrorMessage))
		{
			if (OutData->HasTypedField(TEXT("Variations"), EJson::Array))
			{
				const TArray<TSharedPtr<FJsonValue>>& Variations = OutData->GetArrayField(TEXT("Variations"));

				for (TSharedPtr<FJsonValue> FoliageInfoVariation : Variations)
				{
					if (FoliageInfoVariation->Type == EJson::Object)
					{
						const TSharedPtr<FJsonObject> VariationData = FoliageInfoVariation->AsObject();

						FString VariationName = VariationData->GetStringField(TEXT("name"));

						const TArray<TSharedPtr<FJsonValue>>* OutVariationData;

						if (!VariationData->TryGetArrayField(TEXT("Data"), OutVariationData))
						{
							OutErrorMessage = FString::Printf(TEXT("Unable to load foliage data : Error: Missing data for variation : %s"), *VariationName);
							return false;
						}

						TArray<FPVFoliageInfo> FoliageInfos;
						
						for (TSharedPtr<FJsonValue> FoliageInfoData : *OutVariationData)
						{
							if (const TSharedPtr<FJsonObject> FoliageInfoObject = FoliageInfoData->AsObject())
							{
								FPVFoliageInfo FoliageInfo;
								FString BranchName;

								if (!FoliageInfoObject->TryGetStringField(TEXT("Name"), BranchName))
								{
									OutErrorMessage = FString::Printf(TEXT("FoliageInfo Doesnt contain valid branch name for variation %s"), *VariationName);
									return false;
								}
								
								FoliageInfo.Mesh = InPackagePath / BranchName + "." + BranchName;
								
								if (!FoliageInfoObject->TryGetNumberField(TEXT("scale"), FoliageInfo.Attributes.Scale))
								{
									FoliageInfo.Attributes.Scale = 0;
								}
								if (!FoliageInfoObject->TryGetNumberField(TEXT("upAlignment"), FoliageInfo.Attributes.UpAlignment))
								{
									FoliageInfo.Attributes.UpAlignment = 0;
								}

								if (!FoliageInfoObject->TryGetNumberField(TEXT("light"), FoliageInfo.Attributes.Light))
								{
									FoliageInfo.Attributes.Light = 0;
								}

								if (!FoliageInfoObject->TryGetNumberField(TEXT("health"), FoliageInfo.Attributes.Health))
								{
									FoliageInfo.Attributes.Health = 0;
								}

								float OutTip; 
								if (FoliageInfoObject->TryGetNumberField(TEXT("tip"), OutTip))
								{
									FoliageInfo.Attributes.Tip = static_cast<bool>(OutTip);	
								}
								else
								{
									FoliageInfo.Attributes.Tip = false;
								}
								

								FoliageInfos.Add(FoliageInfo);
							}
						}

						const TArray<TSharedPtr<FJsonValue>>* OutVariationRules;

						if (!VariationData->TryGetArrayField(TEXT("Rules"), OutVariationRules))
						{
							OutErrorMessage = FString::Printf(TEXT("Unable to load foliage rules : Error: Missing data for variation : %s"), *VariationName);
							return false;
						}

						TMap<EPVDistributionCondition, Facades::FFoliageConditonInfo> FoliageDistributionConditions;
						
						for (TSharedPtr<FJsonValue> FoliageRule : *OutVariationRules)
						{
							if (const TSharedPtr<FJsonObject> Rule = FoliageRule->AsObject())
							{
								FString RuleName;
								
								if (!Rule->TryGetStringField(TEXT("name"), RuleName))
								{
									OutErrorMessage = FString::Printf(TEXT("FoliageData: Rule doesnt contain valid name for variation %s"), *VariationName);
									continue;
								}

								EPVDistributionCondition OutCondition;
								
								if (GetConditionFromString(RuleName, OutCondition))
								{
									Facades::FFoliageConditonInfo ConditionSettings;

									const UEnum* EnumPtr = StaticEnum<EPVDistributionCondition>();
									ConditionSettings.Name = EnumPtr->GetNameStringByValue(static_cast<int64>(OutCondition));
									
									if (!Rule->TryGetNumberField(TEXT("weight"), ConditionSettings.Weight))
									{
										ConditionSettings.Weight = 0;
									}

									if (!Rule->TryGetNumberField(TEXT("offset"), ConditionSettings.Offset))
									{
										ConditionSettings.Offset = 0;
									}

									// float OutNormalize; 
									// if (Rule->TryGetNumberField(TEXT("normalize"), OutNormalize))
									// {
									// 	ConditionSettings.Normalize = static_cast<bool>(OutNormalize);	
									// }
									// else
									// {
									// 	ConditionSettings.Normalize = false;
									// }

									if (!FoliageDistributionConditions.Contains(OutCondition))
									{
										FoliageDistributionConditions.Add(OutCondition);
									}
									
									FoliageDistributionConditions[OutCondition] = ConditionSettings;
								}
							}
						}

						if (!OutFoliageVariationsMap.Contains(VariationName))
						{
							OutFoliageVariationsMap.Add(VariationName);
						}

						OutFoliageVariationsMap[VariationName].VariationName = VariationName;
						OutFoliageVariationsMap[VariationName].FoliageInfos = FoliageInfos;
						OutFoliageVariationsMap[VariationName].Conditions = FoliageDistributionConditions;
					}
				}
			}
			else
			{
				OutErrorMessage = FString::Printf(TEXT("Unable to load foliage data : Error: Missing variation data"));
				
				return false;	
			}
		}

		for (auto Mapping : OutFoliageVariationsMap)
		{
			UE_LOGF(LogProceduralVegetation, Log, "Variation name : %ls : Start", *Mapping.Key);

			for (const FPVFoliageInfo& FoliageInfo : Mapping.Value.FoliageInfos)
			{
				UE_LOGF(LogProceduralVegetation, Log, "FoliageInfo : Mesh : %ls : Scale : %f : Light : %f : Tip : %i : UpAlignment :  %f : Health : %f",
					*FoliageInfo.Mesh.ToString(),
					FoliageInfo.Attributes.Scale,
					FoliageInfo.Attributes.Light,
					FoliageInfo.Attributes.Tip,
					FoliageInfo.Attributes.UpAlignment,
					FoliageInfo.Attributes.Health);	
			}

			for (const auto& ConditionEntry : Mapping.Value.Conditions)
			{
				UE_LOGF(LogProceduralVegetation, Log, "FoliageCondition : %ls", *ConditionEntry.Value.ToString());	
			}

			UE_LOGF(LogProceduralVegetation, Log, "Variation name : %ls : End", *Mapping.Key);
		}

		return true;
	}

	bool PVFoliageJSONHelper::LoadFoliageDataInCollection(FManagedArrayCollection& Collection, const FString& InPresetPath, const FPVFoliageVariationData& VariationData, FString& OutErrorMessage)
	{
		TSharedPtr<FJsonObject> OutData = MakeShareable(new FJsonObject);

		FString MegaPlantsData;
		if (!FFileHelper::LoadFileToString(MegaPlantsData, *InPresetPath))
		{
			OutErrorMessage = TEXT("Failed to open skeleton file");
			return false;
		}

		TSharedPtr<FJsonObject> LoadedData = MakeShareable(new FJsonObject);
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(MegaPlantsData), LoadedData))
		{
			OutErrorMessage = TEXT("Failed to deserialize JSON file for skeleton");
			return false;
		}

		Collection.RemoveGroup(GroupNames::FoliageGroup);
		Collection.RemoveGroup(GroupNames::FoliageNamesGroup);
		
		const TSharedPtr<FJsonObject>& Primitives = LoadedData->GetObjectField(TEXT("primitives"));
		const TSharedPtr<FJsonObject>& PrimitiveAttributes = Primitives->GetObjectField(TEXT("attributes"));

		const TSharedPtr<FJsonObject>& InstancerNameObject = PrimitiveAttributes->GetObjectField(TEXT("instancer_name"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerNameValues = InstancerNameObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerPivotObject = PrimitiveAttributes->GetObjectField(TEXT("instancer_pivot"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerPivotValues = InstancerPivotObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerUpVectorObject = PrimitiveAttributes->GetObjectField(TEXT("instancer_UP"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerUpVectorValues = InstancerUpVectorObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerNormalVectorObject = PrimitiveAttributes->GetObjectField(TEXT("instancer_N"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerNormalVectorValues = InstancerNormalVectorObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerScaleObject = PrimitiveAttributes->GetObjectField(TEXT("instancer_scale"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerScaleValues = InstancerScaleObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerLFRObject = PrimitiveAttributes->GetObjectField(TEXT("instancer_LFR"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerLFRValues = InstancerLFRObject->GetArrayField(TEXT("values"));

		
		TMap<FString, int32> FoliageNamesToIndex;
		for (int FoliageIndex = 0; FoliageIndex < VariationData.FoliageInfos.Num(); ++FoliageIndex)
		{
			auto FoliageInfo = VariationData.FoliageInfos[FoliageIndex];
			FoliageNamesToIndex.Emplace(FoliageInfo.Mesh.GetAssetName(), FoliageIndex);
		}
		
		// Compute unique names and number of elements for group
		int32 TotalValues = 0;
		TArray<FString> FoliageNames;
		for (int32 i = 0; i < InstancerNameValues.Num(); ++i)
		{
			const TArray<TSharedPtr<FJsonValue>>& InstancerNames = InstancerNameValues[i]->AsArray();
			for (int32 Index = 0; Index < InstancerNames.Num(); ++Index)
			{
				const FString& Name = InstancerNames[Index]->AsString();

				if (!FoliageNamesToIndex.Contains(Name))
				{
					OutErrorMessage = FString::Printf(TEXT("FoliageData doesnt contain branch info : %s"), *Name);
					return false;
				}
			}
			TotalValues += InstancerNames.Num();
		}
		
		Facades::FFoliageFacade Facade(Collection, TotalValues);
		Facade.SetFoliageInfos(VariationData.FoliageInfos);

		TArray<Facades::FFoliageConditonInfo> Conditions;

		for (auto ConditionEntry : VariationData.Conditions)
		{
			Conditions.Add(ConditionEntry.Value);
		}

		Facades::FFoliageConditionFacade ConditionFacade(Collection);
		ConditionFacade.SetData(Conditions);
		
		int32 FoliageEntryIndex = 0;
		for (int32 Id = 0; Id < InstancerNameValues.Num(); ++Id)
		{
			const TArray<TSharedPtr<FJsonValue>>& InstancerNames = InstancerNameValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& PivotPoints = InstancerPivotValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& UpVectors = InstancerUpVectorValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& NormalVectors = InstancerNormalVectorValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& Scales = InstancerScaleValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& LFRs = InstancerLFRValues[Id]->AsArray();
		
			TArray<int32> FoliageEntryIds;
			for (int32 j = 0; j < InstancerNames.Num(); ++j)
			{
				const FString Name = InstancerNames[j]->AsString();
				const FVector3f PivotPoint = FVector3f(
					PivotPoints[j * 3]->AsNumber(),
					PivotPoints[(j * 3) + 2]->AsNumber(),
					PivotPoints[(j * 3) + 1]->AsNumber()) * 100.0f;
				const FVector3f UpVector = FVector3f(
					UpVectors[j * 3]->AsNumber(),
					UpVectors[(j * 3) + 2]->AsNumber(),
					UpVectors[(j * 3) + 1]->AsNumber());
				const FVector3f NormalVector = FVector3f(
					NormalVectors[j * 3]->AsNumber(),
					NormalVectors[(j * 3) + 2]->AsNumber(),
					NormalVectors[(j * 3) + 1]->AsNumber());
				const float Scale = Scales[j]->AsNumber();
				const float LengthFromRoot = LFRs[j]->AsNumber();
		
				Facade.SetFoliageEntry(FoliageEntryIndex, {
					.NameId = FoliageNamesToIndex[Name],
					.BranchId = Id,
					.PivotPoint = PivotPoint,
					.UpVector = UpVector,
					.NormalVector = NormalVector,
					.Scale = Scale,
					.LengthFromRoot = LengthFromRoot,
				});
		
				FoliageEntryIds.Add(FoliageEntryIndex);
				FoliageEntryIndex++;
			}
		
			Facade.SetFoliageIdsArray(Id, FoliageEntryIds);
		}

		return true;
	}


	void PVFoliageJSONHelper::SetFoliagePaths(FManagedArrayCollection& Collection, const FString& FilePath)
	{
		PV::Facades::FFoliageFacade Facade(Collection);

		FString AbsolutePath = FPaths::ConvertRelativePathToFull(FilePath);
		FString PackagePath;

		if (FPackageName::TryConvertFilenameToLongPackageName(AbsolutePath, PackagePath))
		{
			TArray<FPVFoliageInfo> Infos = Facade.GetFoliageInfos();
			FString PackageFolder = FPackageName::GetLongPackagePath(PackagePath);
			Facade.SetFoliagePath(PackageFolder);

			for (FPVFoliageInfo& Info : Infos)
			{
				Info.Mesh = PackageFolder / Info.Mesh.ToString();
			}
			
			Facade.SetFoliageInfos(Infos);
		}
	}
}
