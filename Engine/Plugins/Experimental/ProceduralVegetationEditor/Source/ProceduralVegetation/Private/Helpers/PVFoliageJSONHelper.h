// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Facades/PVFoliageConditionFacade.h"
#include "Templates/SharedPointer.h"

enum class EPVDistributionCondition : uint8;
class FJsonObject;
struct FPVFoliageInfo;
struct FManagedArrayCollection;

namespace PV
{
	typedef TMap<EPVDistributionCondition, Facades::FFoliageConditonInfo> FoliageConditionMap;
	
	struct FPVFoliageVariationData
	{
		FString VariationName;
		TArray<FPVFoliageInfo> FoliageInfos;
		FoliageConditionMap Conditions;
	};

	typedef TMap<FString, FPVFoliageVariationData> FoliageVariationsMap;
	
	struct PVFoliageJSONHelper
	{
		static bool LoadJSON(const FString& InPath, TSharedPtr<FJsonObject>& OutData, FString& OutErrorMessage);
		static bool LoadFoliageData(const FString& InPath, const FString& InPackagePath, FoliageVariationsMap& OutFoliageVariationsMap, FString& OutErrorMessage);
		static bool LoadFoliageDataInCollection(FManagedArrayCollection& Collection, const FString& InPresetPath, const FPVFoliageVariationData& VariationData, FString& OutErrorMessage);
		static void SetFoliagePaths(FManagedArrayCollection& Collection, const FString& FilePath);
	};	
}