// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVHormoneDistributionHelper.h"
#include "Helpers/PVAttributesHelper.h"
#include "Helpers/PVUtilities.h"

TArray<float> PV::HormoneDistributionHelper::RemapPlantGradientToEthyleneThreshold(
	PV::FPointPlantGradientAttributeConstView PointPlantGradientAttribute,
	PV::FBudHormoneLevelsAttributeConstView BudHormoneLevels,
	float EthyleneThreshold)
{
	TArray<float> NormalizedPlantGradients;

	float MinGradient = 1.0f;
	float MaxGradient = 0.0f;

	const int32 NumPoints = PointPlantGradientAttribute.Num();
	
	for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		const float Ethylene = BudHormoneLevels[PointIndex].Ethylene;
		
		if (Ethylene <= EthyleneThreshold)
		{
			const float PlantGradient = 1 - PointPlantGradientAttribute[PointIndex];
			
			MinGradient = FMath::Min(MinGradient, PlantGradient);
			MaxGradient = FMath::Max(MaxGradient, PlantGradient);
		}
	}

	NormalizedPlantGradients.Reserve(NumPoints);
	
	for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		float NormalizedGradient = 0.0f;
		
		if (MaxGradient > MinGradient)
		{
			float Gradient = 1 - PointPlantGradientAttribute[PointIndex];
			NormalizedGradient = FMath::GetMappedRangeValueClamped(FVector2f(MinGradient, MaxGradient), FVector2f(0.0f, 1.0f),
				FMath::Clamp(Gradient, MinGradient, MaxGradient));
		}
		
		NormalizedPlantGradients.Add(NormalizedGradient);
	}

	return NormalizedPlantGradients;
}

TArray<float> PV::HormoneDistributionHelper::RemapPScalesToEthyleneThreshold(
	PV::FPointScaleAttributeConstView PointScaleAttribute,
	PV::FBudHormoneLevelsAttributeConstView BudHormoneLevels,
	float EthyleneThreshold)
{
	float MinScale = TNumericLimits<float>::Max();
	float MaxScale = TNumericLimits<float>::Lowest();

	const int32 NumPoints = PointScaleAttribute.Num();

	for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		if (BudHormoneLevels[PointIndex].Ethylene <= EthyleneThreshold)
		{
			const float Scale = PointScaleAttribute[PointIndex];
			MinScale = FMath::Min(MinScale, Scale);
			MaxScale = FMath::Max(MaxScale, Scale);
		}
	}

	TArray<float> NormalizedScales;
	NormalizedScales.Reserve(NumPoints);

	for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		float NormalizedScale = 0.0f;

		if (MaxScale > MinScale)
		{
			NormalizedScale = FMath::GetMappedRangeValueClamped(
				FVector2f(MinScale, MaxScale),
				FVector2f(0.0f, 1.0f),
				FMath::Clamp(PointScaleAttribute[PointIndex], MinScale, MaxScale));
		}

		NormalizedScales.Add(NormalizedScale);
	}

	return NormalizedScales;
}

TArray<TArray<float>> PV::HormoneDistributionHelper::RemapBranchGradientToEthyleneThreshold(
	PV::FBranchPointsAttributeConstView BranchPointsAttribute,
	PV::FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
	PV::FBudHormoneLevelsAttributeConstView BudHormoneLevels,
	float EthyleneThreshold)
{
	const int32 NumBranches = BranchPointsAttribute.Num();
	TArray<TArray<float>> NormalizedBranchGradients;
	NormalizedBranchGradients.SetNum(NumBranches);

	TArray<float> BranchGradientArray;

	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];

		if (BranchPoints.Num() < 2)
		{
			continue;
		}

		BranchGradientArray.Reset(BranchPoints.Num());
		float MinGradient = 1.0f;
		float MaxGradient = 0.0f;

		for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
		{
			const float BranchGradient = AttributesHelper::GetBranchPointBranchGradient(PointLengthFromRootAttribute, BranchPointsAttribute, BranchIndex, BranchPointIndex);
			const float Ethylene = BudHormoneLevels[BranchPoints[BranchPointIndex]].Ethylene;

			if (Ethylene <= EthyleneThreshold)
			{
				MinGradient = FMath::Min(MinGradient, BranchGradient);
				MaxGradient = FMath::Max(MaxGradient, BranchGradient);
			}

			BranchGradientArray.Add(BranchGradient);
		}

		TArray<float>& NormalizedGradients = NormalizedBranchGradients[BranchIndex];
		NormalizedGradients.Reserve(BranchPoints.Num());

		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			float NormalizedGradient = 0.0f;

			if (MaxGradient > MinGradient)
			{
				NormalizedGradient = FMath::GetMappedRangeValueClamped(FVector2f(MinGradient, MaxGradient), FVector2f(0.0f, 1.0f),
					FMath::Clamp(BranchGradientArray[i], MinGradient, MaxGradient));
			}

			NormalizedGradients.Add(NormalizedGradient);
		}
	}

	return NormalizedBranchGradients;
}

void PV::HormoneDistributionHelper::ComputeHormoneBasedAttachmentPoints(
	FManagedArrayCollection& OutCollection,
	const DistributionConditionUtils::FNormalizedPointCaches& NormalizedPointCaches,
	const FHormoneSettings::FDistributionSettings& DistributionSettings,
	const FHormoneSettings::FScaleSettings& ScaleSettings,
	const FHormoneSettings::FAxilSettings& AxilSettings,
	const FHormoneSettings::FPhyllotaxySettings& PhyllotaxySettings,
	const FPVDistributionVectorParams& VectorSettings,
	FRandomStream& RandomStream,
	float TrunkOffset,
	TArray<DistributionHelper::FAttachmentPoint>& OutAttachmentPoints
)
{
	auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(OutCollection);
	auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(OutCollection);
	auto BudDirectionAttribute = PV::FBudDirectionAttribute::GetAttribute(OutCollection);
	auto BudHormoneLevels = PV::FBudHormoneLevelsAttribute::GetAttribute(OutCollection);
	auto PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::GetAttribute(OutCollection);
	auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(OutCollection);
	auto PointPositionAttribute = PV::FPointPositionAttribute::GetAttribute(OutCollection);
	auto BranchChildrenAttribute = FBranchChildrenAttribute::GetAttribute(OutCollection);
	auto BranchNumberAttribute = FBranchNumberAttribute::GetAttribute(OutCollection);
	
	if (!ValidateAttributeCollection(
		BranchPointsAttribute,
		BranchParentNumberAttribute,
		BudDirectionAttribute,
		BudHormoneLevels,
		PointLengthFromRootAttribute,
		PointScaleAttribute,
		PointPositionAttribute,
		BranchChildrenAttribute,
		BranchNumberAttribute
	))
	{
		return;
	}

	auto PointPlantGradientAttribute = FPointPlantGradientAttribute::AddAttribute(OutCollection);
	AttributesHelper::ComputePointPlantGradient(
		{
			PointPlantGradientAttribute,
			BranchPointsAttribute,
			BranchParentNumberAttribute,
			BranchNumberAttribute,
			BranchChildrenAttribute
		});

	const TArray<TArray<float>> NormalizedBranchGradients = RemapBranchGradientToEthyleneThreshold(
		BranchPointsAttribute,
		PointLengthFromRootAttribute,
		BudHormoneLevels,
		DistributionSettings.EthyleneThreshold);

	const TArray<float> NormalizedPlantGradients = RemapPlantGradientToEthyleneThreshold(
		PointPlantGradientAttribute,
		BudHormoneLevels,
		DistributionSettings.EthyleneThreshold);
	
	const int32 NumBranches = BranchPointsAttribute.Num();

	const auto [
			PhyllotaxyFormationLeaf,
			MinBudsLeaf,
			MaxBudsLeaf,
			bResetPhyllotaxyLeaf,
			PhyllotaxyOffsetLeaf] =
		PhyllotaxySettingsHelper::ResolvePhyllotaxy(PhyllotaxySettings);

	int32 FoliageInstanceIndex = 0;
	
	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];

		if (BranchPoints.Num() < 2) continue;

#if DO_ENSURE
		for (int32 i = 0; i < BranchPoints.Num() - 1; ++i)
		{
			ensureMsgf(PointLengthFromRootAttribute[BranchPoints[i]] <= PointLengthFromRootAttribute[BranchPoints[i + 1]],
				TEXT("BranchPoints are not ordered root-to-tip at index %d for BranchIndex %d"), i, BranchIndex);
		}
#endif
		
		const float FirstPointLengthFromRoot = PointLengthFromRootAttribute[BranchPoints[0]];
		const float LastPointLengthFromRoot = PointLengthFromRootAttribute[BranchPoints.Last()];
		const float BranchLength = LastPointLengthFromRoot - FirstPointLengthFromRoot;
		
		if (BranchLength <= KINDA_SMALL_NUMBER) continue;

		// Precompute resolved bud directions for each branch point
		TArray<TArray<FVector3f>> BranchBudDirPerPoint;
		
		BranchBudDirPerPoint.Reserve(BranchPoints.Num());
		
		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			BranchBudDirPerPoint.Add(AttributesHelper::GetBudDirection(
				BudDirectionAttribute, BranchPointsAttribute, BranchParentNumberAttribute,
				PointPositionAttribute, BranchIndex, i));
		}

		// Calculate Normalized Attachment Lookups
		TArray<float> NormalizedAttachmentPoints;
		float Increment = DistributionSettings.InstanceSpacing / BranchLength;
		const float AdjustedMaxPerBranch = DistributionSettings.MaxPerBranch == -1
			? BranchPoints.Num()
			: DistributionSettings.MaxPerBranch;
		float LookUp = 0.0f;

		for (int32 i = 0; i < AdjustedMaxPerBranch; ++i)
		{
			if (LookUp > 1.0f) break;

			float InstanceSpacingRampValue = DistributionSettings.InstanceSpacingRamp
				? DistributionSettings.InstanceSpacingRamp->GetRichCurveConst()->Eval(1.0f - LookUp)
				: Increment;

			// The increment is nudged slightly before blending with the weighted ramp value, the number is chosen based on observation
			float AdjustedIncrement = Increment + (InstanceSpacingRampValue * (Increment * 10.0f));
			AdjustedIncrement = FMath::Lerp(Increment, AdjustedIncrement, DistributionSettings.InstanceSpacingRampEffect);

			NormalizedAttachmentPoints.Add(1.0f - LookUp);
			LookUp += AdjustedIncrement;
		}

		// Compute Position, Angles, Scale and LengthFromRoot for attachment points

		// Compute based on first point attributes
		const FBudDirectionConstView InitialBudDirection(BranchBudDirPerPoint[0]);
		
		const FVector3f InitialApicalDirection(InitialBudDirection.Apical);
		FVector3f InitialAxillaryDirection(InitialBudDirection.Axillary);
		const FVector3f InitialUpVector(InitialBudDirection.UpVector);

		if (bResetPhyllotaxyLeaf)
		{
			InitialAxillaryDirection = FVector3f::CrossProduct(InitialApicalDirection, InitialUpVector);
		}

		if (PhyllotaxyOffsetLeaf > 0.01f)
		{
			const FQuat4f RotationQuat(InitialApicalDirection.GetSafeNormal(), FMath::DegreesToRadians(PhyllotaxyOffsetLeaf));
			InitialAxillaryDirection = RotationQuat.RotateVector(InitialAxillaryDirection);
		}

		int32 AxillaryRotationIteration = 0;

		// Tip to Root
		for (int32 Idx = 0; Idx < NormalizedAttachmentPoints.Num(); ++Idx)
		{
			for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num() - 1; ++BranchPointIndex)
			{
				const int32 CurrentPointIndex = BranchPoints[BranchPointIndex];
				const int32 NextPointIndex = BranchPoints[BranchPointIndex + 1];

				const float CurrentPointLengthFromRoot = PointLengthFromRootAttribute[CurrentPointIndex];
				const float NextPointLengthFromRoot = PointLengthFromRootAttribute[NextPointIndex];

				const float CurrentPointLfrNormalized = FMath::GetMappedRangeValueClamped(
					FVector2f(FirstPointLengthFromRoot, LastPointLengthFromRoot), FVector2f(0.0f, 1.0f), CurrentPointLengthFromRoot);
				const float NextPointLfrNormalized = FMath::GetMappedRangeValueClamped(
					FVector2f(FirstPointLengthFromRoot, LastPointLengthFromRoot), FVector2f(0.0f, 1.0f), NextPointLengthFromRoot);

				if (NormalizedAttachmentPoints[Idx] >= CurrentPointLfrNormalized && NormalizedAttachmentPoints[Idx] <= NextPointLfrNormalized)
				{
					const float DistanceAlpha = (NextPointLfrNormalized - CurrentPointLfrNormalized) < KINDA_SMALL_NUMBER
						? 1.0f
						: (NormalizedAttachmentPoints[Idx] - CurrentPointLfrNormalized) / (NextPointLfrNormalized - CurrentPointLfrNormalized);

					const float EthyleneLevel = FMath::Lerp(BudHormoneLevels[CurrentPointIndex].Ethylene, BudHormoneLevels[NextPointIndex].Ethylene,
						DistanceAlpha);

					if ((EthyleneLevel - 0.001f) >= DistributionSettings.EthyleneThreshold) continue;

					// Shared Properties for this segment
					const FVector3f Position = FMath::Lerp(PointPositionAttribute[CurrentPointIndex], PointPositionAttribute[NextPointIndex],
						DistanceAlpha);
					const float AttachmentPointLfr = FMath::GetMappedRangeValueClamped(
						FVector2f(0.0f, 1.0f), FVector2f(FirstPointLengthFromRoot, LastPointLengthFromRoot), NormalizedAttachmentPoints[Idx]);

					// Scale Calculation
					const float CurrentPointBranchGradient = PV::AttributesHelper::GetBranchPointBranchGradient(PointLengthFromRootAttribute, BranchPointsAttribute, BranchIndex, BranchPointIndex);
					const float NextPointBranchGradient = PV::AttributesHelper::GetBranchPointBranchGradient(PointLengthFromRootAttribute, BranchPointsAttribute, BranchIndex, BranchPointIndex + 1);
					const float BranchGradient = FMath::Clamp(FMath::Lerp(CurrentPointBranchGradient, NextPointBranchGradient, DistanceAlpha), 0.0f, 1.0f);
					float BranchGradientRampValue = ScaleSettings.ScaleRamp
						? ScaleSettings.ScaleRamp->GetRichCurveConst()->Eval(1.0f - BranchGradient)
						: 1.0f;

					const float CurrentScale = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, BranchPointIndex);
					const float NextScale = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, BranchPointIndex + 1);
					const float BranchScale = FMath::Lerp(CurrentScale, NextScale, DistanceAlpha);
					const float BranchScaledValue = BranchGradientRampValue * (BranchScale + 0.1f);
					
					BranchGradientRampValue = FMath::Clamp(FMath::Lerp(BranchGradientRampValue, BranchScaledValue, ScaleSettings.BranchScaleImpact),
						0.0f, 1.0f);

					float Scale = FMath::GetMappedRangeValueClamped(FVector2f(0.0f, 1.0f), FVector2f(ScaleSettings.MinScale, ScaleSettings.MaxScale),
						BranchGradientRampValue) * ScaleSettings.BaseScale;
					Scale *= RandomStream.FRandRange(ScaleSettings.RandomScaleMin, ScaleSettings.RandomScaleMax);
					
					// Plant gradient
					const float CurrentPlantGradient = 1 - PointPlantGradientAttribute[CurrentPointIndex];
					const float NextPlantGradient = 1 - PointPlantGradientAttribute[NextPointIndex];
					const float PlantGradient = FMath::Lerp(CurrentPlantGradient, NextPlantGradient, DistanceAlpha);

					// Normalized gradients
					const TArray<float>& BranchNormalizedArray = NormalizedBranchGradients[BranchIndex];
					const float CurrentNormalizedBranchGradient = BranchNormalizedArray.IsValidIndex(BranchPointIndex)     ? BranchNormalizedArray[BranchPointIndex]     : 0.0f;
					const float NextNormalizedBranchGradient    = BranchNormalizedArray.IsValidIndex(BranchPointIndex + 1) ? BranchNormalizedArray[BranchPointIndex + 1] : 0.0f;
					const float NormalizedBranchGradient = FMath::Lerp(CurrentNormalizedBranchGradient, NextNormalizedBranchGradient, DistanceAlpha);

					const float NormalizedPlantGradient = FMath::Lerp(NormalizedPlantGradients[CurrentPointIndex], NormalizedPlantGradients[NextPointIndex], DistanceAlpha);
					
					// Conditions
					const FBudDirectionConstView CurrentBudDir(BranchBudDirPerPoint[BranchPointIndex]);
					const FBudDirectionConstView NextBudDir(BranchBudDirPerPoint[BranchPointIndex + 1]);
					const FVector3f BudApicalDirectionVector = FMath::Lerp(CurrentBudDir.Apical, NextBudDir.Apical, DistanceAlpha);
					
					const FVector3f BudUpVector = FMath::Lerp(CurrentBudDir.UpVector, NextBudDir.UpVector, DistanceAlpha);
					
					const bool bIsTip = (BranchPointIndex == BranchPoints.Num() - 2) && FMath::IsNearlyEqual(DistanceAlpha, 1.0f, 1e-3f);

					DistributionConditionUtils::ConditionPicker::FPointConditionSample ConditionSample;
					ConditionSample.UpAlignment = DistributionConditionUtils::UpAlignmentNormalized(BudApicalDirectionVector);
					ConditionSample.Light      = FMath::Lerp(NormalizedPointCaches.Light[CurrentPointIndex],       NormalizedPointCaches.Light[NextPointIndex],       DistanceAlpha);
					ConditionSample.Health     = FMath::Lerp(NormalizedPointCaches.Health[CurrentPointIndex],      NormalizedPointCaches.Health[NextPointIndex],      DistanceAlpha);
					ConditionSample.Scale      = FMath::Lerp(NormalizedPointCaches.PScales[CurrentPointIndex],     NormalizedPointCaches.PScales[NextPointIndex],     DistanceAlpha);
					ConditionSample.Height     = FMath::Lerp(NormalizedPointCaches.Height[CurrentPointIndex],      NormalizedPointCaches.Height[NextPointIndex],      DistanceAlpha);
					ConditionSample.Generation = FMath::Lerp(NormalizedPointCaches.Generation[CurrentPointIndex],  NormalizedPointCaches.Generation[NextPointIndex],  DistanceAlpha);
					ConditionSample.Tip = DistributionConditionUtils::Tip(bIsTip);

					// Tip detection logic shared by Grafter
					bool bIsTipInstance = Idx == 0 && FMath::IsNearlyEqual(AttachmentPointLfr, LastPointLengthFromRoot, 0.001f);
					int32 TipPointIndex = bIsTipInstance
						? NextPointIndex
						: INDEX_NONE;

					// Determine Rotation / Multi-Buds
					int32 NumBuds = 1;
					TArray<FVector3f> ComputedUpVectors;
					TArray<FVector3f> ComputedNormalVectors;

					const FVector3f BudLightOptimalVector = FMath::Lerp(CurrentBudDir.LightOptimal, NextBudDir.LightOptimal, DistanceAlpha);
					const FVector3f BudLightSubOptimalVector = FMath::Lerp(CurrentBudDir.LightSubOptimal, NextBudDir.LightSubOptimal, DistanceAlpha);
					
					const FQuat4f QuatRotation = FQuat4f::FindBetweenNormals(BudApicalDirectionVector.GetSafeNormal(),
						BudLightOptimalVector.GetSafeNormal());
					const FVector3f BaseNormalVector = QuatRotation.RotateVector(BudApicalDirectionVector);

					FVector3f UpVector = BudApicalDirectionVector;
					FVector3f NormalVector = BaseNormalVector;
					
					float AxillaryRotationDegree = FMath::DegreesToRadians(PhyllotaxyFormationLeaf * AxillaryRotationIteration);
					const FQuat4f RotationQuat(InitialApicalDirection.GetSafeNormal(), AxillaryRotationDegree);
					FVector3f NewAxillaryDirection = RotationQuat.RotateVector(InitialAxillaryDirection.GetSafeNormal());
					
					FQuat4f ApicalCorrection = FQuat4f::FindBetweenNormals(InitialApicalDirection.GetSafeNormal(),
						BudApicalDirectionVector.GetSafeNormal());
					
					NewAxillaryDirection = ApicalCorrection.RotateVector(NewAxillaryDirection).GetSafeNormal();

					NumBuds = bIsTipInstance && PhyllotaxySettings.bSingleBudTip ? 1 : RandomStream.RandRange(MinBudsLeaf, MaxBudsLeaf);
					
					float RotationPerBud = 360.0f / static_cast<float>(NumBuds);

					for (int32 i = 0; i < NumBuds; ++i)
					{
						float RotationAngle = FMath::DegreesToRadians(RotationPerBud * i);
						const FQuat4f AxillaryRotation(InitialApicalDirection.GetSafeNormal(), RotationAngle);
						
						// Reset vectors for each bud.
						UpVector = AxillaryRotation.RotateVector(NewAxillaryDirection.GetSafeNormal());
						NormalVector = BaseNormalVector;

						if (AxilSettings.OverrideAxilAngle)
						{
							float AxilAngleRampValue = AxilSettings.AxilAngleRamp
								? FMath::Clamp(AxilSettings.AxilAngleRamp->GetRichCurveConst()->Eval(1.0f - BranchGradient), 0.0f, 1.0f)
								: 1.0f;
							
							AxilAngleRampValue *= AxilSettings.AxilAngleRampUpperValue;
							
							const float AxilAngleBlended = FMath::Lerp(AxilSettings.AxilAngle, AxilAngleRampValue,
								AxilSettings.AxilAngleRampEffect);

							const FVector3f RotationAxis = FVector3f::CrossProduct(BudApicalDirectionVector.GetSafeNormal(),
								UpVector.GetSafeNormal());
							
							const FQuat4f Rotation(RotationAxis.GetSafeNormal(), FMath::DegreesToRadians(AxilAngleBlended));
							UpVector = Rotation.RotateVector(BudApicalDirectionVector.GetSafeNormal()); 
						}
						
						// Call UpdateVectors here 
						DistributionVectorUtils::FPointVectorData PointVectorData;
						PointVectorData.ApicalDirection = BudApicalDirectionVector;
						PointVectorData.UpVector = BudUpVector;
						PointVectorData.bIsTip = bIsTipInstance;
						PointVectorData.LightOptimal = BudLightOptimalVector;
						PointVectorData.LightSubOptimal = BudLightSubOptimalVector;
						PointVectorData.BranchGradient = BranchGradient;
						PointVectorData.BranchGradientNormalized = NormalizedBranchGradient;
						PointVectorData.PlantGradient = PlantGradient;
						PointVectorData.PlantGradientNormalized = NormalizedPlantGradient;
						
						DistributionVectorUtils::ApplyVectorSettings(VectorSettings, PointVectorData, FoliageInstanceIndex, UpVector, NormalVector);
						
						ComputedUpVectors.Add(UpVector);
						ComputedNormalVectors.Add(NormalVector);
						
						++FoliageInstanceIndex;
					}
					AxillaryRotationIteration++;

					// Create FAttachmentPoint for each generated bud
					for (int32 BudIdx = 0; BudIdx < NumBuds; ++BudIdx)
					{
						const FVector3f Dir = (PointPositionAttribute[NextPointIndex] - PointPositionAttribute[CurrentPointIndex]).GetSafeNormal();
						const FVector3f CrossVector = FVector3f::CrossProduct(ComputedUpVectors[BudIdx], Dir);
						const FVector3f UpVec = FVector3f::CrossProduct(Dir, CrossVector);
						const float OffsetBias = FMath::Lerp(PointScaleAttribute[CurrentPointIndex], PointScaleAttribute[NextPointIndex], DistanceAlpha) * TrunkOffset;
						const FVector3f FoliagePosition = Position + UpVec.GetSafeNormal() * OffsetBias;
						
						DistributionHelper::FAttachmentPoint Point;
						Point.BranchIndex = BranchIndex;
						Point.PointIndexA = CurrentPointIndex;
						Point.PointIndexB = NextPointIndex;
						Point.PointAlpha = DistanceAlpha;
						Point.Position = FoliagePosition;
						Point.UpDirection = ComputedUpVectors[BudIdx];
						Point.NormalDirection = ComputedNormalVectors[BudIdx];
						Point.Scale = Scale;
						Point.LengthFromRoot = AttachmentPointLfr;
						Point.bIsTipInstance = bIsTipInstance;
						Point.TipPointIndex = TipPointIndex;
						Point.ConditionSample = ConditionSample;

						OutAttachmentPoints.Add(Point);
					}
				}
			}
		}
	}
}
