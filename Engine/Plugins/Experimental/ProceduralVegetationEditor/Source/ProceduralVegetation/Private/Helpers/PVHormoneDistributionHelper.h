// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVDistributionHelper.h"
#include "PVPhyllotaxyHelper.h"
#include "Utils/PVAttributes.h"
#include "Utils/PVFloatRamp.h"

namespace PV::HormoneDistributionHelper
{
	namespace FHormoneSettings
	{
		struct FDistributionSettings
		{
			const float EthyleneThreshold;
			const float InstanceSpacing;
			const FPVFloatRamp* InstanceSpacingRamp;
			const float InstanceSpacingRampEffect;
			const int32 MaxPerBranch;
		};

		struct FScaleSettings
		{
			const float BaseScale;
			const float BranchScaleImpact;
			const float MinScale;
			const float MaxScale;
			const float RandomScaleMin;
			const float RandomScaleMax;
			const FPVFloatRamp* ScaleRamp;
		};

		struct FAxilSettings
		{
			const bool OverrideAxilAngle;
			const float AxilAngle;
			const FPVFloatRamp* AxilAngleRamp;
			const float AxilAngleRampUpperValue;
			const float AxilAngleRampEffect;
		};

		struct FPhyllotaxySettings
		{
			const EPhyllotaxyType PhyllotaxyType;
			const EPhyllotaxyFormation PhyllotaxyFormation;
			const int32 MinimumNodeBuds;
			const int32 MaximumNodeBuds;
			const bool bSingleBudTip;
			const float PhyllotaxyAdditionalAngle;
			const bool ResetPhyllotaxy;
			const float PhyllotaxyOffset;
		};
	}
	
	TArray<float> RemapPlantGradientToEthyleneThreshold(
		PV::FPointPlantGradientAttributeConstView PointPlantGradient,
		PV::FBudHormoneLevelsAttributeConstView BudHormoneLevels,
		float EthyleneThreshold);

	TArray<float> RemapPScalesToEthyleneThreshold(
		PV::FPointScaleAttributeConstView PointScaleAttribute,
		PV::FBudHormoneLevelsAttributeConstView BudHormoneLevels,
		float EthyleneThreshold);

	TArray<TArray<float>> RemapBranchGradientToEthyleneThreshold(
		PV::FBranchPointsAttributeConstView BranchPoints,
		PV::FPointLengthFromRootAttributeConstView PointLengthFromRoot,
		PV::FBudHormoneLevelsAttributeConstView BudHormoneLevels,
		float EthyleneThreshold);

	void ComputeHormoneBasedAttachmentPoints(
		FManagedArrayCollection& OutCollection,
		const DistributionConditionUtils::FNormalizedPointCaches& NormalizedPointCaches,
		const FHormoneSettings::FDistributionSettings& DistributionSettings,
		const FHormoneSettings::FScaleSettings& ScaleSettings,
		const FHormoneSettings::FAxilSettings& AxilSettings,
		const FHormoneSettings::FPhyllotaxySettings& PhyllotaxySettings,
		const FPVDistributionVectorParams& VectorSettings,
		FRandomStream& RandomStream,
		float TrunkOffset,
		TArray<DistributionHelper::FAttachmentPoint>& OutAttachmentPoints);
}