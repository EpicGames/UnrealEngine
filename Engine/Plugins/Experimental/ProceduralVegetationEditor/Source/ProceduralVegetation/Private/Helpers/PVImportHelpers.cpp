// Copyright Epic Games, Inc. All Rights Reserved.
#include "Helpers/PVImportHelpers.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "GeometryCollectionToDynamicMesh.h"
#include "GeometryCollection/GeometryCollection.h"

#include "Helpers/PVAttributesHelper.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "Helpers/PVDynamicMeshHelpers.h"
#include "Helpers/PVUtilities.h"

#include "Engine/StaticMesh.h"

namespace PV::ImportHelper
{
	const static TSet<FName> GeometryGroups = {
		FGeometryCollection::VerticesGroup,
		FGeometryCollection::FacesGroup,
		FGeometryCollection::GeometryGroup,
		FGeometryCollection::BreakingGroup,
		FGeometryCollection::MaterialGroup,
		FTransformCollection::TransformGroup,
		FTransformCollection::ConvexGroup
	};

	const static FName MaterialPathAttributeName("MaterialPath");

	static void SetManagedArrayCollectionGeometry(const FGeometryCollection& InGeometryCollection, FManagedArrayCollection& InOutManagedArrayCollection)
	{
		for (const FName GroupName : GeometryGroups)
		{
			InOutManagedArrayCollection.RemoveGroup(GroupName);
		}

		InGeometryCollection.CopyTo(&InOutManagedArrayCollection);
	}

	static void AppendGeometryFromManagedArrayCollection(FGeometryCollection& InOutGeometryCollection, const FManagedArrayCollection& InManagedArrayCollection)
	{
		TArray<FName> GroupsToSkip;
		GroupsToSkip.Reserve(InManagedArrayCollection.GroupNames().Num());

		for (FName GroupName : InManagedArrayCollection.GroupNames())
		{
			if (!GeometryGroups.Contains(GroupName))
			{
				GroupsToSkip.Add(GroupName);
			}
		}

		FGeometryCollection GeometryCollection;
		InManagedArrayCollection.CopyTo(&GeometryCollection, GroupsToSkip);

		const auto* ExistingMaterialPathsAttribute = InOutGeometryCollection.FindAttribute<FString>(MaterialPathAttributeName, FGeometryCollection::MaterialGroup);
		const int32 NumExistingMaterials = ExistingMaterialPathsAttribute ? ExistingMaterialPathsAttribute->Num() : 1;
		const bool bHasCopiedGeometry = InOutGeometryCollection.AppendGeometry(GeometryCollection, NumExistingMaterials, false) != INDEX_NONE;

		// Copy materials. Ideally we'd want to re-use materials where possible, but given this is for debugging we just assign 
		// one material per mesh.
		if (bHasCopiedGeometry)
		{
			if (!InOutGeometryCollection.HasGroup(FGeometryCollection::MaterialGroup))
			{
				InOutGeometryCollection.AddGroup(FGeometryCollection::MaterialGroup);
			}

			auto* MaterialPathsAttributeToCopy = GeometryCollection.FindAttribute<FString>(MaterialPathAttributeName, FGeometryCollection::MaterialGroup);
			const int32 NumMaterialsToCopy = MaterialPathsAttributeToCopy ? MaterialPathsAttributeToCopy->Num() : 1;
			InOutGeometryCollection.Resize(NumExistingMaterials + NumMaterialsToCopy, FGeometryCollection::MaterialGroup);

			if (MaterialPathsAttributeToCopy)
			{
				auto* MaterialPathsAttribute = InOutGeometryCollection.FindOrAddAttributeTyped<FString>(MaterialPathAttributeName, FGeometryCollection::MaterialGroup);
				for (int32 i = NumExistingMaterials; i < NumExistingMaterials + NumMaterialsToCopy; ++i)
				{
					const int32 MaterialIndexToCopy = i - NumExistingMaterials;
					if (MaterialPathsAttributeToCopy->IsValidIndex(MaterialIndexToCopy))
					{
						(*MaterialPathsAttribute)[i] = (*MaterialPathsAttributeToCopy)[MaterialIndexToCopy];
					}
				}
			}
		}
	}
};

void PV::ImportHelper::ComputeLengthFromRoot(
	const UE::Geometry::FDynamicMesh3& DynamicMesh,
	const FBooleanVertexAttribute& RootPointsAttribute,
	const FBooleanVertexAttribute& EndPointsAttribute,
	FFloatVertexAttribute& LengthFromRootAttribute,
	const TFunction<bool(int32)>& VertexFilterPredicate
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeLengthFromRoot);

	// This algorithm starts at the root point and walks along every vertex, keeping track of its distance to the root point and stores that distance
	// in the LengthFromRootAttribute.

	for (int32 VertexID : DynamicMesh.VertexIndicesItr())
	{
		LengthFromRootAttribute.SetValue(VertexID, -1);
	}

	TArray<int32> SearchQueue;
	SearchQueue.Reserve(DynamicMesh.VertexCount());
	for (int32 VertexID : RootPointsAttribute.FindAllNonZero())
	{
		SearchQueue.Add(VertexID);
		LengthFromRootAttribute.SetValue(VertexID, 0);
	}

	while (SearchQueue.Num() > 0)
	{
		const int32& VertexID = SearchQueue.Pop(EAllowShrinking::No);

		const float PrevLengthFromRoot = LengthFromRootAttribute.GetValue(VertexID);
		const FVector PrevVertexPosition = DynamicMesh.GetVertex(VertexID);

		const PV::DynamicMeshHelper::FFindMeshElemsResult Neighbours = PV::DynamicMeshHelper::GetVertexNeighbours(DynamicMesh, VertexID);
		for (int32 NeighbourVertexID : Neighbours)
		{
			if (!EndPointsAttribute.GetValue(NeighbourVertexID) // Ingore vertex filter for end points as that means we've reached the end of the branch
				&& !VertexFilterPredicate(NeighbourVertexID))
			{
				continue;
			}

			const FVector VertexPosition = DynamicMesh.GetVertex(NeighbourVertexID);
			const float NewLengthFromRoot = PrevLengthFromRoot + (VertexPosition - PrevVertexPosition).Size();
			const float ExistingLengthFromRoot = LengthFromRootAttribute.GetValue(NeighbourVertexID);

			if (ExistingLengthFromRoot >= 0 
				&& ExistingLengthFromRoot <= NewLengthFromRoot)
			{
				// If we find a shorter path then we should ideally remove vertices from the queue
				// which are computing the distance using old values. 
				// The current algorithm will still produce correct results, but may compute a large number
				// of unecessary distances which will later be over-written when a closer path is found.
				continue;
			}

			LengthFromRootAttribute.SetValue(NeighbourVertexID, NewLengthFromRoot);

			SearchQueue.Insert(NeighbourVertexID, 0);
		}
	}
}

void PV::ImportHelper::TraceBranches(
	const UE::Geometry::FDynamicMesh3& DynamicMesh,
	const FFloatVertexAttribute& LengthFromRootAttribute,
	const FBooleanVertexAttribute& RootPointAttribute,
	const FBooleanVertexAttribute& EndPointAttribute,
	FIntVertexAttribute& NextBranchPointAttribute,
	const TFunction<bool(int32)>& VertexFilterPredicate
)
{
	using namespace PV::DynamicMeshHelper;

	TRACE_CPUPROFILER_EVENT_SCOPE(TraceBranches);

	for (int32 VertexID : DynamicMesh.VertexIndicesItr())
	{
		NextBranchPointAttribute.SetValue(VertexID, INDEX_NONE);
	}

	const auto FindNextPoint = [&](int32 PointIndex, float MaxLengthFromRoot)->int32
	{
		int32 OutNextPointIndex = INDEX_NONE;
		float ClosestLengthFromRoot = MAX_flt;
	
		const FVector ThisVertexPosition = DynamicMesh.GetVertex(PointIndex);
		const float ThisLengthFromRoot = LengthFromRootAttribute.GetValue(PointIndex);
		const PV::DynamicMeshHelper::FFindMeshElemsResult Neighbours = GetVertexNeighbours(DynamicMesh, PointIndex);
		for (int32 Neighbour : Neighbours)
		{
			if (RootPointAttribute.GetValue(Neighbour))
			{
				OutNextPointIndex = Neighbour;
				break;
			}

			if (!VertexFilterPredicate(Neighbour))
			{
				continue;
			}

			const float NeighbourLengthFromRoot = LengthFromRootAttribute.GetValue(Neighbour);
			check(NeighbourLengthFromRoot >= 0);
			if (NeighbourLengthFromRoot >= MaxLengthFromRoot)
			{
				continue;
			}
		
			const FVector NeighbourVertexPosition = DynamicMesh.GetVertex(Neighbour);
			const float DistToNeighbour = (NeighbourVertexPosition - ThisVertexPosition).Size();
			const float PathLengthFromRoot = NeighbourLengthFromRoot + DistToNeighbour;
			if (PathLengthFromRoot >= ClosestLengthFromRoot)
			{
				continue;
			}
		
			ClosestLengthFromRoot = PathLengthFromRoot;
			OutNextPointIndex = Neighbour;
		}
		return OutNextPointIndex;
	};

	const TArray<int32> EndPoints = EndPointAttribute.FindAllNonZero();

	TArray<bool> bHasProcessedVertex;

	for (int32 i = 0; i < EndPoints.Num(); i++)
	{
		int32 CurrentPointIndex = EndPoints[i];

		float MaxLengthFromRoot = LengthFromRootAttribute.GetValue(CurrentPointIndex);
		if (MaxLengthFromRoot < 0)
		{
			continue; // This end point was unable to trace down to a root point, perhaps log a warning?
		}

		bHasProcessedVertex.Reset();
		bHasProcessedVertex.SetNumZeroed(DynamicMesh.MaxVertexID());

		while (true)
		{
			const int32 NextPointIndex = FindNextPoint(CurrentPointIndex, MaxLengthFromRoot);
			if (NextPointIndex == INDEX_NONE)
			{
				break;
			}
				
			NextBranchPointAttribute.SetValue(CurrentPointIndex, NextPointIndex);

			if (NextBranchPointAttribute.GetValue(NextPointIndex) != INDEX_NONE
				|| RootPointAttribute.GetValue(NextPointIndex))
			{
				break;
			}

			if (!ensureMsgf(!bHasProcessedVertex[NextPointIndex], TEXT("Infinite loop detected")))
			{
				break;
			}

			bHasProcessedVertex[NextPointIndex] = true;

			CurrentPointIndex = NextPointIndex;
			MaxLengthFromRoot = LengthFromRootAttribute.GetValue(CurrentPointIndex);
		}
	}
}

void PV::ImportHelper::ExtractBranchHierarchies(
	const UE::Geometry::FDynamicMesh3& DynamicMesh,
	const FBooleanVertexAttribute& RootPointAttribute,
	const FBooleanVertexAttribute& EndPointAttribute,
	const FIntVertexAttribute& NextBranchPointAttribute,
	const FFloatVertexAttribute& VertexScaleAttribute,
	const FFloatVertexAttribute& LengthFromRootAttribute,
	TArray<FMeshBranchHierarchy>& OutMeshBranchHierarchies
)
{
	using namespace PV::DynamicMeshHelper;

	TRACE_CPUPROFILER_EVENT_SCOPE(ExtractBranchHierarchies);

	TArray<bool> bProcessedVertices;
	bProcessedVertices.SetNumZeroed(DynamicMesh.MaxVertexID());

	TArray<float> MaxSubtreeLengthFromRoot;
	{
		MaxSubtreeLengthFromRoot.SetNumZeroed(DynamicMesh.MaxVertexID());

		const TArray<int32> EndPoints = EndPointAttribute.FindAllNonZero();
		for (int32 EndVertexID : EndPoints)
		{
			const float EndpointLength = LengthFromRootAttribute.GetValue(EndVertexID);
			if (EndpointLength <= 0.f)
			{
				continue; // endpoint not reachable from root, skip
			}
			int32 Current = EndVertexID;
			while (Current != INDEX_NONE)
			{
				if (EndpointLength <= MaxSubtreeLengthFromRoot[Current])
				{
					break;
				}
				MaxSubtreeLengthFromRoot[Current] = EndpointLength;
				Current = NextBranchPointAttribute.GetValue(Current);
			}
		}
	}
		
	const auto FindBestParentPoint = [&](int32 PrevVertexID, int32 VertexID, const TArray<int32>& NextPoints) -> int32
	{
		if (NextPoints.Num() == 1)
		{
			return NextPoints[0];
		}
		check(NextPoints.Num() > 0);

		const FVector3d VertexPos = DynamicMesh.GetVertex(VertexID);
		const FVector3d PrevVertexPos = DynamicMesh.GetVertex(PrevVertexID);
		const FVector3d ApicalDirection = (VertexPos - PrevVertexPos).GetSafeNormal();
		const float CurrentRadius = VertexScaleAttribute.GetValue(VertexID);

		TArray<float, TInlineAllocator<4>> Dots;
		TArray<float, TInlineAllocator<4>> Lengths;
		Dots.SetNumUninitialized(NextPoints.Num());
		Lengths.SetNumUninitialized(NextPoints.Num());

		for (int32 i = 0; i < NextPoints.Num(); ++i)
		{
			const int32 NextVertexID = NextPoints[i];
			const FVector3d NextPos = DynamicMesh.GetVertex(NextVertexID);
			const FVector3d NextDir = (NextPos - VertexPos).GetSafeNormal();
			Dots[i] = (1.0 + FVector3d::DotProduct(ApicalDirection, NextDir)) / 2.0;
			Lengths[i] = MaxSubtreeLengthFromRoot[NextVertexID] - LengthFromRootAttribute.GetValue(NextVertexID);
		}

		// Normalize a raw float array (all values >= 0) by dividing by the sum.
		// This preserves proportionality: a value of 10000 vs 1 stays near 1.0 vs 0.0001,
		// whereas a value of 10 vs 9 stays near 0.526 vs 0.473.
		// Min-max normalization collapses both cases to {1, 0}, losing magnitude information.
		// bInvert: output 1-normalized instead of normalized
		const auto NormalizeArray = [](TArray<float, TInlineAllocator<4>>& InOutArray, bool bInvert)
		{
			float Sum = 0.f;
			for (int32 i = 0; i < InOutArray.Num(); ++i)
			{
				Sum += InOutArray[i];
			}

			if (Sum < UE_SMALL_NUMBER)
			{
				for (int32 i = 0; i < InOutArray.Num(); ++i)
				{
					InOutArray[i] = 0.f;
				}
			}
			else
			{
				for (int32 i = 0; i < InOutArray.Num(); ++i)
				{
					InOutArray[i] = InOutArray[i] / Sum;
				}

				if (bInvert)
				{
					for (int32 i = 0; i < InOutArray.Num(); ++i)
					{
						InOutArray[i] = 1.f - InOutArray[i];
					}
				}
			}
		};

		NormalizeArray(Lengths, false);

		float BestScore = -MAX_flt;
		int32 BestIndex = INDEX_NONE;
		for (int32 i = 0; i < NextPoints.Num(); ++i)
		{
			const float Score = Dots[i] + Lengths[i];
			if (Score > BestScore)
			{
				BestScore = Score;
				BestIndex = NextPoints[i];
			}
		}

		check(BestIndex != INDEX_NONE);
		return BestIndex;
	};

	const auto FindNextBranchPoints = [&](int32 VertexID)
	{
		TArray<int32> NextBranchPoints;
		const PV::DynamicMeshHelper::FFindMeshElemsResult Neighbours = GetVertexNeighbours(DynamicMesh, VertexID);

		for (int32 Neighbour : Neighbours)
		{
			if (NextBranchPointAttribute.GetValue(Neighbour) == VertexID
				&& !bProcessedVertices[Neighbour])
			{
				NextBranchPoints.Add(Neighbour);
			}
		}
		return NextBranchPoints;
	};

	const TFunction<void(int32, TArray<FMeshBranchHierarchy::FBranch>&)> TraceBranchesRecursive = [&](
		int32 VertexID, 
		TArray<FMeshBranchHierarchy::FBranch>& OutBranches
	)
	{
		const int32 BranchIndex = OutBranches.Num() - 1;
		OutBranches[BranchIndex].BranchPoints.Add(VertexID);
			
		bProcessedVertices[VertexID] = true;

		if (EndPointAttribute.GetValue(VertexID))
		{
			return;
		}

		const TArray<int32> NextBranchPoints = FindNextBranchPoints(VertexID);
		check(NextBranchPoints.Num() > 0);

		for (int32 NextVertexID : NextBranchPoints)
		{
			bProcessedVertices[NextVertexID] = true;
		}

		const int32 PrevVertexID = OutBranches[BranchIndex].BranchPoints.Last(1);
		const int32 NextBranchPointsIndex = FindBestParentPoint(PrevVertexID, VertexID, NextBranchPoints);
			
		TraceBranchesRecursive(NextBranchPointsIndex, OutBranches);

		for (int32 NextVertexID : NextBranchPoints)
		{
			if (NextVertexID != NextBranchPointsIndex)
			{
				FMeshBranchHierarchy::FBranch& NextBranch = OutBranches.AddDefaulted_GetRef();
				NextBranch.BranchPoints.Add(VertexID);
				NextBranch.ParentBranch = BranchIndex;

				TraceBranchesRecursive(NextVertexID, OutBranches);
			}
		}
	};

	const TArray<int32> RootPoints = RootPointAttribute.FindAllNonZero();
	OutMeshBranchHierarchies.Reserve(RootPoints.Num());

	for (const int32 RootPointIndex : RootPoints)
	{
		TArray<TArray<FMeshBranchHierarchy::FBranch>> BranchHierarchies;

		const TArray<int32> NextBranchPoints = FindNextBranchPoints(RootPointIndex);
		for (int32 NextVertexID : NextBranchPoints)
		{
			bProcessedVertices[NextVertexID] = true;
		}

		for (int32 NextVertexID : NextBranchPoints)
		{
			TArray<FMeshBranchHierarchy::FBranch>& Branches = BranchHierarchies.AddDefaulted_GetRef();
			FMeshBranchHierarchy::FBranch& RootBranch = Branches.AddDefaulted_GetRef();
			RootBranch.BranchPoints.Add(RootPointIndex);
			TraceBranchesRecursive(NextVertexID, Branches);
		}

		if (BranchHierarchies.Num() > 0)
		{
			// TODO: Handle multiple branches starting from root
			TArray<FMeshBranchHierarchy::FBranch>* BestBranches = Algo::MaxElement(
				BranchHierarchies, 
				[](const auto& A, const auto& B) { return A.Num() < B.Num(); }
			);
			OutMeshBranchHierarchies.Emplace(MoveTemp(*BestBranches));
		}
	}
}

const PV::TDynamicMeshVertexAttributeDefinition<float> PV::ImportHelper::VertexScaleAttributeDefinition(TEXT("VertexScale"));
const PV::TDynamicMeshVertexAttributeDefinition<bool> PV::ImportHelper::EndPointAttributeDefinition(TEXT("EndPoint"));
const PV::TDynamicMeshVertexAttributeDefinition<bool> PV::ImportHelper::RootPointAttributeDefinition(TEXT("RootPoint"));
const PV::TDynamicMeshVertexAttributeDefinition<float> PV::ImportHelper::LengthFromRootAttributeDefinition(TEXT("LengthFromRoot"));
const PV::TDynamicMeshVertexAttributeDefinition<int32> PV::ImportHelper::NextBranchPointAttributeDefinition(TEXT("NextBranchPoint"));

void PV::ImportHelper::AddDynamicMeshToCollection(
	const UE::Geometry::FDynamicMesh3& DynamicMesh, 
	FManagedArrayCollection& InOutCollection, 
	const FTransform& Offset
)
{
	FGeometryCollection MeshGeomCollection;

	UE::Geometry::FGeometryCollectionToDynamicMeshes::FToCollectionOptions Options2;
	UE::Geometry::FGeometryCollectionToDynamicMeshes::AppendMeshToCollection(MeshGeomCollection, DynamicMesh, Offset.Inverse(), Options2);

	AppendGeometryFromManagedArrayCollection(MeshGeomCollection, InOutCollection);
	SetManagedArrayCollectionGeometry(MeshGeomCollection, InOutCollection);
}

void PV::ImportHelper::AddDynamicMeshVerticesToCollection(
	const UE::Geometry::FDynamicMesh3& DynamicMesh,
	const FBooleanVertexAttribute& Filter,
	const FTransform& VertexOffset,
	FManagedArrayCollection& InOutCollection,
	const FName TargetGroupName,
	const FName TargetAttributeName
)
{
	const TArray<int32> VertexIndices = Filter.FindAllNonZero();

	if (!ensure(!InOutCollection.HasAttribute(TargetAttributeName, TargetGroupName)))
	{
		return;
	}

	if (!InOutCollection.HasGroup(TargetGroupName))
	{
		InOutCollection.AddGroup(TargetGroupName);
	}
	
	InOutCollection.AddElements(VertexIndices.Num(), TargetGroupName);

	auto& Attribute = InOutCollection.AddAttribute<FVector3f>(TargetAttributeName, TargetGroupName);
	for (int32 i = 0; i < VertexIndices.Num(); ++i)
	{
		Attribute[i] = FVector3f(VertexOffset.TransformPosition(DynamicMesh.GetVertex(VertexIndices[i])));
	}
}

void PV::ImportHelper::AddBranchHierarchiesToCollection(
	const TArray<FPVBranchHierarchyDescription>& BranchHierarchyDescriptions,
	FManagedArrayCollection& InOutCollection,
	const FVector& Offset,
	const FName TargetAttributeName,
	const FName TargetGroupName
)
{
	InOutCollection.AddGroup(TargetGroupName);
	int32 AttributeIndex = 0;
	for (const FPVBranchHierarchyDescription& BranchHierarchy : BranchHierarchyDescriptions)
	{
		InOutCollection.AddElements(BranchHierarchy.Branches.Num(), TargetGroupName);

		TManagedArray<TArray<FVector3f>>& Attribute = *InOutCollection.FindOrAddAttributeTyped<TArray<FVector3f>>(TargetAttributeName, TargetGroupName);
		for (int32 i = 0; i < BranchHierarchy.Branches.Num(); ++i)
		{
			const FPVBranchDescription& Branch = BranchHierarchy.Branches[i];
			Attribute[AttributeIndex].Reserve(Branch.PointIndices.Num());
			for (int32 VertexIndex : Branch.PointIndices)
			{
				Attribute[AttributeIndex].Add(BranchHierarchy.Points[VertexIndex] + FVector3f(Offset));
			}
			AttributeIndex++;
		}
	}
}

void PV::ImportHelper::AddLabelsToCollection(const TArray<FLabel>& Labels, FManagedArrayCollection& InOutCollection)
{
	if (!InOutCollection.HasGroup(PVImportNames::LabelsGroup))
	{
		InOutCollection.AddGroup(PVImportNames::LabelsGroup);
	}

	if (!InOutCollection.HasAttribute(PVImportNames::LabelTextAttribute, PVImportNames::LabelsGroup))
	{
		InOutCollection.AddAttribute<FString>(PVImportNames::LabelTextAttribute, PVImportNames::LabelsGroup);
	}

	if (!InOutCollection.HasAttribute(PVImportNames::LabelPositionAttribute, PVImportNames::LabelsGroup))
	{
		InOutCollection.AddAttribute<FVector3f>(PVImportNames::LabelPositionAttribute, PVImportNames::LabelsGroup);
	}

	if (!InOutCollection.HasAttribute(PVImportNames::LabelScaleAttribute, PVImportNames::LabelsGroup))
	{
		InOutCollection.AddAttribute<float>(PVImportNames::LabelScaleAttribute, PVImportNames::LabelsGroup);
	}

	if (!ensure(InOutCollection.GetAttributeType(PVImportNames::LabelTextAttribute, PVImportNames::LabelsGroup) == FManagedArrayCollection::EArrayType::FStringType))
	{
		return;
	}

	if (!ensure(InOutCollection.GetAttributeType(PVImportNames::LabelPositionAttribute, PVImportNames::LabelsGroup) == FManagedArrayCollection::EArrayType::FVectorType))
	{
		return;
	}

	if (!ensure(InOutCollection.GetAttributeType(PVImportNames::LabelScaleAttribute, PVImportNames::LabelsGroup) == FManagedArrayCollection::EArrayType::FFloatType))
	{
		return;
	}

	const int32 StartIndex = InOutCollection.NumElements(PVImportNames::LabelsGroup);

	InOutCollection.AddElements(Labels.Num(), PVImportNames::LabelsGroup);

	TManagedArray<FString>& LabelTextAttribute = *InOutCollection.FindAttribute<FString>(PVImportNames::LabelTextAttribute, PVImportNames::LabelsGroup);
	TManagedArray<FVector3f>& LabelPositionAttribute = *InOutCollection.FindAttribute<FVector3f>(PVImportNames::LabelPositionAttribute, PVImportNames::LabelsGroup);
	TManagedArray<float>& LabelScaleAttribute = *InOutCollection.FindAttribute<float>(PVImportNames::LabelScaleAttribute, PVImportNames::LabelsGroup);

	for (int32 i = 0; i < Labels.Num(); ++i)
	{
		LabelPositionAttribute[StartIndex + i] = Labels[i].Position;
		LabelTextAttribute[StartIndex + i] = Labels[i].Text;
		LabelScaleAttribute[StartIndex + i] = Labels[i].Scale;
	}
}

void PV::ImportHelper::ComputeDynamicMeshBranchHierarchies(
	const UE::Geometry::FDynamicMesh3& DynamicMesh,
	const FBooleanVertexAttribute& RootPointsAttribute,
	const FBooleanVertexAttribute& EndPointsAttribute,
	const FFloatVertexAttribute& VertexScaleAttribute,
	FFloatVertexAttribute& LengthFromRootAttribute,
	FIntVertexAttribute& NextBranchPointAttribute,
	const TFunction<bool(int32)>& VertexFilterPredicate,
	TArray<FMeshBranchHierarchy>& OutDynamicMeshBranchHierarchies
)
{
	ComputeLengthFromRoot(
		DynamicMesh,
		RootPointsAttribute,
		EndPointsAttribute,
		LengthFromRootAttribute,
		VertexFilterPredicate
	);

	TraceBranches(
		DynamicMesh,
		LengthFromRootAttribute,
		RootPointsAttribute,
		EndPointsAttribute,
		NextBranchPointAttribute,
		VertexFilterPredicate
	);

	ExtractBranchHierarchies(
		DynamicMesh,
		RootPointsAttribute,
		EndPointsAttribute,
		NextBranchPointAttribute,
		VertexScaleAttribute,
		LengthFromRootAttribute,
		OutDynamicMeshBranchHierarchies
	);
}

void PV::ImportHelper::ConvertDynamicMeshBranchHierarchiesToBranchDescription(
	const UE::Geometry::FDynamicMesh3& DynamicMesh,
	const FFloatVertexAttribute& VertexScaleAttribute,
	const FBooleanVertexAttribute& EndPointAttribute,
	const TArray<FMeshBranchHierarchy>& InMeshBranchHierarchies,
	TArray<FPVBranchHierarchyDescription>& OutBranchHierarchyDescriptions
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConvertMeshBranchHierarchiesToBranchDescription);

	for (const FMeshBranchHierarchy& BranchHierarchy : InMeshBranchHierarchies)
	{
		int32 MaxNrOfPoints = 0;
		for (const auto& Branch : BranchHierarchy.Branches)
		{
			MaxNrOfPoints += Branch.BranchPoints.Num();
		}

		FPVBranchHierarchyDescription& OutBranchHierarchy = OutBranchHierarchyDescriptions.AddDefaulted_GetRef();
		OutBranchHierarchy.Branches.Reserve(BranchHierarchy.Branches.Num());
		OutBranchHierarchy.Points.Reserve(MaxNrOfPoints);
		OutBranchHierarchy.PointsRadii.Reserve(MaxNrOfPoints);

		TMap<int32, int32> VertexIndexToPointIndex;
		VertexIndexToPointIndex.Reserve(MaxNrOfPoints);

		for (const FMeshBranchHierarchy::FBranch& Branch : BranchHierarchy.Branches)
		{
			FPVBranchDescription& OutBranchDescription = OutBranchHierarchy.Branches.AddDefaulted_GetRef();
			OutBranchDescription.ParentBranchIndex = Branch.ParentBranch;
			OutBranchDescription.BranchIndex = OutBranchHierarchy.Branches.Num() - 1;
			OutBranchDescription.PointIndices.Reserve(Branch.BranchPoints.Num());

			for (int32 i = 0; i < Branch.BranchPoints.Num(); ++i)
			{
				const int32 VertexIndex = Branch.BranchPoints[i];
				int32 PointIndex = INDEX_NONE;

				int32* ExistingPointIndex = VertexIndexToPointIndex.Find(VertexIndex);
				if (!ExistingPointIndex)
				{
					PointIndex = OutBranchHierarchy.Points.Add(FVector3f(DynamicMesh.GetVertex(VertexIndex)));

					const bool bIsEndPoint = EndPointAttribute.GetValue(VertexIndex);
					if (bIsEndPoint)
					{
						OutBranchHierarchy.PointsRadii.Add(0.01f);
					}
					else
					{
						const float VertexRadius = VertexScaleAttribute.GetValue(VertexIndex);
						OutBranchHierarchy.PointsRadii.Add(VertexRadius);
					}

					VertexIndexToPointIndex.Add(VertexIndex, PointIndex);
				}
				else
				{
					PointIndex = *ExistingPointIndex;
				}

				OutBranchDescription.PointIndices.Add(PointIndex);
			}
		}

		for (int32 i = 0; i < OutBranchHierarchy.Branches.Num(); ++i)
		{
			const FPVBranchDescription& BranchDescription = OutBranchHierarchy.Branches[i];
			if (BranchDescription.ParentBranchIndex != INDEX_NONE)
			{
				OutBranchHierarchy.Branches[BranchDescription.ParentBranchIndex].ChildBranchIndices.Add(i);
			}
			else
			{
				check(OutBranchHierarchy.RootBranchIndex == INDEX_NONE);
				OutBranchHierarchy.RootBranchIndex = i;
			}
		}
	}
}

FPVBranchHierarchyDescription::EValidateHierarchyResult FPVBranchHierarchyDescription::ValidateHierarchy() const
{
	if (!Branches.IsValidIndex(RootBranchIndex))
	{
		return EValidateHierarchyResult::InvalidRootBranchIndex;
	}
	
	if (Branches[RootBranchIndex].ParentBranchIndex != INDEX_NONE)
	{
		return EValidateHierarchyResult::InvalidRootBranchParentIndex;
	}
	
	for (int32 BranchIndex = 0; BranchIndex < Branches.Num(); BranchIndex++)
	{
		const FPVBranchDescription& Branch = Branches[BranchIndex];
		if (Branch.BranchIndex != BranchIndex)
		{
			return EValidateHierarchyResult::BranchIndexMismatch;
		}
		
		if (Branch.ParentBranchIndex != INDEX_NONE
			&& !Branches.IsValidIndex(Branch.ParentBranchIndex))
		{
			return EValidateHierarchyResult::InvalidParentBranchIndex;
		}
		
		if (Branch.ParentBranchIndex == INDEX_NONE
			&& RootBranchIndex != Branch.BranchIndex)
		{
			return EValidateHierarchyResult::InvalidRootBranchIndex;
		}
		
		for (int32 ChildBranchIndex : Branch.ChildBranchIndices)
		{
			if (!Branches.IsValidIndex(ChildBranchIndex))
			{
				return EValidateHierarchyResult::InvalidChildBranchIndex;
			}
		}
		
		for (int32 PointIndex : Branch.PointIndices)
		{
			if (!Points.IsValidIndex(PointIndex))
			{
				return EValidateHierarchyResult::InvalidPointIndex; 
			}
		}
	}

	TSet<int32> FoundBranches;
	FoundBranches.Reserve(Branches.Num());

	const TFunction<bool(int32)> WalkHierarchyRecursive = [&](int32 BranchIndex)->bool
	{
		bool bAlreadyAdded = false;
		FoundBranches.Add(BranchIndex, &bAlreadyAdded);

		if (bAlreadyAdded)
		{
			return false;
		}

		for (int32 ChildIndex : Branches[BranchIndex].ChildBranchIndices)
		{
			if (!WalkHierarchyRecursive(ChildIndex))
			{
				return false;
			}
		}

		return true;
	};

	if (!WalkHierarchyRecursive(RootBranchIndex))
	{
		return EValidateHierarchyResult::CircularHierarchy;
	}

	if (FoundBranches.Num() != Branches.Num())
	{
		return EValidateHierarchyResult::DisjointedHierarchy;
	}

	return EValidateHierarchyResult::Success;
}

PV::ImportHelper::EAppendGrowthDataResult PV::ImportHelper::AppendGrowthData(FManagedArrayCollection& Target, const FManagedArrayCollection& Source)
{
	if (!PV::Utilities::IsValidGrowthData(Target))
	{
		return EAppendGrowthDataResult::InvalidTarget;
	}

	if (!PV::Utilities::IsValidGrowthData(Source))
	{
		return EAppendGrowthDataResult::InvalidSource;
	}

	const int32 TargetPointOffset  = Target.NumElements(PV::GroupNames::PointGroup);
	const int32 TargetBranchOffset = Target.NumElements(PV::GroupNames::BranchGroup);
	const int32 SourcePointCount   = Source.NumElements(PV::GroupNames::PointGroup);
	const int32 SourceBranchCount  = Source.NumElements(PV::GroupNames::BranchGroup);

	int32 PlantNumberOffset = 0;
	if (TargetBranchOffset > 0)
	{
		auto TargetPlantNumbers = FBranchPlantNumberAttribute::GetAttribute(Target);
		for (int32 i = 0; i < TargetBranchOffset; ++i)
		{
			PlantNumberOffset = FMath::Max(PlantNumberOffset, TargetPlantNumbers[i] + 1);
		}
	}

	Target.AddElements(SourcePointCount,  PV::GroupNames::PointGroup);
	Target.AddElements(SourceBranchCount, PV::GroupNames::BranchGroup);

	// PointGroup — direct copy (no index references)
	{
		auto SrcPos           = FPointPositionAttribute::GetAttribute(Source);
		auto SrcLenRoot       = FPointLengthFromRootAttribute::GetAttribute(Source);
		auto SrcLenSeed       = FPointLengthFromSeedAttribute::GetAttribute(Source);
		auto SrcHullGrad      = FPointHullGradientAttribute::GetAttribute(Source);
		auto SrcTrunkGrad     = FPointMainTrunkGradientAttribute::GetAttribute(Source);
		auto SrcScale         = FPointScaleAttribute::GetAttribute(Source);
		auto SrcBudDir        = FBudDirectionAttribute::GetAttribute(Source);
		auto SrcBudStatus     = FBudStatusAttribute::GetAttribute(Source);
		auto SrcBudLateral    = FBudLateralMeristemAttribute::GetAttribute(Source);
		auto SrcBudHormone    = FBudHormoneLevelsAttribute::GetAttribute(Source);
		auto SrcPlantGrad     = FPointPlantGradientAttribute::GetAttribute(Source);
		auto SrcGroundGrad    = FPointGroundGradientAttribute::GetAttribute(Source);
		auto SrcBudLight      = FBudLightDetectedAttribute::GetAttribute(Source);
		auto SrcBudDev        = FBudDevelopmentAttribute::GetAttribute(Source);

		auto DstPos           = FPointPositionAttribute::GetAttribute(Target);
		auto DstLenRoot       = FPointLengthFromRootAttribute::GetAttribute(Target);
		auto DstLenSeed       = FPointLengthFromSeedAttribute::GetAttribute(Target);
		auto DstHullGrad      = FPointHullGradientAttribute::GetAttribute(Target);
		auto DstTrunkGrad     = FPointMainTrunkGradientAttribute::GetAttribute(Target);
		auto DstScale         = FPointScaleAttribute::GetAttribute(Target);
		auto DstBudDir        = FBudDirectionAttribute::GetAttribute(Target);
		auto DstBudStatus     = FBudStatusAttribute::GetAttribute(Target);
		auto DstBudLateral    = FBudLateralMeristemAttribute::GetAttribute(Target);
		auto DstBudHormone    = FBudHormoneLevelsAttribute::GetAttribute(Target);
		auto DstPlantGrad     = FPointPlantGradientAttribute::GetAttribute(Target);
		auto DstGroundGrad    = FPointGroundGradientAttribute::GetAttribute(Target);
		auto DstBudLight      = FBudLightDetectedAttribute::GetAttribute(Target);
		auto DstBudDev        = FBudDevelopmentAttribute::GetAttribute(Target);

		for (int32 i = 0; i < SourcePointCount; ++i)
		{
			const int32 Dst = TargetPointOffset + i;
			DstPos[Dst]              = SrcPos[i];
			DstLenRoot[Dst]          = SrcLenRoot[i];
			DstLenSeed[Dst]          = SrcLenSeed[i];
			DstHullGrad[Dst]         = SrcHullGrad[i];
			DstTrunkGrad[Dst]        = SrcTrunkGrad[i];
			DstScale[Dst]            = SrcScale[i];
			DstPlantGrad[Dst]        = SrcPlantGrad[i];
			DstGroundGrad[Dst]       = SrcGroundGrad[i];
			DstBudDir[Dst].Array     = SrcBudDir[i].Array;
			DstBudStatus[Dst].Array  = SrcBudStatus[i].Array;
			DstBudLateral[Dst].Array = SrcBudLateral[i].Array;
			DstBudHormone[Dst].Array = SrcBudHormone[i].Array;
			DstBudLight[Dst].Array   = SrcBudLight[i].Array;
			DstBudDev[Dst].Array     = SrcBudDev[i].Array;
		}
	}

	// PointGroup — bud numbers are 1-based sequential indices global to the collection
	// NjordPixelIndex - BudNumber + intra-bud fraction; offset matches the bud number offset.
	{
		auto SrcBudNum = FPointBudNumberAttribute::GetAttribute(Source);
		auto DstBudNum = FPointBudNumberAttribute::GetAttribute(Target);
		
		for (int32 i = 0; i < SourcePointCount; ++i)
		{
			DstBudNum[TargetPointOffset + i] = SrcBudNum[i] + TargetPointOffset;
		}
	}

	// BranchGroup
	{
		auto SrcBranchNum       = FBranchNumberAttribute::GetAttribute(Source);
		auto SrcParentNum       = FBranchParentNumberAttribute::GetAttribute(Source);
		auto SrcHierarchyNum    = FBranchHierarchyNumberAttribute::GetAttribute(Source);
		auto SrcPlantNum        = FBranchPlantNumberAttribute::GetAttribute(Source);
		auto SrcSourceBudNum    = FBranchSourceBudNumberAttribute::GetAttribute(Source);
		auto SrcParents         = FBranchParentsAttribute::GetAttribute(Source);
		auto SrcChildren        = FBranchChildrenAttribute::GetAttribute(Source);
		auto SrcPoints          = FBranchPointsAttribute::GetAttribute(Source);
		auto SrcFoliageIDs      = FBranchFoliageIDsAttribute::GetAttribute(Source);

		auto DstBranchNum       = FBranchNumberAttribute::GetAttribute(Target);
		auto DstParentNum       = FBranchParentNumberAttribute::GetAttribute(Target);
		auto DstHierarchyNum    = FBranchHierarchyNumberAttribute::GetAttribute(Target);
		auto DstPlantNum        = FBranchPlantNumberAttribute::GetAttribute(Target);
		auto DstSourceBudNum    = FBranchSourceBudNumberAttribute::GetAttribute(Target);
		auto DstParents         = FBranchParentsAttribute::GetAttribute(Target);
		auto DstChildren        = FBranchChildrenAttribute::GetAttribute(Target);
		auto DstPoints          = FBranchPointsAttribute::GetAttribute(Target);
		auto DstFoliageIDs      = FBranchFoliageIDsAttribute::GetAttribute(Target);

		for (int32 i = 0; i < SourceBranchCount; ++i)
		{
			const int32 Dst = TargetBranchOffset + i;

			DstBranchNum[Dst]    = SrcBranchNum[i] + TargetBranchOffset;
			// 0 is the root sentinel meaning "no parent"; preserve it without offset.
			DstParentNum[Dst]    = SrcParentNum[i] != 0 ? SrcParentNum[i] + TargetBranchOffset : 0;
			DstHierarchyNum[Dst] = SrcHierarchyNum[i];
			DstPlantNum[Dst]     = SrcPlantNum[i] + PlantNumberOffset;
			DstSourceBudNum[Dst] = SrcSourceBudNum[i] + TargetPointOffset;

			// BranchParents stores the ancestor chain as branch numbers; 0 is the root sentinel.
			TArray<int32> Parents = SrcParents[i];
			for (int32& BranchNum : Parents)
			{
				if (BranchNum != 0)
				{
					BranchNum += TargetBranchOffset;
				}
			}
			DstParents[Dst] = MoveTemp(Parents);

			TArray<int32> Children = SrcChildren[i];
			for (int32& BranchNum : Children)
			{
				BranchNum += TargetBranchOffset;
			}
			DstChildren[Dst] = MoveTemp(Children);

			TArray<int32> Points = SrcPoints[i];
			for (int32& PointIdx : Points)
			{
				PointIdx += TargetPointOffset;
			}
			DstPoints[Dst] = MoveTemp(Points);

			// FoliageIDs reference the DetailsGroup which is not appended; copy as-is.
			DstFoliageIDs[Dst] = SrcFoliageIDs[i];
		}
	}

	return EAppendGrowthDataResult::Success;
}

void PV::ImportHelper::CreateEmptyGrowthData(FManagedArrayCollection& OutCollection, int32 NumPoints, int32 NumBranches)
{
	// PointGroup
	OutCollection.AddGroup(PV::GroupNames::PointGroup);
	OutCollection.AddElements(NumPoints, PV::GroupNames::PointGroup);
	FPointPositionAttribute::AddAttribute(OutCollection, FVector3f::ZeroVector);
	FPointLengthFromRootAttribute::AddAttribute(OutCollection, 0);
	FPointLengthFromSeedAttribute::AddAttribute(OutCollection, 0);
	FPointHullGradientAttribute::AddAttribute(OutCollection, 0);
	FPointMainTrunkGradientAttribute::AddAttribute(OutCollection, 0);
	FPointScaleAttribute::AddAttribute(OutCollection, 0.f);
	FBudDirectionAttribute::AddAttribute(OutCollection);
	FBudStatusAttribute::AddAttribute(OutCollection);
	FBudLateralMeristemAttribute::AddAttribute(OutCollection);
	FBudHormoneLevelsAttribute::AddAttribute(OutCollection);
	FPointPlantGradientAttribute::AddAttribute(OutCollection, 0);
	FPointGroundGradientAttribute::AddAttribute(OutCollection);
	FPointBudNumberAttribute::AddAttribute(OutCollection, 0);
	FBudLightDetectedAttribute::AddAttribute(OutCollection);
	FBudDevelopmentAttribute::AddAttribute(OutCollection);
	FPointSeedPScaleAttribute::AddAttribute(OutCollection, 1.f);
	FPointSeedPScaleRatioAttribute::AddAttribute(OutCollection, 1.f);

	// BranchGroup
	OutCollection.AddGroup(PV::GroupNames::BranchGroup);
	OutCollection.AddElements(NumBranches, PV::GroupNames::BranchGroup);
	FBranchParentsAttribute::AddAttribute(OutCollection);
	FBranchChildrenAttribute::AddAttribute(OutCollection);
	FBranchPointsAttribute::AddAttribute(OutCollection);
	FBranchNumberAttribute::AddAttribute(OutCollection, 0);
	FBranchSourceBudNumberAttribute::AddAttribute(OutCollection, 0);
	FBranchHierarchyNumberAttribute::AddAttribute(OutCollection, 0);
	FBranchParentNumberAttribute::AddAttribute(OutCollection, 0);
	FBranchPlantNumberAttribute::AddAttribute(OutCollection, 0);
	FBranchFoliageIDsAttribute::AddAttribute(OutCollection);

	// DetailsGroup
	OutCollection.AddGroup(PV::GroupNames::DetailsGroup);
	OutCollection.AddElements(1, PV::GroupNames::DetailsGroup);
	FDetailFoliagePathAttribute::AddAttribute(OutCollection, TEXT(""));
	FDetailTrunkMaterialPathAttribute::AddAttribute(OutCollection, TEXT(""));
	FDetailTrunkURangeAttribute::AddAttribute(OutCollection, { });
	FDetailLeafPhyllotaxyAttribute::AddAttribute(OutCollection, { 0, 200, 50, 1, 1, 0, 0, 0 });
}

void PV::ImportHelper::FillAttributesFromBranchHierarchy(
	FManagedArrayCollection& OutCollection,
	TArrayView<const FPVBranchHierarchyDescription> InBranchHierarchies
)
{
	FillAttributesFromBranchHierarchy(
		FPointPositionAttribute::GetAttribute(OutCollection),
		FPointScaleAttribute::GetAttribute(OutCollection),
		FBranchPointsAttribute::GetAttribute(OutCollection),
		FBranchParentsAttribute::GetAttribute(OutCollection),
		FBranchChildrenAttribute::GetAttribute(OutCollection),
		FBranchParentNumberAttribute::GetAttribute(OutCollection),
		FBranchNumberAttribute::GetAttribute(OutCollection),
		FBranchHierarchyNumberAttribute::GetAttribute(OutCollection),
		FBranchPlantNumberAttribute::GetAttribute(OutCollection),
		InBranchHierarchies
	);
}

void PV::ImportHelper::FillAttributesFromBranchHierarchy(
	PV::FPointPositionAttributeView PointPositionAttribute,
	PV::FPointScaleAttributeView PointScaleAttribute,
	PV::FBranchPointsAttributeView BranchPointsAttribute,
	PV::FBranchParentsAttributeView BranchParentsAttribute,
	PV::FBranchChildrenAttributeView BranchChildrenAttribute,
	PV::FBranchParentNumberAttributeView BranchParentNumberAttribute,
	PV::FBranchNumberAttributeView BranchNumberAttribute,
	PV::FBranchHierarchyNumberAttributeView BranchHierarchyNumberAttribute,
	PV::FBranchPlantNumberAttributeView BranchPlantNumberAttribute,
	TArrayView<const FPVBranchHierarchyDescription> InBranchHierarchies
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FillAttributesFromBranchHierarchy);

	const static auto BranchIndexToBranchNumber = [](int32 BranchIndex) 
	{ 
		// Branch Number is BranchIndex + 1 since branch with parent number 0 is a root in PVE and our root has parent index -1.
		return BranchIndex + 1; 
	}; 

	const static auto BranchNumberToBranchIndex = [](int32 BranchNumber)
	{ 
		// Note that BranchNumberToBranchIndex only works within the local scope of FillAttributesFromBranchHierarchy due to how
		// we compute the BranchNumber in BranchIndexToBranchNumber
		return BranchNumber - 1;
	}; 

	using FFillBranchesRecursive = TFunction<void(const FPVBranchHierarchyDescription&, int32, int32, int32, int32, int32)>;

	const FFillBranchesRecursive FillBranchesRecursive = [&](
		const FPVBranchHierarchyDescription& BranchHierarchy, 
		int32 PlantNumber,
		int32 BranchIndex,
		int32 AttributeBranchIndexOffset,
		int32 AttributePointIndexOffset,
		int32 Depth
	)
	{
		const FPVBranchDescription& Branch = BranchHierarchy.Branches[BranchIndex];

		const int32 AttributeRootBranchIndex = BranchHierarchy.RootBranchIndex + AttributeBranchIndexOffset;
		const int32 AttributeBranchIndex = BranchIndex + AttributeBranchIndexOffset;
		const int32 AttributeParentBranchIndex = Branch.ParentBranchIndex != INDEX_NONE
			? Branch.ParentBranchIndex + AttributeBranchIndexOffset
			: INDEX_NONE;

		TArray<int32> AttributeBranchPoints;
		AttributeBranchPoints.Reserve(Branch.PointIndices.Num());

		for (int32 i = 0; i < Branch.PointIndices.Num(); ++i)
		{
			const int32 AttributePointIndex = Branch.PointIndices[i] + AttributePointIndexOffset;
			AttributeBranchPoints.Add(AttributePointIndex);
		}
		BranchPointsAttribute[AttributeBranchIndex] = MoveTemp(AttributeBranchPoints);

		const bool bIsRootBranch = AttributeParentBranchIndex == INDEX_NONE;
		if (bIsRootBranch)
		{
			BranchParentsAttribute[AttributeBranchIndex] = { 0 };
		}
		else
		{
			TArray<int32> Parents = BranchParentsAttribute[AttributeParentBranchIndex];
			Parents.Add(BranchIndexToBranchNumber(AttributeParentBranchIndex));
			BranchParentsAttribute[AttributeBranchIndex] = MoveTemp(Parents);
		}

		TArray<int32> AttributeBranchChildren;
		AttributeBranchChildren.SetNumUninitialized(Branch.ChildBranchIndices.Num());
		for (int32 i = 0; i < Branch.ChildBranchIndices.Num(); ++i)
		{
			const int32 AttributeChildBranchIndex = Branch.ChildBranchIndices[i] + AttributeBranchIndexOffset;

			check(AttributeChildBranchIndex != AttributeRootBranchIndex);
			AttributeBranchChildren[i] = BranchIndexToBranchNumber(AttributeChildBranchIndex);
		}

		BranchChildrenAttribute[AttributeBranchIndex] = MoveTemp(AttributeBranchChildren);

		BranchParentNumberAttribute[AttributeBranchIndex] = BranchIndexToBranchNumber(AttributeParentBranchIndex);
		BranchNumberAttribute[AttributeBranchIndex] = BranchIndexToBranchNumber(AttributeBranchIndex);

		BranchHierarchyNumberAttribute[AttributeBranchIndex] = Depth;
		BranchPlantNumberAttribute[AttributeBranchIndex] = PlantNumber;

		for (int32 ChildIndex : Branch.ChildBranchIndices)
		{
			FillBranchesRecursive(
				BranchHierarchy, 
				PlantNumber,
				ChildIndex, 
				AttributeBranchIndexOffset,
				AttributePointIndexOffset,
				Depth + 1
			);
		}
	};

	int32 AttributeBranchIndexOffset = 0;
	int32 AttributePointIndexOffset = 0;
	int32 PlantNumber = 0;
	for (const FPVBranchHierarchyDescription& BranchHierarchy : InBranchHierarchies)
	{
		FillBranchesRecursive(
			BranchHierarchy, 
			PlantNumber,
			BranchHierarchy.RootBranchIndex, 
			AttributeBranchIndexOffset,
			AttributePointIndexOffset,
			0
		);

		for (int32 i = 0; i < BranchHierarchy.Points.Num(); ++i)
		{
			PointPositionAttribute[i + AttributePointIndexOffset] = BranchHierarchy.Points[i];
			PointScaleAttribute[i + AttributePointIndexOffset] = BranchHierarchy.PointsRadii[i] > 0 ? BranchHierarchy.PointsRadii[i] : 0;
		}

		AttributeBranchIndexOffset += BranchHierarchy.Branches.Num();
		AttributePointIndexOffset += BranchHierarchy.Points.Num();
		PlantNumber += 1;
	}
	
	// Fill all descendants in the BranchChildren attribute by walking the tree hierarchy backwards
	// and appending the children of our children. This only works since we know for certain
	// all branches only contain the immediate children at this stage.
	PV::PlantTraversalHelper::RecursiveWalkBranches_Reversed(
		BranchParentNumberAttribute,
		BranchChildrenAttribute,
		BranchNumberAttribute,
		[&](int32 BranchIndex)
		{
			const TArray<int32> ImmediateChildBranchNumbers = BranchChildrenAttribute[BranchIndex]; // We know these are the immediate children due to not filling the descendants above.
			for (const int32 ChildBranchNumber : ImmediateChildBranchNumbers)
			{
				const int32 ChildBranchIndex = BranchNumberToBranchIndex(ChildBranchNumber);
				if (ensure(BranchChildrenAttribute.IsValidIndex(ChildBranchIndex)))
				{
					BranchChildrenAttribute[BranchIndex].Append(BranchChildrenAttribute[ChildBranchIndex]);
				}
			}

			return PV::PlantTraversalHelper::EForEachResult::Continue;
		}
	);
}

bool PV::ImportHelper::GenerateGrowthDataFromBranchHierarchies(FManagedArrayCollection& OutCollection, TArrayView<const FPVBranchHierarchyDescription> InBranchHierarchies)
{
	using namespace PV::AttributesHelper;

	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateGrowthDataFromBranchHierarchies);

	if (InBranchHierarchies.Num() < 1)
	{
		return false;
	}

	int32 NumPoints = 0;
	int32 NumBranches = 0;

	for (const FPVBranchHierarchyDescription& BranchHierarchy : InBranchHierarchies)
	{
		NumPoints += BranchHierarchy.GetNumPoints();
		NumBranches += BranchHierarchy.GetNumBranches();

		if (BranchHierarchy.ValidateHierarchy() != FPVBranchHierarchyDescription::EValidateHierarchyResult::Success)
		{
			return false;
		}
	}

	CreateEmptyGrowthData(OutCollection, NumPoints, NumBranches);
	
	auto PointPositionAttribute = FPointPositionAttribute::GetAttribute(OutCollection);
	auto PointScaleAttribute = FPointScaleAttribute::GetAttribute(OutCollection);
	auto BranchPointsAttribute = FBranchPointsAttribute::GetAttribute(OutCollection);
	auto BranchParentNumberAttribute = FBranchParentNumberAttribute::GetAttribute(OutCollection);
	auto BranchParentsAttribute = FBranchParentsAttribute::GetAttribute(OutCollection);
	auto BranchNumberAttribute = FBranchNumberAttribute::GetAttribute(OutCollection);
	auto BranchChildrenAttribute = FBranchChildrenAttribute::GetAttribute(OutCollection);
	auto BranchHierarchyNumberAttribute = FBranchHierarchyNumberAttribute::GetAttribute(OutCollection);
	auto BranchPlantNumberAttribute = FBranchPlantNumberAttribute::GetAttribute(OutCollection);
	FillAttributesFromBranchHierarchy(
		PointPositionAttribute,
		PointScaleAttribute,
		BranchPointsAttribute,
		BranchParentsAttribute,
		BranchChildrenAttribute,
		BranchParentNumberAttribute,
		BranchNumberAttribute,
		BranchHierarchyNumberAttribute,
		BranchPlantNumberAttribute,
		InBranchHierarchies
	);

	const FRecomputeGrowthDataAttributes GrowthDataAttributes(OutCollection);
	if (!GrowthDataAttributes.IsValid())
	{
		return false;
	}

	RecomputeAllGrowthDataAttributes(GrowthDataAttributes);

	return true;
}

bool PV::ImportHelper::GenerateGrowthDataFromBranchHierarchy(FManagedArrayCollection& OutCollection, const FPVBranchHierarchyDescription& InBranchHierarchy)
{
	TArrayView<const FPVBranchHierarchyDescription> BranchHierarchyArrayView(&InBranchHierarchy, 1);
	return GenerateGrowthDataFromBranchHierarchies(OutCollection, BranchHierarchyArrayView);
}

void PV::ImportHelper::DistToBranch(const FVector3f& Point, const TArray<FVector3f>& Points, const TArray<int32>& BranchPointIndices, FVector3f& OutClosestPoint, float& OutDistSquared, int32& OutPointIndex)
{
	OutClosestPoint = FVector3f::ZeroVector;
	OutDistSquared = MAX_FLT;
	OutPointIndex = INDEX_NONE;

	for (int32 i = 0; i < BranchPointIndices.Num() - 1; ++i)
	{
		const int32 PointIndex0 = BranchPointIndices[i];
		const int32 PointIndex1 = BranchPointIndices[i + 1];
		if (!Points.IsValidIndex(PointIndex0) || !Points.IsValidIndex(PointIndex1))
		{
			continue;
		}

		const FVector3f& BranchPoint1 = Points[PointIndex0];
		const FVector3f& BranchPoint2 = Points[PointIndex1];

		const FVector3f ClosestPointOnLine = FMath::ClosestPointOnSegment(Point, BranchPoint1, BranchPoint2);
		const float DistToLineSquared = (Point - ClosestPointOnLine).SizeSquared();
		if (DistToLineSquared < OutDistSquared)
		{
			OutDistSquared = DistToLineSquared;
			OutClosestPoint = ClosestPointOnLine;
			OutPointIndex = PointIndex0;
		}
	}
}

float PV::ImportHelper::DistToBranch(const FVector3f& Point, const TArray<FVector3f>& Points, const TArray<int32>& BranchPointIndices)
{
	FVector3f ClosestPoint;
	float ClosestDistSquared = MAX_FLT;
	int32 PointIndex;
	DistToBranch(Point, Points, BranchPointIndices, ClosestPoint, ClosestDistSquared, PointIndex);

	return ClosestDistSquared > 0 ? FMath::Sqrt(ClosestDistSquared) : 0;
}

TArray<int32> PV::ImportHelper::EstimateBranchHierarchy(const TArray<FVector3f>& Points, const TArray<TArray<int32>>& Branches, int32 RootBranchIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PV::ImportHelper::EstimateBranchHierarchy);

	TArray<int32> OutBranchParents;
	OutBranchParents.SetNum(Branches.Num());

	for (const TArray<int32>& Branch : Branches)
	{
		for (int32 PointIndex : Branch)
		{
			if (!Points.IsValidIndex(PointIndex))
			{
				return TArray<int32>();
			}
		}
	}

	for (int32 i = 0; i < Branches.Num(); ++i)
	{
		if (i == RootBranchIndex)
		{
			OutBranchParents[i] = INDEX_NONE;
			continue;
		}

		if (Branches[i].Num() == 0)
		{
			continue;
		}

		const int32 BranchRootPointIndex = Branches[i][0];
		const FVector3f& BranchRootPoint = Points[BranchRootPointIndex];
		float ClosestBranchDistSquared = MAX_FLT;
		int32 ClosestBranchIndex = INDEX_NONE;
		bool ClosestBranchPointIsRoot = true;

		for (int32 j = 0; j < Branches.Num(); ++j)
		{
			if (i == j)
			{
				continue;
			}

			FVector3f ClosestPointOnBranch;
			float BranchDistSquared = MAX_FLT;
			int32 ClosestPointIndex;
			ImportHelper::DistToBranch(BranchRootPoint, Points, Branches[j], ClosestPointOnBranch, BranchDistSquared, ClosestPointIndex);

			const bool bBranchPointIsRoot = ClosestPointIndex == BranchRootPointIndex;

			// If there are multiple branches growing from the same point then we may select the wrong point as the distance to the branch will be the same.
			// We therefore prefer the branch where the closest point is not root.
			if ((FMath::IsNearlyEqual(BranchDistSquared, ClosestBranchDistSquared) && ClosestBranchPointIsRoot && !bBranchPointIsRoot)
				|| BranchDistSquared < ClosestBranchDistSquared)
			{
				ClosestBranchIndex = j;
				ClosestBranchDistSquared = BranchDistSquared;
				ClosestBranchPointIsRoot = bBranchPointIsRoot;
			}
		}

		OutBranchParents[i] = ClosestBranchIndex;
	}

	return OutBranchParents;
}

int32 PV::ImportHelper::FindRootBranch(const TArray<FVector3f>& InPoints, const TArray<TArray<int32>>& BranchIndices)
{
	float LowestRootPoint = MAX_FLT;
	int32 LowestRootPointIdx = INDEX_NONE;
	for (int32 i = 0; i < BranchIndices.Num(); ++i)
	{
		const TArray<int32>& Branch = BranchIndices[i];
		if (Branch.Num() > 0 && InPoints[Branch[0]].Z < LowestRootPoint)
		{
			LowestRootPoint = InPoints[Branch[0]].Z;
			LowestRootPointIdx = i;
		}
	}
	return LowestRootPointIdx;
}

FPVBranchHierarchyDescription PV::ImportHelper::CreateBranchHierarchyFromPoints(const TArray<FVector3f>& InPoints, const TArray<float>& InPointsRadii, const TArray<TArray<int32>>& InBranchPointIndices)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PV::ImportHelper::CreateBranchHierarchyFromPoints);

	if (InBranchPointIndices.Num() == 0 || InPoints.Num() == 0)
	{
		return FPVBranchHierarchyDescription();
	}
	
	if (InPointsRadii.Num() != InPoints.Num())
	{
		return FPVBranchHierarchyDescription();
	}

	if (InBranchPointIndices.Num() == 1)
	{
		FPVBranchHierarchyDescription OutBranchHierarchy;
		OutBranchHierarchy.RootBranchIndex = 0;
		OutBranchHierarchy.Points = InPoints;
		OutBranchHierarchy.PointsRadii = InPointsRadii;
		FPVBranchDescription& Branch = OutBranchHierarchy.Branches.AddDefaulted_GetRef();
		Branch.PointIndices = InBranchPointIndices[0];
		Branch.ParentBranchIndex = INDEX_NONE;
		Branch.BranchIndex = 0;
		return OutBranchHierarchy;
	}

	const int32 RootBranchIndex = FindRootBranch(InPoints, InBranchPointIndices);

	FPVBranchHierarchyDescription OutBranchHierarchy;
	OutBranchHierarchy.RootBranchIndex = RootBranchIndex;
	OutBranchHierarchy.Points = InPoints;
	OutBranchHierarchy.PointsRadii = InPointsRadii;

	// Fill branch points
	for (const TArray<int32>& BranchPointIndices : InBranchPointIndices)
	{
		FPVBranchDescription& Branch = OutBranchHierarchy.Branches.AddDefaulted_GetRef();
		Branch.PointIndices = BranchPointIndices;
	}

	// Set up branch hierarchy
	const TArray<int32> BranchParents = EstimateBranchHierarchy(InPoints, InBranchPointIndices, RootBranchIndex);
	for (int32 i = 0; i < BranchParents.Num(); ++i)
	{
		OutBranchHierarchy.Branches[i].BranchIndex = i;
		OutBranchHierarchy.Branches[i].ParentBranchIndex = BranchParents[i];

		if (BranchParents[i] != INDEX_NONE)
		{
			OutBranchHierarchy.Branches[BranchParents[i]].ChildBranchIndices.Add(i);
		}
	}

	if (OutBranchHierarchy.ValidateHierarchy() != FPVBranchHierarchyDescription::EValidateHierarchyResult::Success)
	{
		return FPVBranchHierarchyDescription();
	}

	return OutBranchHierarchy;
}
