// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVImportCommon.h"
#include "Utils/PVDynamicMeshVertexAttribute.h"
#include "Utils/PVAttributes.h"

struct FManagedArrayCollection;

namespace UE::Geometry
{
	class FDynamicMesh3;
}

struct FPVBranchDescription
{
	int32 ParentBranchIndex = INDEX_NONE;
	int32 BranchIndex = INDEX_NONE;
	TArray<int32> ChildBranchIndices;
	TArray<int32> PointIndices;
};

struct FPVBranchHierarchyDescription
{
	int32 RootBranchIndex = INDEX_NONE;
	TArray<FPVBranchDescription> Branches;
	TArray<FVector3f> Points;
	TArray<float> PointsRadii;

	int32 GetNumBranches() const { return Branches.Num(); }
	int32 GetNumPoints() const { return Points.Num(); }

	enum class EValidateHierarchyResult
	{
		InvalidRootBranchIndex,
		InvalidRootBranchParentIndex,
		InvalidParentBranchIndex,
		InvalidBranchIndex,
		InvalidChildBranchIndex,
		InvalidPointIndex,
		BranchIndexMismatch,
		DisjointedHierarchy,
		CircularHierarchy,
		Success
	};
	EValidateHierarchyResult ValidateHierarchy() const;
};

namespace PV::ImportHelper
{
	using FBooleanVertexAttribute = TDynamicMeshVertexAttributeExt<bool>;
	using FIntVertexAttribute     = TDynamicMeshVertexAttributeExt<int32>;
	using FFloatVertexAttribute   = TDynamicMeshVertexAttributeExt<float>;

	extern const PV::TDynamicMeshVertexAttributeDefinition<float> VertexScaleAttributeDefinition;
	extern const PV::TDynamicMeshVertexAttributeDefinition<bool> EndPointAttributeDefinition;
	extern const PV::TDynamicMeshVertexAttributeDefinition<bool> RootPointAttributeDefinition;
	extern const PV::TDynamicMeshVertexAttributeDefinition<float> LengthFromRootAttributeDefinition;
	extern const PV::TDynamicMeshVertexAttributeDefinition<int32> NextBranchPointAttributeDefinition;

	void AddDynamicMeshToCollection(
		const UE::Geometry::FDynamicMesh3& DynamicMesh, 
		FManagedArrayCollection& InOutCollection, 
		const FTransform& Offset = FTransform::Identity
	);

	void AddDynamicMeshVerticesToCollection(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FBooleanVertexAttribute& Filter,
		const FTransform& VertexOffset,
		FManagedArrayCollection& InOutCollection,
		const FName TargetGroupName,
		const FName TargetAttributeName
	);
	
	void AddBranchHierarchiesToCollection(
		const TArray<FPVBranchHierarchyDescription>& BranchHierarchyDescriptions,
		FManagedArrayCollection& InOutCollection,
		const FVector& Offset = FVector::ZeroVector,
		const FName TargetAttributeName = PVImportNames::BranchHierarchyAttribute,
		const FName TargetGroupName = PVImportNames::BranchHierarchyGroup
	);

	struct FLabel
	{
		FVector3f Position;
		FString Text;
		float Scale;
	};
	void AddLabelsToCollection(const TArray<FLabel>& Labels, FManagedArrayCollection& InOutCollection);

	struct FMeshBranchHierarchy
	{
		struct FBranch
		{
			TArray<int32> BranchPoints;
			int32 ParentBranch = INDEX_NONE;
		};

		TArray<FBranch> Branches;
	};

	void ComputeDynamicMeshBranchHierarchies(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FBooleanVertexAttribute& RootPointsAttribute,
		const FBooleanVertexAttribute& EndPointsAttribute,
		const FFloatVertexAttribute& VertexScaleAttribute,
		FFloatVertexAttribute& LengthFromRootAttribute,
		FIntVertexAttribute& NextBranchPointAttribute,
		const TFunction<bool(int32)>& VertexFilterPredicate,
		TArray<FMeshBranchHierarchy>& OutDynamicMeshBranchHierarchies
	);

	void ConvertDynamicMeshBranchHierarchiesToBranchDescription(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FFloatVertexAttribute& VertexScaleAttribute,
		const FBooleanVertexAttribute& EndPointAttribute,
		const TArray<FMeshBranchHierarchy>& InMeshBranchHierarchies,
		TArray<FPVBranchHierarchyDescription>& OutBranchHierarchyDescriptions
	);

	enum class EAppendGrowthDataResult
	{
		Success,
		InvalidTarget,
		InvalidSource
	};
	EAppendGrowthDataResult AppendGrowthData(FManagedArrayCollection& Target, const FManagedArrayCollection& Source);

	// Creates a collection containing all attributes (of the supplied size) required for creating PV growth data.
	void CreateEmptyGrowthData(FManagedArrayCollection& OutCollection, int32 NumPoints, int32 NumBranches);

	void FillAttributesFromBranchHierarchy(
		FManagedArrayCollection& OutCollection,
		TArrayView<const FPVBranchHierarchyDescription> InBranchHierarchies
	);

	// Copies the branch hierarchies from the input BranchHierarchies to the attributes.
	// Assumes the attributes are large enough to receive the hierarchies.
	void FillAttributesFromBranchHierarchy(
		PV::FPointPositionAttributeView PointPositionAttribute,
		PV::FPointScaleAttributeView PointScaleAttribute,
		PV::FBranchPointsAttributeView BranchPointsAttribute,
		PV::FBranchParentsAttributeView BranchParentsAttribute,
		PV::FBranchChildrenAttributeView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeView BranchParentNumberAttribute,
		PV::FBranchNumberAttributeView BranchNumberAttribute,
		PV::FBranchHierarchyNumberAttributeView BranchHierarchyNumberAttribute,
		PV::FBranchPlantNumberAttributeView BranchPlantNumberAttribute,
		TArrayView<const FPVBranchHierarchyDescription> InBranchHierarchies
	);

	void ComputeLengthFromRoot(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FBooleanVertexAttribute& RootPointsAttribute,
		const FBooleanVertexAttribute& EndPointsAttribute,
		FFloatVertexAttribute& LengthFromRootAttribute,
		const TFunction<bool(int32)>& VertexFilterPredicate
	);

	void TraceBranches(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FFloatVertexAttribute& LengthFromRootAttribute,
		const FBooleanVertexAttribute& RootPointAttribute,
		const FBooleanVertexAttribute& EndPointAttribute,
		FIntVertexAttribute& NextBranchPointAttribute,
		const TFunction<bool(int32)>& VertexFilterPredicate
	);

	void ExtractBranchHierarchies(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FBooleanVertexAttribute& RootPointAttribute,
		const FBooleanVertexAttribute& EndPointAttribute,
		const FIntVertexAttribute& NextBranchPointAttribute,
		const FFloatVertexAttribute& VertexScaleAttribute,
		const FFloatVertexAttribute& LengthFromRootAttribute,
		TArray<FMeshBranchHierarchy>& OutMeshBranchHierarchies
	);

	bool GenerateGrowthDataFromBranchHierarchies(FManagedArrayCollection& OutCollection, TArrayView<const FPVBranchHierarchyDescription> InBranchHierarchies);

	bool GenerateGrowthDataFromBranchHierarchy(FManagedArrayCollection& OutCollection, const FPVBranchHierarchyDescription& InBranchHierarchy);

	void DistToBranch(const FVector3f& Point, const TArray<FVector3f>& Points, const TArray<int32>& BranchPointIndices, FVector3f& OutClosestPoint, float& OutDistSquared, int32& OutPointIndex);

	float DistToBranch(const FVector3f& Point, const TArray<FVector3f>& Points, const TArray<int32>& BranchPointIndices);

	// Returns array of parent branch index for each supplied branch. Root branch will have parent index INDEX_NONE.
	TArray<int32> EstimateBranchHierarchy(const TArray<FVector3f>& Points, const TArray<TArray<int32>>& Branches, int32 RootBranchIndex);

	int32 FindRootBranch(const TArray<FVector3f>& InPoints, const TArray<TArray<int32>>& BranchIndices);

	FPVBranchHierarchyDescription CreateBranchHierarchyFromPoints(const TArray<FVector3f>& InPoints, const TArray<float>& InPointsRadii, const TArray<TArray<int32>>& InBranchPointIndices);
};