// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVJSONHelper.h"

#include "ProceduralVegetationModule.h"
#include "PVAttributesHelper.h"
#include "DataAssets/ProceduralVegetationPlantProfileDataAsset.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Facades/PVAttributesNames.h"
#include "GeometryCollection/ManagedArray.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVMetaInfoFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVProfileFacade.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

namespace PV::JSON
{
	bool LoadJSON(const FString& InPath, TSharedPtr<FJsonObject>& OutData, FString& OutErrorMessage)
	{
		FString JsonString;
		if (!FFileHelper::LoadFileToString(JsonString, *InPath))
		{
			OutErrorMessage = TEXT("Failed to open file");
			return false;
		}

		OutData = MakeShareable(new FJsonObject);
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonString), OutData))
		{
			OutErrorMessage = TEXT("Failed to deserialize JSON file");
			return false;
		}

		return true;
	}
	
	void FillAttributes(FManagedArrayCollection& Collection, const FName Group, const TSharedPtr<FJsonObject>& AttributesObject)
	{
		for (const auto& [Key, Attribute] : AttributesObject->Values)
		{
			const TSharedPtr<FJsonObject>& AttributeObject = Attribute->AsObject();

			const FString Type = AttributeObject->GetStringField(TEXT("type"));
			const int32 ElementCount = AttributeObject->GetNumberField(TEXT("size"));
			const bool IsArray = AttributeObject->GetBoolField(TEXT("isArray"));
			const TArray<TSharedPtr<FJsonValue>>& Values = AttributeObject->GetArrayField(TEXT("values"));

			if (IsArray)
			{
				if (ElementCount == 1)
				{
					if (Type == "int")
					{
						TManagedArray<TArray<int32>>& Attr = Collection.AddAttribute<TArray<int32>>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							TArray<int32>& CurrentAttrib = Attr[Index];
							const TArray<TSharedPtr<FJsonValue>>& AttribValues = Values[Index]->AsArray();
							for (const TSharedPtr<FJsonValue>& AttribValue : AttribValues)
							{
								CurrentAttrib.Add(AttribValue->AsNumber());
							}
						}
					}
					else if (Type == "float")
					{
						TManagedArray<TArray<float>>& Attr = Collection.AddAttribute<TArray<float>>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							TArray<float>& CurrentAttrib = Attr[Index];
							const TArray<TSharedPtr<FJsonValue>>& AttribValues = Values[Index]->AsArray();
							for (const TSharedPtr<FJsonValue>& AttribValue : AttribValues)
							{
								CurrentAttrib.Add(AttribValue->AsNumber());
							}
						}
					}
					else if (Type == "string")
					{
						// TODO: Figure out how to manage string arrays
					}
				}
				else if (ElementCount == 3)
				{
					if (Type == "int")
					{
						TManagedArray<TArray<FIntVector>>& Attr = Collection.AddAttribute<TArray<FIntVector>>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							TArray<FIntVector>& CurrentAttrib = Attr[Index];
							const TArray<TSharedPtr<FJsonValue>>& AttribValues = Values[Index]->AsArray();
							for (int ElemIndex = 0; ElemIndex < AttribValues.Num(); ElemIndex += ElementCount)
							{
								CurrentAttrib.Emplace(AttribValues[ElemIndex]->AsNumber(), AttribValues[ElemIndex + 2]->AsNumber(),
									AttribValues[ElemIndex + 1]->AsNumber());
							}
						}
					}
					else if (Type == "float")
					{
						TManagedArray<TArray<FVector3f>>& Attr = Collection.AddAttribute<TArray<FVector3f>>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							TArray<FVector3f>& CurrentAttrib = Attr[Index];
							const TArray<TSharedPtr<FJsonValue>>& AttribValues = Values[Index]->AsArray();
							for (int ElemIndex = 0; ElemIndex < AttribValues.Num(); ElemIndex += ElementCount)
							{
								CurrentAttrib.Emplace(AttribValues[ElemIndex]->AsNumber(), AttribValues[ElemIndex + 2]->AsNumber(),
									AttribValues[ElemIndex + 1]->AsNumber());
							}
						}
					}
				}
			}
			else
			{
				if (ElementCount == 1)
				{
					if (Type == "int")
					{
						TManagedArray<int32>& Attr = Collection.AddAttribute<int32>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							Attr[Index] = Values[Index]->AsNumber();
						}
					}
					else if (Type == "float")
					{
						TManagedArray<float>& Attr = Collection.AddAttribute<float>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							Attr[Index] = Values[Index]->AsNumber();
						}
					}
					else if (Type == "string")
					{
						TManagedArray<FString>& Attr = Collection.AddAttribute<FString>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							Attr[Index] = Values[Index]->AsString();
						}
					}
				}
				else if (ElementCount == 3)
				{
					if (Type == "int")
					{
						TManagedArray<FIntVector>& Attr = Collection.AddAttribute<FIntVector>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							const TArray<TSharedPtr<FJsonValue>>& VectorAttribute = Values[Index]->AsArray();
							Attr[Index] = FIntVector(VectorAttribute[0]->AsNumber(), VectorAttribute[2]->AsNumber(), VectorAttribute[1]->AsNumber());
						}
					}
					else if (Type == "float")
					{
						TManagedArray<FVector3f>& Attr = Collection.AddAttribute<FVector3f>(*Key, Group);
						for (int32 Index = 0; Index < Values.Num(); ++Index)
						{
							const TArray<TSharedPtr<FJsonValue>>& VectorAttribute = Values[Index]->AsArray();
							Attr[Index] = FVector3f(VectorAttribute[0]->AsNumber(), VectorAttribute[2]->AsNumber(), VectorAttribute[1]->AsNumber());
						}
					}
				}
			}
		}
	}

	void FillDetailsAttributes(FManagedArrayCollection& Collection, const FName Group, const TSharedPtr<FJsonObject>& AttributesObject)
	{
		for (const auto& [Key, Attribute] : AttributesObject->Values)
		{
			const TSharedPtr<FJsonObject>& AttributeObject = Attribute->AsObject();

			const FString Type = AttributeObject->GetStringField(TEXT("type"));
			const bool IsArray = AttributeObject->GetBoolField(TEXT("isArray"));

			if (IsArray)
			{
				const TArray<TSharedPtr<FJsonValue>>& Values = AttributeObject->GetArrayField(TEXT("value"));
				if (Type == "int")
				{
					TManagedArray<TArray<int32>>& Attr = Collection.AddAttribute<TArray<int32>>(*Key, Group);
					TArray<int32>& CurrentAttrib = Attr[0];
					for (const TSharedPtr<FJsonValue>& AttribValue : Values)
					{
						CurrentAttrib.Add(AttribValue->AsNumber());
					}
				}
				else if (Type == "float")
				{
					TManagedArray<TArray<float>>& Attr = Collection.AddAttribute<TArray<float>>(*Key, Group);
					TArray<float>& CurrentAttrib = Attr[0];
					for (const TSharedPtr<FJsonValue>& AttribValue : Values)
					{
						CurrentAttrib.Add(AttribValue->AsNumber());
					}
				}
				else if (Type == "string")
				{
					// TODO: Figure out how to manage string arrays
				}
			}
			else
			{
				if (Type == "int")
				{
					TManagedArray<int32>& Attr = Collection.AddAttribute<int32>(*Key, Group);
					Attr[0] = AttributeObject->GetNumberField(TEXT("value"));
				}
				else if (Type == "float")
				{
					TManagedArray<float>& Attr = Collection.AddAttribute<float>(*Key, Group);
					Attr[0] = AttributeObject->GetNumberField(TEXT("value"));
				}
				else if (Type == "string")
				{
					TManagedArray<FString>& Attr = Collection.AddAttribute<FString>(*Key, Group);
					Attr[0] = AttributeObject->GetStringField(TEXT("value"));
				}
			}
		}
	}

	void FillPlantProfilesData(FManagedArrayCollection& Collection, const FName Group, const TSharedPtr<FJsonObject>& AttributesObject)
	{
		Facades::FPlantProfileFacade Facade(Collection);

		for (const auto& [Key, Attribute] : AttributesObject->Values)
		{
			if (const TSharedPtr<FJsonObject>& AttributeObject = Attribute->AsObject();
				AttributeObject
				&& AttributeObject->HasTypedField<EJson::Array>(TEXT("value"))
				&& FStringView(Key).StartsWith(TEXT("plantProfile_")))
			{
				const TArray<TSharedPtr<FJsonValue>>& Values = AttributeObject->GetArrayField(TEXT("value"));
				TArray<float> Points;
				Points.Reserve(Values.Num());

				for (const TSharedPtr<FJsonValue>& AttribValue : Values)
				{
					if (double Point; AttribValue->TryGetNumber(Point))
					{
						Points.Add(static_cast<float>(AttribValue->AsNumber()));
					}
				}

				Facade.AddProfileEntry(Points);
			}
		}
	}

	void FillFoliageData(FManagedArrayCollection& Collection, const TSharedPtr<FJsonObject>& PrimitiveAttributesObject, const FString& InPath)
	{
		const TSharedPtr<FJsonObject>& InstancerNameObject = PrimitiveAttributesObject->GetObjectField(TEXT("instancer_name"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerNameValues = InstancerNameObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerPivotObject = PrimitiveAttributesObject->GetObjectField(TEXT("instancer_pivot"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerPivotValues = InstancerPivotObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerUpVectorObject = PrimitiveAttributesObject->GetObjectField(TEXT("instancer_UP"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerUpVectorValues = InstancerUpVectorObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerNormalVectorObject = PrimitiveAttributesObject->GetObjectField(TEXT("instancer_N"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerNormalVectorValues = InstancerNormalVectorObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerScaleObject = PrimitiveAttributesObject->GetObjectField(TEXT("instancer_scale"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerScaleValues = InstancerScaleObject->GetArrayField(TEXT("values"));

		const TSharedPtr<FJsonObject>& InstancerLFRObject = PrimitiveAttributesObject->GetObjectField(TEXT("instancer_LFR"));
		const TArray<TSharedPtr<FJsonValue>>& InstancerLFRValues = InstancerLFRObject->GetArrayField(TEXT("values"));

		// Compute unique names and number of elements for group
		int32 TotalValues = 0;
		int32 NextFoliageIndex = 0;
		TMap<FString, int32> FoliageNamesToIndex;
		TArray<FString> FoliageNames;
		for (int32 i = 0; i < InstancerNameValues.Num(); ++i)
		{
			const TArray<TSharedPtr<FJsonValue>>& InstancerNames = InstancerNameValues[i]->AsArray();
			for (int32 Index = 0; Index < InstancerNames.Num(); ++Index)
			{
				const FString& Name = InstancerNames[Index]->AsString();
				if (!FoliageNamesToIndex.Contains(Name))
				{
					FoliageNamesToIndex.Emplace(Name, NextFoliageIndex++);
					FoliageNames.Emplace(Name);
				}
			}
			TotalValues += InstancerNames.Num();
		}

		Facades::FFoliageFacade Facade(Collection, TotalValues);

		for (FString& FoliageName : FoliageNames)
		{
			FoliageName = FoliageName + '.' + FoliageName;
		}
		
		Facade.SetFoliageNames(FoliageNames);

		int32 FoliageEntryIndex = 0;
		for (int32 Id = 0; Id < InstancerNameValues.Num(); ++Id)
		{
			const TArray<TSharedPtr<FJsonValue>>& InstancerNames = InstancerNameValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& PivotPoints = InstancerPivotValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& UpVectors = InstancerUpVectorValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& NormalVectors = InstancerNormalVectorValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& Scales = InstancerScaleValues[Id]->AsArray();
			const TArray<TSharedPtr<FJsonValue>>& LFRs = InstancerLFRValues[Id]->AsArray();

			TArray<int32> FoliageEntryIds;
			for (int32 j = 0; j < InstancerNames.Num(); ++j)
			{
				const FString Name = InstancerNames[j]->AsString();
				const FVector3f PivotPoint = FVector3f(
					PivotPoints[j * 3]->AsNumber(),
					PivotPoints[(j * 3) + 2]->AsNumber(),
					PivotPoints[(j * 3) + 1]->AsNumber()) * 100.0f;
				const FVector3f UpVector = FVector3f(
					UpVectors[j * 3]->AsNumber(),
					UpVectors[(j * 3) + 2]->AsNumber(),
					UpVectors[(j * 3) + 1]->AsNumber());
				const FVector3f NormalVector = FVector3f(
					NormalVectors[j * 3]->AsNumber(),
					NormalVectors[(j * 3) + 2]->AsNumber(),
					NormalVectors[(j * 3) + 1]->AsNumber());
				const float Scale = Scales[j]->AsNumber();
				const float LengthFromRoot = LFRs[j]->AsNumber();

				Facade.SetFoliageEntry(FoliageEntryIndex, {
					.NameId = FoliageNamesToIndex[Name],
					.BranchId = Id,
					.PivotPoint = PivotPoint,
					.UpVector = UpVector,
					.NormalVector = NormalVector,
					.Scale = Scale,
					.LengthFromRoot = LengthFromRoot,
				});

				FoliageEntryIds.Add(FoliageEntryIndex);
				FoliageEntryIndex++;
			}

			Facade.SetFoliageIdsArray(Id, FoliageEntryIds);
		}
	}

	void SetFoliagePaths(FManagedArrayCollection& Collection,const FString& FilePath)
	{
		Facades::FFoliageFacade Facade(Collection);

		FString AbsolutePath = FPaths::ConvertRelativePathToFull(FilePath);
		FString PackagePath;

		if (FPackageName::TryConvertFilenameToLongPackageName(AbsolutePath, PackagePath))
		{
			TArray<FString> FoliageNames = Facade.GetFoliageNames();
			FString PackageFolder = FPackageName::GetLongPackagePath(PackagePath);
			Facade.SetFoliagePath(PackageFolder);
			for (FString& FoliageName : FoliageNames)
			{
				FoliageName = PackageFolder / FoliageName;
			}
			Facade.SetFoliageNames(FoliageNames);
		}
		
	}

	bool HasJsonFieldPath(const TSharedPtr<FJsonObject>& JsonObject, const FString& Path)
	{
		if (!JsonObject.IsValid())
		{
			return false;
		}

		TArray<FString> Keys;
		Path.ParseIntoArray(Keys, TEXT("."), true);

		TSharedPtr<FJsonObject> CurrentObject = JsonObject;

		for (int32 i = 0; i < Keys.Num(); ++i)
		{
			const FString& CurrentKey = Keys[i];
			if (i == Keys.Num() - 1)
			{
				if (!CurrentObject->HasField(CurrentKey))
				{
					return false;
				}
			}
			else
			{
				const TSharedPtr<FJsonObject>* NextObject;
				if (CurrentObject->TryGetObjectField(CurrentKey, NextObject))
				{
					CurrentObject = *NextObject;
				}
				else
				{
					return false;
				}
			}
		}

		return true;
	}

	bool LoadMegaPlantsJsonToCollection(FManagedArrayCollection& Collection, const FString& FilePath, FString& OutErrorMessage)
	{
		FString MegaPlantsData;
		if (!FFileHelper::LoadFileToString(MegaPlantsData, *FilePath))
		{
			OutErrorMessage = TEXT("Failed to open skeleton file");
			return false;
		}

		TSharedPtr<FJsonObject> LoadedData = MakeShareable(new FJsonObject);
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(MegaPlantsData), LoadedData))
		{
			OutErrorMessage = TEXT("Failed to deserialize JSON file for skeleton");
			return false;
		}

		TArray<FString> RequiredJSONPaths = {
			TEXT("points.attributes.pscale"),
			TEXT("points.positions"),
			TEXT("points.attributes.lengthFromRoot.values"),
			TEXT("points.attributes.LOD_totalPscaleGradient.values"),
			TEXT("points.attributes.budDirection.values"),
			TEXT("primitives.points"),
			TEXT("primitives.attributes.instancer_name.values"),
			TEXT("primitives.attributes.instancer_pivot.values"),
			TEXT("primitives.attributes.instancer_UP.values"),
			TEXT("primitives.attributes.instancer_scale.values"),
			TEXT("primitives.attributes.instancer_LFR.values"),
			TEXT("primitives.attributes.parents.values"),
			TEXT("primitives.attributes.children.values"),
			TEXT("primitives.attributes.branchNumber.values"),
			TEXT("globalAttributes.phyllotaxyLeaf.value")
		};

		for (const FString& Path : RequiredJSONPaths)
		{
			if (!HasJsonFieldPath(LoadedData, Path))
			{
				OutErrorMessage = FString::Printf(TEXT("Failed to find required path in JSON file: %s"), *Path);
				return false;
			}
		}

		const TSharedPtr<FJsonObject>& Points = LoadedData->GetObjectField(TEXT("points"));
		const TSharedPtr<FJsonObject>& PointAttributes = Points->GetObjectField(TEXT("attributes"));

		const TArray<TSharedPtr<FJsonValue>>& PointPositions = Points->GetArrayField(TEXT("positions"));
		const TArray<TSharedPtr<FJsonValue>>& PointScales = PointAttributes->GetObjectField(TEXT("pscale"))->GetArrayField(TEXT("values"));

		Collection.AddGroup(GroupNames::PointGroup);
		Collection.AddElements(PointPositions.Num(), GroupNames::PointGroup);

		TManagedArray<FVector3f>& PositionArray = Collection.AddAttribute<FVector3f>(AttributeNames::PointPosition, GroupNames::PointGroup);
		TManagedArray<float>& ScaleArray = Collection.AddAttribute<float>("Scale", GroupNames::PointGroup);

		for (int32 PointIndex = 0; PointIndex < PointPositions.Num(); ++PointIndex)
		{
			const TArray<TSharedPtr<FJsonValue>>& PointPosition = PointPositions[PointIndex]->AsArray();

			const FVector3f Position = FVector3f(PointPosition[0]->AsNumber(), PointPosition[2]->AsNumber(), PointPosition[1]->AsNumber()) * 100.0f;
			const float Scale = PointScales[PointIndex]->AsNumber() * 100.0f;

			PositionArray[PointIndex] = Position;
			ScaleArray[PointIndex] = Scale;
		}

		const TSharedPtr<FJsonObject>& Primitives = LoadedData->GetObjectField(TEXT("primitives"));
		const TSharedPtr<FJsonObject>& PrimitiveAttributes = Primitives->GetObjectField(TEXT("attributes"));

		const TSharedPtr<FJsonObject>& GlobalAttributes = LoadedData->GetObjectField(TEXT("globalAttributes"));

		const TArray<TSharedPtr<FJsonValue>>& PrimitivesPoints = Primitives->GetArrayField(TEXT("points"));

		Collection.AddGroup(GroupNames::BranchGroup);
		Collection.AddElements(PrimitivesPoints.Num(), GroupNames::BranchGroup);

		TManagedArray<TArray<int32>>& PointsArray = Collection.AddAttribute<TArray<int32>>(
			PV::AttributeNames::BranchPoints, PV::GroupNames::BranchGroup);

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitivesPoints.Num(); ++PrimitiveIndex)
		{
			const TArray<TSharedPtr<FJsonValue>>& PrimitivePoints = PrimitivesPoints[PrimitiveIndex]->AsArray();
			for (const TSharedPtr<FJsonValue>& PrimitivePoint : PrimitivePoints)
			{
				PointsArray[PrimitiveIndex].Add(PrimitivePoint->AsNumber());
			}
		}

		Collection.AddGroup(GroupNames::DetailsGroup);
		Collection.AddElements(1, GroupNames::DetailsGroup);

		// Please note that LFR (length from root) for both skeleton points and foliage points are not being scaled by
		// 100 at the moment, which is OK for now since they're only being compared relative to each other for now,
		// however if they're used in any computation in the future in any meaningful way where other quantities are
		// involved they will end up giving erroneous results.
		FillAttributes(Collection, GroupNames::PointGroup, PointAttributes);
		FillAttributes(Collection, GroupNames::BranchGroup, PrimitiveAttributes);
		FillDetailsAttributes(Collection, GroupNames::DetailsGroup, GlobalAttributes);
		FillPlantProfilesData(Collection, GroupNames::PlantProfilesGroup, GlobalAttributes);

		FillFoliageData(Collection, PrimitiveAttributes, FilePath);

		Facades::FMetaInfoFacade MetaInfoFacade(Collection);
		MetaInfoFacade.CreateGuid(FilePath);

		OutErrorMessage = "";
		return true;
	}

	void AddGrowthMissingData(FManagedArrayCollection& Collection)
	{
		if (!PV::FPointScaleAttribute::HasAttribute(Collection))
		{
			auto PointScaleAttributes = PV::FPointScaleAttribute::AddAttribute(Collection, 0);
			auto BudLateralMeristemAttribute = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
			if (BudLateralMeristemAttribute)
			{
				for (int32 i = 0; i < BudLateralMeristemAttribute.Num(); ++i)
				{
					PointScaleAttributes[i] = BudLateralMeristemAttribute[i].LateralMeristem * 100.f;
				}
			}
		}

		if (!PV::FPointLengthFromRootAttribute::HasAttribute(Collection)
			|| !PV::FPointLengthFromSeedAttribute::HasAttribute(Collection))
		{
			auto PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::AddAttribute(Collection);
			auto PointLengthFromSeedAttribute = PV::FPointLengthFromSeedAttribute::AddAttribute(Collection);

			auto BudLateralMeristemAttribute = PV::FBudLateralMeristemAttribute::FindAttribute(Collection);
			if (BudLateralMeristemAttribute)
			{
				for (int32 i = 0; i < BudLateralMeristemAttribute.Num(); ++i)
				{
					PointLengthFromRootAttribute[i] = BudLateralMeristemAttribute[i].RootDistance;
					PointLengthFromSeedAttribute[i] = BudLateralMeristemAttribute[i].RootDistance;
				}
			}
			else
			{
				PV::AttributesHelper::ComputeLengthFromRoot(Collection);
			}
		}

		if (!PV::FBranchHierarchyNumberAttribute::HasAttribute(Collection))
		{
			PV::FBranchHierarchyNumberAttribute::AddAttribute(Collection);
			PV::AttributesHelper::ComputeBranchHierarchyNumbers(Collection);
		}

		if (!PV::FPointSeedPScaleAttribute::HasAttribute(Collection))
		{
			PV::FPointSeedPScaleAttribute::AddAttribute(Collection, 1.f);
		}

		if (!PV::FPointSeedPScaleRatioAttribute::HasAttribute(Collection))
		{
			PV::FPointSeedPScaleRatioAttribute::AddAttribute(Collection, 1.f);
		}
	}

	bool LoadGrowthDataJsonToCollection(FManagedArrayCollection& Collection, const FString& FilePath, FString& OutErrorMessage)
	{
		FString MegaPlantsData;
		if (!FFileHelper::LoadFileToString(MegaPlantsData, *FilePath))
		{
			OutErrorMessage = TEXT("Failed to open skeleton file");
			return false;
		}

		TSharedPtr<FJsonObject> LoadedData = MakeShareable(new FJsonObject);
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(MegaPlantsData), LoadedData))
		{
			OutErrorMessage = TEXT("Failed to deserialize JSON file for skeleton");
			return false;
		}

		TArray<FString> RequiredJSONPaths = {
			TEXT("points.positions"),
			TEXT("points.attributes.budDirection.values"),
			TEXT("primitives.points"),
			TEXT("primitives.attributes.parents.values"),
			TEXT("primitives.attributes.children.values"),
			TEXT("primitives.attributes.branchNumber.values"),
		};

		for (const FString& Path : RequiredJSONPaths)
		{
			if (!HasJsonFieldPath(LoadedData, Path))
			{
				OutErrorMessage = FString::Printf(TEXT("Failed to find required path in JSON file: %s"), *Path);
				return false;
			}
		}

		const TSharedPtr<FJsonObject>& Points = LoadedData->GetObjectField(TEXT("points"));
		const TSharedPtr<FJsonObject>& PointAttributes = Points->GetObjectField(TEXT("attributes"));
		const TArray<TSharedPtr<FJsonValue>>& PointPositions = Points->GetArrayField(TEXT("positions"));

		Collection.AddGroup(GroupNames::PointGroup);
		Collection.AddElements(PointPositions.Num(), GroupNames::PointGroup);

		TManagedArray<FVector3f>& PositionArray = Collection.AddAttribute<FVector3f>(AttributeNames::PointPosition, GroupNames::PointGroup);

		for (int32 PointIndex = 0; PointIndex < PointPositions.Num(); ++PointIndex)
		{
			const TArray<TSharedPtr<FJsonValue>>& PointPosition = PointPositions[PointIndex]->AsArray();
			const FVector3f Position = FVector3f(PointPosition[0]->AsNumber(), PointPosition[2]->AsNumber(), PointPosition[1]->AsNumber()) * 100.0f;
			PositionArray[PointIndex] = Position;
		}

		const TSharedPtr<FJsonObject>& Primitives = LoadedData->GetObjectField(TEXT("primitives"));
		const TSharedPtr<FJsonObject>& PrimitiveAttributes = Primitives->GetObjectField(TEXT("attributes"));
		const TArray<TSharedPtr<FJsonValue>>& PrimitivesPoints = Primitives->GetArrayField(TEXT("points"));

		Collection.AddGroup(GroupNames::BranchGroup);
		Collection.AddElements(PrimitivesPoints.Num(), GroupNames::BranchGroup);

		TManagedArray<TArray<int32>>& PointsArray = Collection.AddAttribute<TArray<int32>>(
			PV::AttributeNames::BranchPoints, PV::GroupNames::BranchGroup);

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitivesPoints.Num(); ++PrimitiveIndex)
		{
			const TArray<TSharedPtr<FJsonValue>>& PrimitivePoints = PrimitivesPoints[PrimitiveIndex]->AsArray();
			for (const TSharedPtr<FJsonValue>& PrimitivePoint : PrimitivePoints)
			{
				PointsArray[PrimitiveIndex].Add(PrimitivePoint->AsNumber());
			}
		}

		Collection.AddGroup(GroupNames::DetailsGroup);
		Collection.AddElements(1, GroupNames::DetailsGroup);

		FillAttributes(Collection, GroupNames::PointGroup, PointAttributes);
		FillAttributes(Collection, GroupNames::BranchGroup, PrimitiveAttributes);

		TManagedArray<float>& ScaleArray = Collection.AddAttribute<float>("Scale", GroupNames::PointGroup);
		FBudLateralMeristemAttributeConstView BudLateralMeristem = FBudLateralMeristemAttribute::GetAttribute(Collection);

		for (int32 PointIndex = 0; PointIndex < PointPositions.Num(); ++PointIndex)
		{
			float Scale = BudLateralMeristem[PointIndex].LateralMeristem;
			
			Scale *= 100.0f;

			ScaleArray[PointIndex] = Scale;
		}

		// We have had issues with incorrectly generated budNumbers and branchSourceBudNumbers in the imported json, 
		// so always re-compute the bud numbers as it's a relatively cheap operation.
		PV::AttributesHelper::ComputeBudNumbers(Collection);
		
		AddGrowthMissingData(Collection);
		
		Facades::FMetaInfoFacade MetaInfoFacade(Collection);
		MetaInfoFacade.CreateGuid(FilePath);

		OutErrorMessage = "";
		
		return true;
	}

	TSharedPtr<FJsonObject> LoadMetaFileIntoJsonObject(const FString& FilePath, FString& OutErrorMessage)
	{
		FString MetaData;
		if (!FFileHelper::LoadFileToString(MetaData, *FilePath))
		{
			OutErrorMessage = TEXT("Failed to open skeleton file");
			return nullptr;
		}

		TSharedPtr<FJsonObject> LoadedData = MakeShareable(new FJsonObject);
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(MetaData), LoadedData))
		{
			OutErrorMessage = TEXT("Failed to deserialize JSON file for skeleton");
			return nullptr;
		}

		TArray<FString> RequiredJSONPaths = {
			TEXT("apix_uv"),
			TEXT("apix_texture_channels")
		};

		for (const FString& Path : RequiredJSONPaths)
		{
			if (!HasJsonFieldPath(LoadedData, Path))
			{
				OutErrorMessage = FString::Printf(TEXT("Failed to find required path in JSON file: %s"), *Path);
				return nullptr;
			}
		}

		OutErrorMessage = "";
		return LoadedData;
	}

	bool LoadMetaJsonToCollection(FManagedArrayCollection& Collection, TSharedPtr<FJsonObject> LoadedData)
	{
		
		const TArray<TSharedPtr<FJsonValue>>& MaterialURanges = LoadedData->GetArrayField(TEXT("apix_uv"));

		TArray<FVector2f> URanges;
		URanges.Reserve(MaterialURanges.Num());
		
		for (int i=0; (i + 1) < MaterialURanges.Num(); i+= 2)
		{
			URanges.Add(FVector2f(MaterialURanges[i]->AsNumber() , MaterialURanges[i+1]->AsNumber()));
		}

		Facades::FBranchFacade Facade = Facades::FBranchFacade(Collection);
		Facade.SetTrunkURange(URanges);
		
		return true;
	}

	bool LoadProfileData(FString InPath, TArray<FPVPlantProfile>& OutProfileData, FString& OutErrorMessage)
	{
		TSharedPtr<FJsonObject> OutData = MakeShareable(new FJsonObject);
		
		OutProfileData.Empty();

		if (LoadJSON(InPath, OutData, OutErrorMessage))
		{
			if (OutData->HasTypedField(TEXT("PlantProfiles"), EJson::Array))
			{
				TArray<FPVPlantProfile> Profiles;
				
				const TArray<TSharedPtr<FJsonValue>>& ProfilesArray = OutData->GetArrayField(TEXT("PlantProfiles"));

				int Index = 0;

				for (TSharedPtr<FJsonValue> ProfileJSONValue : ProfilesArray)
				{
					if (ProfileJSONValue->Type == EJson::Object)
					{
						FPVPlantProfile Profile;
						
						const TSharedPtr<FJsonObject> ProfilesObject = ProfileJSONValue->AsObject();

						if (!ProfilesObject->TryGetStringField(TEXT("Name"), Profile.Name))
						{
							OutErrorMessage = FString::Printf(TEXT("ProfileInfo Doesnt contain valid name for Profile[%i]"), Index);
							return false;
						}

						if (ProfilesObject->HasTypedField<EJson::Array>(TEXT("Points")))
						{
							const TArray<TSharedPtr<FJsonValue>>& Values = ProfilesObject->GetArrayField(TEXT("Points"));
						
							Profile.Points.Reserve(Values.Num());

							for (const TSharedPtr<FJsonValue>& AttribValue : Values)
							{
								if (double Point; AttribValue->TryGetNumber(Point))
								{
									Profile.Points.Add(static_cast<float>(AttribValue->AsNumber()));
								}
							}	
						}
						else
						{
							OutErrorMessage = FString::Printf(TEXT("ProfileInfo doesnt contain valid points for Profile[%i]"), Index);
							return false;
						}

						Profiles.Add(Profile);
					}

					Index++;
				}

				if (Profiles.Num())
				{
					OutProfileData = Profiles;
				}
			}
			else
			{
				OutErrorMessage = FString::Printf(TEXT("Unable to load profile data"));
				
				return false;	
			}
		}
		else
		{
			return false;
		}

		return true;
	}
}
