// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FPVPlantProfile;
struct FManagedArrayCollection;
class FJsonObject;

namespace PV::JSON
{
	void FillAttributes(FManagedArrayCollection& Collection, const FName Group, const TSharedPtr<FJsonObject>& AttributesObject);

	void FillDetailsAttributes(FManagedArrayCollection& Collection, const FName Group, const TSharedPtr<FJsonObject>& AttributesObject);

	void FillPlantProfilesData(FManagedArrayCollection& Collection, const FName Group, const TSharedPtr<FJsonObject>& AttributesObject);

	void FillFoliageData(FManagedArrayCollection& Collection, const TSharedPtr<FJsonObject>& PrimitiveAttributesObject, const FString& InPath);

	void SetFoliagePaths(FManagedArrayCollection& Collection,const FString& FilePath);

	bool HasJsonFieldPath(const TSharedPtr<FJsonObject>& JsonObject, const FString& Path);
	
	bool LoadMegaPlantsJsonToCollection(FManagedArrayCollection& Collection, const FString& FilePath, FString& OutErrorMessage);

	bool LoadGrowthDataJsonToCollection(FManagedArrayCollection& Collection, const FString& FilePath, FString& OutErrorMessage);

	TSharedPtr<FJsonObject> LoadMetaFileIntoJsonObject(const FString& FilePath, FString& OutErrorMessage);

	bool LoadMetaJsonToCollection(FManagedArrayCollection& Collection, TSharedPtr<FJsonObject> LoadedData);

	bool LoadProfileData(FString FilePath, TArray<FPVPlantProfile>& OutProfileData, FString& OutErrorMessage);
}
