// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVParametricDistributionHelper.h"

#include "PVAttributesHelper.h"
#include "Facades/PVBranchFacade.h"
#include "Helpers/PVUtilities.h"

float PV::ParametricDistributionHelper::IndexedRandom01(const int32 Seed)
{
	const FRandomStream LocalRandom(Seed);
	return LocalRandom.GetFraction();
}

float PV::ParametricDistributionHelper::IndexedRandomRange(const int32 Seed, const float MinValue, const float MaxValue)
{
	return FMath::Lerp(MinValue, MaxValue, IndexedRandom01(Seed));
}

int32 PV::ParametricDistributionHelper::IndexedRandomIntInclusive(const int32 Seed, const int32 MinValue, const int32 MaxValue)
{
	if (MinValue >= MaxValue)
	{
		return MinValue;
	}

	const FRandomStream LocalRandom(Seed);
	return LocalRandom.RandRange(MinValue, MaxValue);
}

TArray<float> PV::ParametricDistributionHelper::RemapPScalesToGradientRange(
	FBranchPointsAttributeConstView BranchPointsAttribute,
	FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
	FPointScaleAttributeConstView PointScaleAttribute,
	float RelativeStart,
	float RelativeEnd)
{
	float MinScale = TNumericLimits<float>::Max();
	float MaxScale = TNumericLimits<float>::Lowest();

	for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];

		if (BranchPoints.Num() < 2)
		{
			continue;
		}

		const float LFRofRoot = PointLengthFromRootAttribute[BranchPoints[0]];
		const float LFRofTip  = PointLengthFromRootAttribute[BranchPoints.Last()];

		if (LFRofTip <= LFRofRoot + KINDA_SMALL_NUMBER)
		{
			continue;
		}

		for (int32 PointIndex : BranchPoints)
		{
			const float BranchGradient = FMath::GetMappedRangeValueClamped(
				FVector2f(LFRofRoot, LFRofTip),
				FVector2f(0.0f, 1.0f),
				PointLengthFromRootAttribute[PointIndex]);

			if (BranchGradient >= RelativeStart && BranchGradient <= RelativeEnd)
			{
				const float Scale = PointScaleAttribute[PointIndex];
				MinScale = FMath::Min(MinScale, Scale);
				MaxScale = FMath::Max(MaxScale, Scale);
			}
		}
	}

	TArray<float> NormalizedScales;
	NormalizedScales.Reserve(PointScaleAttribute.Num());

	for (int32 PointIndex = 0; PointIndex < PointScaleAttribute.Num(); ++PointIndex)
	{
		float NormalizedScale = 0.0f;

		if (MaxScale > MinScale)
		{
			NormalizedScale = FMath::GetMappedRangeValueClamped(
				FVector2f(MinScale, MaxScale),
				FVector2f(0.0f, 1.0f),
				FMath::Clamp(PointScaleAttribute[PointIndex], MinScale, MaxScale));
		}

		NormalizedScales.Add(NormalizedScale);
	}

	return NormalizedScales;
}

float PV::ParametricDistributionHelper::ComputeAttachmentScale(
	const int32 BudIndex,
	const int32 RandomSeed,
	const float PlantGradient,
	const float BranchGradient,
	const float LengthRatio,
	const FParametricSettings::FScaleSettings& ScaleSettings, 
	const float RelativeStart, 
	const float RelativeEnd)
{
	float RampLookup =
		(ScaleSettings.ScaleRampBasis == FParametricSettings::EDistributionBasis::Branch)
		? BranchGradient
		: PlantGradient;

	RampLookup = FMath::GetMappedRangeValueClamped(
			FVector2f(RelativeStart, RelativeEnd),
			FVector2f(0.0f, 1.0f),
			RampLookup);
	float ScaleRampValue = ScaleSettings.ScaleRamp->GetRichCurveConst()->Eval(RampLookup);

	if (ScaleSettings.ScaleRampBasis == FParametricSettings::EDistributionBasis::Branch)
	{
		const float ModifiedLengthRatio = FMath::Lerp(1,LengthRatio, ScaleSettings.BranchScaleImpact);
		
		ScaleRampValue = FMath::GetMappedRangeValueClamped(
			FVector2f(0.0f, 1.0f),
			FVector2f(0.0f, ModifiedLengthRatio),
			ScaleRampValue);
	}
	const uint32 SeedHash = HashCombine(GetTypeHash(BudIndex), GetTypeHash(RandomSeed));
	const float RandomScaleFactor = IndexedRandomRange(
		SeedHash,
		ScaleSettings.RandomizeScaleMinimum,
		ScaleSettings.RandomizeScaleMaximum);

	return ScaleRampValue * RandomScaleFactor * ScaleSettings.BaseScale;
}

void PV::ParametricDistributionHelper::ComputeAttachmentPoints(
	FManagedArrayCollection& OutCollection,
	const FNormalizedPointCaches& NormalizedPointCaches,
	const FParametricSettings::FSpacingSettings& SpacingSettings,
	const FParametricSettings::FPhyllotaxySettings& PhyllotaxySettings,
	const FParametricSettings::FAngleSettings& AngleSettings,
	const FParametricSettings::FScaleSettings& ScaleSettings,
	const FPVDistributionVectorParams& VectorSettings,
	const int32 RandomSeed,
	float TrunkOffset,
	TArray<DistributionHelper::FAttachmentPoint>& OutAttachmentPoints)
{
	auto BranchParentNumberAttribute = FBranchParentNumberAttribute::GetAttribute(OutCollection);
	auto BranchChildrenAttribute = FBranchChildrenAttribute::GetAttribute(OutCollection);
	auto BranchNumberAttribute = FBranchNumberAttribute::GetAttribute(OutCollection);
	auto BranchPointsAttribute = FBranchPointsAttribute::GetAttribute(OutCollection);
	auto PointPositionAttribute = FPointPositionAttribute::GetAttribute(OutCollection);
	auto BudDirectionAttribute = FBudDirectionAttribute::GetAttribute(OutCollection);
	auto BudDevelopmentAttribute = FBudDevelopmentAttribute::GetAttribute(OutCollection);
	auto PointLengthFromRootAttribute = FPointLengthFromRootAttribute::GetAttribute(OutCollection);
	auto PointScaleAttribute = FPointScaleAttribute::GetAttribute(OutCollection);

	if (!ValidateAttributeCollection(
		BranchParentNumberAttribute,
		BranchChildrenAttribute,
		BranchNumberAttribute,
		BranchPointsAttribute,
		PointPositionAttribute,
		BudDirectionAttribute,
		BudDevelopmentAttribute,
		PointLengthFromRootAttribute,
		PointScaleAttribute
	))
	{
		return;
	}

	const int32 NumOfBranches = BranchPointsAttribute.Num();

	const auto [
			PhyllotaxyFormationLeaf,
			MinBudsLeaf,
			MaxBudsLeaf,
			bResetPhyllotaxyLeaf,
			PhyllotaxyOffsetLeaf] =
		PhyllotaxySettingsHelper::ResolvePhyllotaxy(PhyllotaxySettings);
	
	// Only need to do this temporarily since legacy skeleton has inconsistent data in children
	// TODO: Remove this once support for legacy skeletons is dropped
	PV::Facades::FBranchFacade BranchFacadeOut(OutCollection);
	BranchFacadeOut.RecomputeBranchChildren();
	BranchFacadeOut.RecomputeBranchParents();

	auto PointPlantGradientAttribute = FPointPlantGradientAttribute::AddAttribute(OutCollection);
	AttributesHelper::ComputePointPlantGradient(
		{
			PointPlantGradientAttribute,
			BranchPointsAttribute,
			BranchParentNumberAttribute,
			BranchNumberAttribute,
			BranchChildrenAttribute
		});

	TMap<int32, float> BranchIndicesToBranchLengths = ComputeBranchLengths(PointLengthFromRootAttribute, BranchPointsAttribute);
	float MaxBranchLength = -1.0f;
	float AverageBranchLength = 0.0f;

	for (const auto IndexLengthPair : BranchIndicesToBranchLengths)
	{
		const float BranchLength = IndexLengthPair.Value;
		MaxBranchLength = FMath::Max(BranchLength, MaxBranchLength);
		AverageBranchLength = AverageBranchLength + BranchLength;
	}

	check(MaxBranchLength > KINDA_SMALL_NUMBER);
	if (MaxBranchLength < KINDA_SMALL_NUMBER)
	{
		return;
	}

	AverageBranchLength = AverageBranchLength / BranchIndicesToBranchLengths.Num();
	AverageBranchLength = FMath::Lerp(AverageBranchLength, MaxBranchLength, 0.33f);
	
	int32 GlobalBudIndex = 0;

	for (int32 BranchIndex = 0; BranchIndex < NumOfBranches; BranchIndex++)
	{
		const int32 Generation = PV::AttributesHelper::GetBranchGeneration(
			BudDevelopmentAttribute,
			BranchPointsAttribute,
			BranchParentNumberAttribute,
			BranchIndex
		);

		if (SpacingSettings.LimitStartGeneration && Generation < SpacingSettings.StartGeneration)
		{
			continue;
		}

		if (SpacingSettings.LimitEndGeneration && Generation > SpacingSettings.EndGeneration)
		{
			continue;
		}

		const float LengthRatio = BranchIndicesToBranchLengths[BranchIndex] / MaxBranchLength;
		int32 LoopNumber = SpacingSettings.BranchDensity;
		if (SpacingSettings.SpacingBasis == FParametricSettings::EDistributionBasis::Branch)
		{
			LoopNumber = static_cast<int32>(LoopNumber * LengthRatio);
		}
		else if (SpacingSettings.SpacingBasis == FParametricSettings::EDistributionBasis::Plant)
		{
			LoopNumber = static_cast<int32>(LoopNumber * (AverageBranchLength / MaxBranchLength));
		}

		LoopNumber = FMath::Max(LoopNumber, 1);

		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];

		if (BranchPoints.Num() < 2)
		{
			continue;
		}
		
		TArray<TArray<FVector3f>> BranchBudDirPerPoint;                                                                                                            
		BranchBudDirPerPoint.Reserve(BranchPoints.Num());

		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			BranchBudDirPerPoint.Add(PV::AttributesHelper::GetBudDirection(BudDirectionAttribute, BranchPointsAttribute, BranchParentNumberAttribute, PointPositionAttribute, BranchIndex, i));
		}

#if DO_ENSURE
		for (int32 i = 0; i < BranchPoints.Num() - 1; ++i)
		{
			ensureMsgf(
				PointLengthFromRootAttribute[BranchPoints[i]] <= PointLengthFromRootAttribute[BranchPoints[i + 1]],
				TEXT("BranchPoints are not ordered root-to-tip at index %d for BranchIndex %d"), i, BranchIndex);
		}
#endif

		const int32 FirstPointIndex = BranchPoints[0];
		const int32 TipPointIndex   = BranchPoints.Last();
		const float PlantGradientAtRoot = 1.0f - PointPlantGradientAttribute[FirstPointIndex];
		const float LFRofRoot = PointLengthFromRootAttribute[FirstPointIndex];
		const float LFRofTip = PointLengthFromRootAttribute[TipPointIndex];

		FVector3f PreviousAcceptedPosition = PointPositionAttribute[FirstPointIndex];

		const PV::FBudDirectionConstView InitialBudDirection(BranchBudDirPerPoint[0]);
		const FVector3f InitialApicalDirection = InitialBudDirection.Apical.GetSafeNormal();
		const FVector3f InitialUpVector = InitialBudDirection.UpVector.GetSafeNormal();

		FVector3f InitialAxillaryDirection = InitialBudDirection.Axillary.GetSafeNormal();
		if (bResetPhyllotaxyLeaf)
		{
			InitialAxillaryDirection = FVector3f::CrossProduct(InitialApicalDirection, InitialUpVector).GetSafeNormal();
		}

		if (InitialAxillaryDirection.IsNearlyZero())
		{
			FVector Axis1, Axis2;
			FVector(InitialApicalDirection).FindBestAxisVectors(Axis1, Axis2);
			InitialAxillaryDirection = FVector3f(Axis2);
		}

		if (FMath::Abs(PhyllotaxyOffsetLeaf) > KINDA_SMALL_NUMBER)
		{
			const FQuat4f OffsetQuat(InitialApicalDirection, FMath::DegreesToRadians(PhyllotaxyOffsetLeaf));
			InitialAxillaryDirection = OffsetQuat.RotateVector(InitialAxillaryDirection).GetSafeNormal();
		}

		int32 AxillaryRotationIteration = 0;
		
		for (int32 j = 0; j < LoopNumber; j++)
		{
			float BaseValue = static_cast<float>(j) / FMath::Max(LoopNumber - 1, 1);
			float AttachmentPointLengthFromRoot = 0.0f;
			FVector3f Pos(0.0f, 0.0f, 0.0f);
			int32 PointIndexA = INDEX_NONE;
			int32 PointIndexB = INDEX_NONE;
			int32 PreviousBranchPointIndex = INDEX_NONE;
			int32 NextBranchPointIndex = INDEX_NONE;
			
			float PointAlpha = 0.0f;
			
			float PlantGradientRemapped = BaseValue;
			PlantGradientRemapped = SpacingSettings.SpacingRamp->GetRichCurveConst()->Eval(PlantGradientRemapped);
			PlantGradientRemapped = FMath::GetMappedRangeValueClamped(
				FVector2f(0.0f, 1.0f),
				FVector2f(SpacingSettings.RelativeStart, SpacingSettings.RelativeEnd),
				PlantGradientRemapped);
			
			const float PlantGradientNormalized = FMath::GetMappedRangeValueClamped(
				FVector2f(SpacingSettings.RelativeStart, SpacingSettings.RelativeEnd),
				FVector2f(0, 1),
				PlantGradientRemapped);
			
			float BranchGradientRemapped = SpacingSettings.SpacingRamp->GetRichCurveConst()->Eval(BaseValue);
			BranchGradientRemapped = FMath::GetMappedRangeValueClamped(
				FVector2f(0.0f, 1.0f),
				FVector2f(SpacingSettings.RelativeStart, SpacingSettings.RelativeEnd),
				BranchGradientRemapped);
			
			const float BranchGradientNormalized = FMath::GetMappedRangeValueClamped(
				FVector2f(SpacingSettings.RelativeStart, SpacingSettings.RelativeEnd),
				FVector2f(0, 1),
				BranchGradientRemapped);
			
			if (SpacingSettings.SpacingBasis == FParametricSettings::EDistributionBasis::Plant)
			{
				if (PlantGradientRemapped < PlantGradientAtRoot)
				{
					continue;
				}

				Pos = SampleVectorByFloat(
					PlantGradientRemapped,
					BranchPoints,
					PointPlantGradientAttribute,
					PointPositionAttribute);

				AttachmentPointLengthFromRoot = SampleFloatByFloat(
					PlantGradientRemapped,
					BranchPoints,
					PointPlantGradientAttribute,
					PointLengthFromRootAttribute);
				
				float BranchGradientForPoint = FMath::GetMappedRangeValueClamped(
					FVector2f(LFRofRoot, LFRofTip),
					FVector2f(0.0f, 1.0f),
					AttachmentPointLengthFromRoot);
				
				FindPointByNormalizedLengthFromRoot(
					BranchGradientForPoint,
					BranchPoints,
					PointLengthFromRootAttribute,
					LFRofRoot,
					LFRofTip,
					PreviousBranchPointIndex,
					NextBranchPointIndex,
					PointAlpha);
			}
			else // SpacingBasis is Branch
			{
				if (SpacingSettings.BranchDensity == 1)
				{
					BranchGradientRemapped = SpacingSettings.RelativeEnd;
					BaseValue = 1.0f;
				}

				if (!FindPointByNormalizedLengthFromRoot(
					BranchGradientRemapped,
					BranchPoints,
					PointLengthFromRootAttribute,
					LFRofRoot,
					LFRofTip,
					PreviousBranchPointIndex,
					NextBranchPointIndex,
					PointAlpha))
				{
					continue;
				}
				PointIndexA = BranchPoints[PreviousBranchPointIndex];
				PointIndexB = BranchPoints[NextBranchPointIndex];

				const FVector3f P0 = PointPositionAttribute[PointIndexA];
				const FVector3f P1 = PointPositionAttribute[PointIndexB];
				Pos = FMath::Lerp(P0, P1, PointAlpha);

				AttachmentPointLengthFromRoot = FMath::GetMappedRangeValueClamped(
					FVector2f(0.0f, 1.0f),
					FVector2f(LFRofRoot, LFRofTip),
					BranchGradientRemapped);
			}
			
			if (PreviousBranchPointIndex == INDEX_NONE || NextBranchPointIndex == INDEX_NONE)
			{
				continue;
			}

			// When the sample lands exactly on the branch's root point we intentionally
			// skip emitting a bud — but advance the phyllotaxy rotation phase so the
			// remaining buds on this branch keep their intended whorl/spiral pattern.
			const bool bSampleIsOnBranchRoot = PreviousBranchPointIndex == 0 && FMath::IsNearlyZero(PointAlpha);
			if (bSampleIsOnBranchRoot)
			{
				++AxillaryRotationIteration;
				continue;
			}
			
			PointIndexA = BranchPoints[PreviousBranchPointIndex];
			PointIndexB = BranchPoints[NextBranchPointIndex];

			const float PlantGradient0 = 1.0f - PointPlantGradientAttribute[PointIndexA];
			const float PlantGradient1 = 1.0f - PointPlantGradientAttribute[PointIndexB];
			float PlantGradient = FMath::Lerp(PlantGradient0, PlantGradient1, PointAlpha);
			
			const float BranchGradient0 = AttributesHelper::GetBranchPointBranchGradient(PointLengthFromRootAttribute, BranchPointsAttribute, BranchIndex, PreviousBranchPointIndex);
			const float BranchGradient1 = AttributesHelper::GetBranchPointBranchGradient(PointLengthFromRootAttribute, BranchPointsAttribute, BranchIndex, NextBranchPointIndex);
			float BranchGradient = FMath::Lerp(BranchGradient0, BranchGradient1, PointAlpha);

			const float PreviousDistance = FVector3f::Distance(Pos, PreviousAcceptedPosition);
			if (PreviousDistance < 0.001f) // Prevent placement too close together
			{
				continue;
			}
			PreviousAcceptedPosition = Pos;

			// Hash-mix loop vars so adjacent (BranchIndex, j) positions cross RandRange buckets instead of clumping.
			const uint32 SeedHash = HashCombine(HashCombine(GetTypeHash(BranchIndex), GetTypeHash(j)), GetTypeHash(RandomSeed));
			int32 NumBuds = IndexedRandomIntInclusive(
				static_cast<int32>(SeedHash),
				MinBudsLeaf,
				MaxBudsLeaf);

			NumBuds = FMath::Max(NumBuds, MinBudsLeaf);

			const bool bIsTipInstance =
				FMath::IsNearlyEqual(AttachmentPointLengthFromRoot, LFRofTip, 0.001f) &&
				(BranchGradientRemapped >= 0.999f || FMath::IsNearlyEqual(BranchGradientRemapped, SpacingSettings.RelativeEnd, 0.001f));
			
			if (bIsTipInstance && PhyllotaxySettings.bSingleBudTip)
			{
				NumBuds = 1;
			}

			const FBudDirectionConstView BudDirA(BranchBudDirPerPoint[PreviousBranchPointIndex]);
			const FBudDirectionConstView BudDirB(BranchBudDirPerPoint[NextBranchPointIndex]);

			const FVector3f ApicalDirection = FMath::Lerp(BudDirA.Apical, BudDirB.Apical, PointAlpha);
			const FVector3f BudDirectionUp = FMath::Lerp(BudDirA.UpVector, BudDirB.UpVector, PointAlpha);
			const FVector3f BudLightOptimalVector = FMath::Lerp(BudDirA.LightOptimal, BudDirB.LightOptimal, PointAlpha);
			const FVector3f BudLightSubOptimalVector = FMath::Lerp(BudDirA.LightSubOptimal, BudDirB.LightSubOptimal, PointAlpha);
			
			const float SitePhaseDegrees = PhyllotaxyFormationLeaf * static_cast<float>(AxillaryRotationIteration);

			const FQuat4f SitePhaseQuat(InitialApicalDirection, FMath::DegreesToRadians(SitePhaseDegrees));
			FVector3f SiteAxillaryDirection = SitePhaseQuat.RotateVector(InitialAxillaryDirection).GetSafeNormal();

			const FQuat4f ApicalCorrection = FQuat4f::FindBetweenNormals(
				InitialApicalDirection,
				ApicalDirection.GetSafeNormal());

			SiteAxillaryDirection = ApicalCorrection.RotateVector(SiteAxillaryDirection).GetSafeNormal();

			for (int32 k = 0; k < NumBuds; ++k)
			{
				const float Scale = ComputeAttachmentScale(
					GlobalBudIndex,
					RandomSeed,
					PlantGradient,
					BranchGradientRemapped,
					LengthRatio,
					ScaleSettings, 
					SpacingSettings.RelativeStart, 
					SpacingSettings.RelativeEnd);

				FVector3f UpDirection;
				FVector3f NormalDirection;

				ComputeRotationAnglesForAttachment(
					ApicalDirection,
					BudDirectionUp,
					SiteAxillaryDirection,
					PlantGradient,
					BranchGradientRemapped,
					j,
					k,
					NumBuds,
					GlobalBudIndex,
					RandomSeed,
					AngleSettings,
					SpacingSettings,
					UpDirection,
					NormalDirection);

				ConditionPicker::FPointConditionSample Sample;
				Sample.UpAlignment = UpAlignmentNormalized(ApicalDirection);
				Sample.Tip = Tip(bIsTipInstance);

				if (PointIndexA != INDEX_NONE && PointIndexB != INDEX_NONE)
				{
					Sample.Light      = FMath::Lerp(NormalizedPointCaches.Light[PointIndexA],       NormalizedPointCaches.Light[PointIndexB],       PointAlpha);
					Sample.Health     = FMath::Lerp(NormalizedPointCaches.Health[PointIndexA],      NormalizedPointCaches.Health[PointIndexB],      PointAlpha);
					Sample.Scale      = FMath::Lerp(NormalizedPointCaches.PScales[PointIndexA],     NormalizedPointCaches.PScales[PointIndexB],     PointAlpha);
					Sample.Height     = FMath::Lerp(NormalizedPointCaches.Height[PointIndexA],      NormalizedPointCaches.Height[PointIndexB],      PointAlpha);
					Sample.Generation = FMath::Lerp(NormalizedPointCaches.Generation[PointIndexA],  NormalizedPointCaches.Generation[PointIndexB],  PointAlpha);
				}
				else
				{
					Sample.Light      = 0.0f;
					Sample.Health     = 0.0f;
					Sample.Scale      = 0.0f;
					Sample.Height     = 0.0f;
					Sample.Generation = 0.0f;
				}
				
				// Update Vector Settings
				DistributionVectorUtils::FPointVectorData PointVectorData;
				PointVectorData.ApicalDirection = ApicalDirection;
				PointVectorData.UpVector = BudDirectionUp;
				PointVectorData.bIsTip = bIsTipInstance;
				PointVectorData.LightOptimal = BudLightOptimalVector;
				PointVectorData.LightSubOptimal = BudLightSubOptimalVector;
				PointVectorData.BranchGradient = BranchGradient;
				PointVectorData.BranchGradientNormalized = BranchGradientNormalized; 
				PointVectorData.PlantGradient = PlantGradient;
				PointVectorData.PlantGradientNormalized = PlantGradientNormalized;
							
				DistributionVectorUtils::ApplyVectorSettings(VectorSettings, PointVectorData, GlobalBudIndex, UpDirection, NormalDirection);

				const FVector3f Dir = (PointPositionAttribute[PointIndexB] - PointPositionAttribute[PointIndexA]).GetSafeNormal();
				const FVector3f CrossVector = FVector3f::CrossProduct(UpDirection, Dir);
				const FVector3f UpVector = FVector3f::CrossProduct(Dir, CrossVector);
				const float OffsetBias = FMath::Lerp(PointScaleAttribute[PointIndexA], PointScaleAttribute[PointIndexB], PointAlpha) * TrunkOffset;
				const FVector3f FoliagePosition = Pos + UpVector.GetSafeNormal() * OffsetBias;
				
				OutAttachmentPoints.Add({
					.BranchIndex = BranchIndex,
					.PointIndexA = PointIndexA,
					.PointIndexB = PointIndexB,
					.PointAlpha = PointAlpha,
					.Position = FoliagePosition,
					.UpDirection = UpDirection,
					.NormalDirection = NormalDirection,
					.Scale = Scale,
					.LengthFromRoot = AttachmentPointLengthFromRoot,
					.bIsTipInstance = bIsTipInstance,
					.TipPointIndex = TipPointIndex,
					.ConditionSample = Sample
				});

				++GlobalBudIndex;
			}

			++AxillaryRotationIteration;
		}
	}
}

TMap<int32, float> PV::ParametricDistributionHelper::ComputeBranchLengths(FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
                                                                          FBranchPointsAttributeConstView BranchPointsAttribute)
{
	const int32 NumOfBranches = BranchPointsAttribute.Num();
	TMap<int32, float> BranchIndicesToBranchLengths;
	BranchIndicesToBranchLengths.Reserve(NumOfBranches);

	for (int32 BranchIndex = 0; BranchIndex < NumOfBranches; BranchIndex++)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
		if (BranchPoints.Num() < 2)
		{
			BranchIndicesToBranchLengths.Add(BranchIndex, 0.0f);
			continue;
		}

#if DO_ENSURE
		for (int32 i = 0; i < BranchPoints.Num() - 1; ++i)
		{
			ensureMsgf(
				PointLengthFromRootAttribute[BranchPoints[i]] <= PointLengthFromRootAttribute[BranchPoints[i + 1]],
				TEXT("BranchPoints are not ordered root-to-tip at index %d for BranchIndex %d"), i, BranchIndex);
		}
#endif

		const int32 RootPointIndex      = BranchPoints[0];
		const int32 BranchTipPointIndex = BranchPoints.Last();

		const float RootPointLFR = PointLengthFromRootAttribute[RootPointIndex];
		const float TipPointLFR = PointLengthFromRootAttribute[BranchTipPointIndex];
		const float BranchLength = TipPointLFR - RootPointLFR;

		BranchIndicesToBranchLengths.Add(BranchIndex, BranchLength);
	}

	return BranchIndicesToBranchLengths;
}

FVector3f PV::ParametricDistributionHelper::SampleVectorByFloat(const float SampleValue, const TArray<int32>& BranchPoints,
                                                                FPointPlantGradientAttributeConstView PointPlantGradientAttribute,
                                                                FPointPositionAttributeConstView PointPositionAttribute)
{
	if (BranchPoints.Num() < 2)
	{
		return FVector3f();
	}

	const int32 FirstPointIndex = BranchPoints[0];
	const int32 LastPointIndex = BranchPoints.Last();
	float Value0 = 1.0f - PointPlantGradientAttribute[FirstPointIndex];
	float Value1 = 1.0f - PointPlantGradientAttribute[LastPointIndex];
	FVector3f OutValue(Value0, Value0, Value0);

	if (FMath::IsWithinInclusive(SampleValue, Value0, Value1))
	{
		for (int32 i = 1; i < BranchPoints.Num(); i++)
		{
			Value1 = 1.0f - PointPlantGradientAttribute[BranchPoints[i]];
			if (FMath::IsWithinInclusive(SampleValue, Value0, Value1))
			{
				FVector3f P0 = PointPositionAttribute[BranchPoints[i - 1]];
				FVector3f P1 = PointPositionAttribute[BranchPoints[i]];

				float Blend = FMath::GetMappedRangeValueClamped(
					FVector2f(Value0, Value1),
					FVector2f(0.0f, 1.0f),
					SampleValue);

				OutValue = FMath::Lerp(P0, P1, Blend);
			}

			Value0 = Value1;
		}
	}
	else
	{
		if (FMath::Abs(Value0 - SampleValue) > FMath::Abs(Value1 - SampleValue))
		{
			OutValue = FVector3f(Value1, Value1, Value1);
		}
	}

	return OutValue;
}

float PV::ParametricDistributionHelper::SampleFloatByFloat(const float SampleValue, const TArray<int32>& BranchPoints,
                                                           FPointPlantGradientAttributeConstView PointPlantGradientAttribute,
                                                           FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute)
{
	if (BranchPoints.Num() < 2)
	{
		return -1.0f;
	}

	const int32 FirstPointIndex = BranchPoints[0];
	const int32 LastPointIndex = BranchPoints.Last();
	float Value0 = 1.0f - PointPlantGradientAttribute[FirstPointIndex];
	float Value1 = 1.0f - PointPlantGradientAttribute[LastPointIndex];
	float OutValue = Value0;

	if (FMath::IsWithinInclusive(SampleValue, Value0, Value1))
	{
		for (int32 i = 1; i < BranchPoints.Num(); i++)
		{
			Value1 = 1.0f - PointPlantGradientAttribute[BranchPoints[i]];
			if (FMath::IsWithinInclusive(SampleValue, Value0, Value1))
			{
				float P0 = PointLengthFromRootAttribute[BranchPoints[i - 1]];
				float P1 = PointLengthFromRootAttribute[BranchPoints[i]];

				float Blend = FMath::GetMappedRangeValueClamped(
					FVector2f(Value0, Value1),
					FVector2f(0.0f, 1.0f),
					SampleValue);

				OutValue = FMath::Lerp(P0, P1, Blend);
			}

			Value0 = Value1;
		}
	}
	else
	{
		if (FMath::Abs(Value0 - SampleValue) > FMath::Abs(Value1 - SampleValue))
		{
			OutValue = Value1;
		}
	}

	return OutValue;
}

bool PV::ParametricDistributionHelper::FindPointByNormalizedLengthFromRoot(float SampleValue, const TArray<int32>& BranchPoints,
                                                                           FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
                                                                           const float LFRofRoot, const float LFRofTip,
                                                                           int32& PreviousBranchPointIndex, int32& NextBranchPointIndex, float& OutAlpha)
{
	if (BranchPoints.Num() < 2)
	{
		return false;
	}

	SampleValue = FMath::Clamp(SampleValue, 0.0f, 1.0f);

	const float SampleLengthFromRoot = FMath::GetMappedRangeValueClamped(
		FVector2f(0.0f, 1.0f),
		FVector2f(LFRofRoot, LFRofTip),
		SampleValue);

	for (int32 i = 0; i < BranchPoints.Num() - 1; ++i)
	{
		const int32 P = BranchPoints[i];
		const int32 N = BranchPoints[i + 1];
		const float LFR0 = PointLengthFromRootAttribute[P];
		const float LFR1 = PointLengthFromRootAttribute[N];

		if (FMath::IsWithinInclusive(SampleLengthFromRoot, LFR0, LFR1)
			|| FMath::IsWithinInclusive(SampleLengthFromRoot, LFR1, LFR0))
		{
			const float Denominator = (LFR1 - LFR0);
			OutAlpha = (FMath::Abs(Denominator) <= KINDA_SMALL_NUMBER)
				? 0.0f
				: ((SampleLengthFromRoot - LFR0) / Denominator);
			OutAlpha = FMath::Clamp(OutAlpha, 0.0f, 1.0f);
			PreviousBranchPointIndex = i;
			NextBranchPointIndex = i + 1;

			return true;
		}
	}

	PreviousBranchPointIndex = 0;
	NextBranchPointIndex = BranchPoints.Num() - 1;
	OutAlpha = (SampleValue < 0.5f)
		? 0.0f
		: 1.0f;

	return true;
}

void PV::ParametricDistributionHelper::ComputeRotationAnglesForAttachment(
	const FVector3f& ApicalDirection,
	const FVector3f& BudDirectionUp,
	const FVector3f& BaseAxillaryDirection,
	const float PlantGradient,
	const float CurveU,
	const int32 Iteration,
	const int32 SubIteration,
	const int32 NumSubIterations,
	const int32 BudIndex,
	const int32 RandomSeed,
	const FParametricSettings::FAngleSettings& AngleSettings,
	const FParametricSettings::FSpacingSettings& SpacingSettings,
	FVector3f& OutUpDirection,
	FVector3f& OutNormalDirection)
{
	const FVector3f ApicalDirectionNormalized = ApicalDirection.GetSafeNormal();
	if (ApicalDirectionNormalized.IsNearlyZero())
	{
		OutUpDirection = FVector3f(1, 0, 0);
		OutNormalDirection = FVector3f(0, 0, 1);
		return;
	}

	FVector3f BudDirectionNormalized = BaseAxillaryDirection.GetSafeNormal();
	if (BudDirectionNormalized.IsNearlyZero())
	{
		BudDirectionNormalized = BudDirectionUp.GetSafeNormal();
	}

	if (BudDirectionNormalized.IsNearlyZero())
	{
		FVector Axis1, Axis2;
		FVector(ApicalDirectionNormalized).FindBestAxisVectors(Axis1, Axis2);
		BudDirectionNormalized = FVector3f(Axis2);
	}

	FVector3f AxilAxis = FVector3f::CrossProduct(BudDirectionNormalized, ApicalDirectionNormalized).GetSafeNormal();
	if (AxilAxis.IsNearlyZero())
	{
		FVector Axis1, Axis2;
		FVector(ApicalDirectionNormalized).FindBestAxisVectors(Axis1, Axis2);
		AxilAxis = FVector3f(Axis2);
	}

	float RotationDegrees = AngleSettings.Rotation * static_cast<float>(Iteration);
	if (NumSubIterations > 0)
	{
		RotationDegrees += (360.0f / static_cast<float>(NumSubIterations)) * static_cast<float>(SubIteration + 1);
	}

	float AxilRampLookup = CurveU;
	if (AngleSettings.AxilAngleRampBasis == FParametricSettings::EDistributionBasis::Plant)
	{
		AxilRampLookup = PlantGradient;
	}

	AxilRampLookup = FMath::GetMappedRangeValueClamped(
		FVector2f(SpacingSettings.RelativeStart, SpacingSettings.RelativeEnd),
		FVector2f(0.0f, 1.0f),
		AxilRampLookup);
	const float AxilRampValue = AngleSettings.AxilAngleRamp->GetRichCurveConst()->Eval(AxilRampLookup);

	float AxilAngleInDegrees = AngleSettings.AxilAngle * AxilRampValue;
	const uint32 SeedHash = HashCombine(GetTypeHash(BudIndex), GetTypeHash(RandomSeed));
	AxilAngleInDegrees += IndexedRandomRange(
		SeedHash,
		AngleSettings.RandomizeAxilAngleMinimum,
		AngleSettings.RandomizeAxilAngleMaximum);

	AxilAngleInDegrees = FMath::Clamp(AxilAngleInDegrees, -90.0f, 90.0f);

	const FQuat4f AxilQuat(AxilAxis, FMath::DegreesToRadians(AxilAngleInDegrees));
	const FVector3f AxilAdjusted = AxilQuat.RotateVector(BudDirectionNormalized);

	const FQuat4f TwistQuat(ApicalDirectionNormalized, FMath::DegreesToRadians(RotationDegrees));

	OutUpDirection = TwistQuat.RotateVector(AxilAdjusted).GetSafeNormal();
	OutNormalDirection = TwistQuat.RotateVector(AxilQuat.RotateVector(ApicalDirectionNormalized)).GetSafeNormal();
}
