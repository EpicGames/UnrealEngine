// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVDistributionHelper.h"
#include "PVPhyllotaxyHelper.h"
#include "Utils/PVAttributes.h"
#include "Utils/PVFloatRamp.h"

using namespace PV::DistributionConditionUtils;

namespace PV::ParametricDistributionHelper
{
	namespace FParametricSettings
	{
		enum class EDistributionBasis : uint8
		{
			Plant,
			Branch
		};

		struct FSpacingSettings
		{
			const int32 BranchDensity;
			const float RelativeStart;
			const float RelativeEnd;
			const bool LimitStartGeneration;
			const int32 StartGeneration;
			const bool LimitEndGeneration;
			const int32 EndGeneration;
			const EDistributionBasis SpacingBasis;
			const FPVFloatRamp* SpacingRamp;
		};

		struct FPhyllotaxySettings
		{
			const EPhyllotaxyType PhyllotaxyType;
			const EPhyllotaxyFormation PhyllotaxyFormation;
			const int32 MinimumNodeBuds;
			const int32 MaximumNodeBuds;
			const bool bSingleBudTip = true;
			const float PhyllotaxyAdditionalAngle;
			const bool ResetPhyllotaxy;
			const float PhyllotaxyOffset;
		};

		struct FAngleSettings
		{
			const float Rotation;
			const float AxilAngle;
			const float RandomizeAxilAngleMinimum;
			const float RandomizeAxilAngleMaximum;
			const EDistributionBasis AxilAngleRampBasis;
			const FPVFloatRamp* AxilAngleRamp;
		};

		struct FScaleSettings
		{
			const EDistributionBasis ScaleRampBasis;
			const float BaseScale;
			const float RandomizeScaleMinimum;
			const float RandomizeScaleMaximum;
			const FPVFloatRamp* ScaleRamp;
			const float BranchScaleImpact;
		};
	}

	float IndexedRandom01(const int32 Seed);

	float IndexedRandomRange(const int32 Seed, const float MinValue, const float MaxValue);

	int32 IndexedRandomIntInclusive(const int32 Seed, const int32 MinValue, const int32 MaxValue);

	static float ComputeAttachmentScale(
		const int32 BudIndex,
		const int32 RandomSeed,
		const float PlantGradient,
		const float BranchGradient,
		const float LengthRatio,
		const FParametricSettings::FScaleSettings& ScaleSettings, 
		float RelativeStart, 
		float RelativeEnd);

	void ComputeAttachmentPoints(
		FManagedArrayCollection& OutCollection,
		const FNormalizedPointCaches& NormalizedPointCaches,
		const FParametricSettings::FSpacingSettings& SpacingSettings,
		const FParametricSettings::FPhyllotaxySettings& PhyllotaxySettings,
		const FParametricSettings::FAngleSettings& AngleSettings,
		const FParametricSettings::FScaleSettings& ScaleSettings,
		const FPVDistributionVectorParams& VectorSettings,
		const int32 RandomSeed,
		float TrunkOffset,
		TArray<DistributionHelper::FAttachmentPoint>& OutAttachmentPoints);

	TArray<float> RemapPScalesToGradientRange(
		FBranchPointsAttributeConstView BranchPointsAttribute,
		FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
		FPointScaleAttributeConstView PointScaleAttribute,
		float RelativeStart,
		float RelativeEnd);

	TMap<int32, float> ComputeBranchLengths(
		FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
		FBranchPointsAttributeConstView BranchPointsAttribute);

	FVector3f SampleVectorByFloat(
		const float SampleValue,
		const TArray<int32>& BranchPoints,
		FPointPlantGradientAttributeConstView PointPlantGradientAttribute,
		FPointPositionAttributeConstView PointPositionAttribute);

	float SampleFloatByFloat(
		const float SampleValue,
		const TArray<int32>& BranchPoints,
		FPointPlantGradientAttributeConstView PointPlantGradientAttribute,
		FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute);

	bool FindPointByNormalizedLengthFromRoot(
		float SampleValue,
		const TArray<int32>& BranchPoints,
		FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute,
		const float LFRofRoot,
		const float LFRofTip,
		int32& PreviousBranchPointIndex,
		int32& NextBranchPointIndex,
		float& OutAlpha);

	void ComputeRotationAnglesForAttachment(
		const FVector3f& ApicalDirection,
		const FVector3f& BudDirectionUp,
		const FVector3f& BaseAxillaryDirection,
		const float PlantGradient,
		const float CurveU,
		const int32 Iteration,
		const int32 SubIteration,
		const int32 NumSubIterations,
		const int32 BudIndex,
		const int32 RandomSeed,
		const FParametricSettings::FAngleSettings& AngleSettings,
		const FParametricSettings::FSpacingSettings& SpacingSettings,
		FVector3f& OutUpDirection,
		FVector3f& OutNormalDirection);
}
