// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PVPhyllotaxyHelper.generated.h"

UENUM(BlueprintType)
enum class EPhyllotaxyType : uint8
{
	Alternate UMETA(DisplayName = "Alternate"),
	Opposite UMETA(DisplayName = "Opposite"),
	Decussate UMETA(DisplayName = "Decussate"),
	Whorled UMETA(DisplayName = "Whorled"),
	Spiral UMETA(DisplayName = "Spiral")
};

UENUM(BlueprintType)
enum class EPhyllotaxyFormation : uint8
{
	Distichous UMETA(DisplayName = "Distichous"),
	Tristichous UMETA(DisplayName = "Tristichous"),
	Pentastichous UMETA(DisplayName = "Pentastichous"),
	Octastichous UMETA(DisplayName = "Octastichous"),
	Parastichous UMETA(DisplayName = "Parastichous")
};

namespace PV::PhyllotaxySettingsHelper
{
	struct FResolvedPhyllotaxy
	{
		float FormationDegrees = 180.0f;
		int32 MinBuds = 1;
		int32 MaxBuds = 1;
		bool bResetPhyllotaxy = false;
		float OffsetDegrees = 0.0f;
	};

	template <typename TPhyllotaxySettings>
	FResolvedPhyllotaxy ResolvePhyllotaxy(const TPhyllotaxySettings& Settings)
	{
		FResolvedPhyllotaxy Result;
		Result.bResetPhyllotaxy = Settings.ResetPhyllotaxy;
		Result.OffsetDegrees = Settings.PhyllotaxyOffset;

		switch (Settings.PhyllotaxyType)
		{
		case EPhyllotaxyType::Alternate:
			Result.FormationDegrees = 180.0f + Settings.PhyllotaxyAdditionalAngle;
			Result.MinBuds = 1;
			Result.MaxBuds = 1;
			break;

		case EPhyllotaxyType::Opposite:
			Result.FormationDegrees = Settings.PhyllotaxyAdditionalAngle;
			Result.MinBuds = 2;
			Result.MaxBuds = 2;
			break;

		case EPhyllotaxyType::Decussate:
			Result.FormationDegrees = 90.0f + Settings.PhyllotaxyAdditionalAngle;
			Result.MinBuds = 2;
			Result.MaxBuds = 2;
			break;

		case EPhyllotaxyType::Whorled:
			Result.FormationDegrees = 90.0f + Settings.PhyllotaxyAdditionalAngle;
			Result.MinBuds = Settings.MinimumNodeBuds;
			Result.MaxBuds = Settings.MaximumNodeBuds;
			break;

		case EPhyllotaxyType::Spiral:
			switch (Settings.PhyllotaxyFormation)
			{
			case EPhyllotaxyFormation::Distichous:
				Result.FormationDegrees = 180.0f + Settings.PhyllotaxyAdditionalAngle;
				break;
			case EPhyllotaxyFormation::Tristichous:
				Result.FormationDegrees = 120.0f + Settings.PhyllotaxyAdditionalAngle;
				break;
			case EPhyllotaxyFormation::Pentastichous:
				Result.FormationDegrees = 144.0f + Settings.PhyllotaxyAdditionalAngle;
				break;
			case EPhyllotaxyFormation::Octastichous:
				Result.FormationDegrees = 135.0f + Settings.PhyllotaxyAdditionalAngle;
				break;
			case EPhyllotaxyFormation::Parastichous:
				Result.FormationDegrees = Settings.PhyllotaxyAdditionalAngle;
				break;
			}

			Result.MinBuds = 1;
			Result.MaxBuds = 1;
			break;
		}

		return Result;
	}
}
