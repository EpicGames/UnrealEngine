// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Utils/PVAttributes.h"

namespace PV::PlantTraversalHelper
{
	// Lookup table that maps branch numbers to their array indices.
	// Internally uses an offset array for dense ranges or a TMap for sparse ranges,
	// chosen at construction time based on the ratio of element count to value range.
	struct FBranchNumberToIndexTable
	{
		struct FArrayTable
		{
			int32 Offset;
			TArray<int32> Table;
		};

		using FMapTable = TMap<int32, int32>;

		TVariant<FEmptyVariantState, FArrayTable, FMapTable> Table;

		FBranchNumberToIndexTable() = default;
		FBranchNumberToIndexTable(PV::FBranchNumberAttributeView BranchNumberAttribute) { Init(BranchNumberAttribute); }
		FBranchNumberToIndexTable(PV::FBranchNumberAttributeConstView BranchNumberAttribute) { Init(BranchNumberAttribute); }

		void Init(PV::FBranchNumberAttributeConstView BranchNumberAttribute)
		{
			if (BranchNumberAttribute.Num() == 0)
			{
				return;
			}

			int32 Min = TNumericLimits<int32>::Max();
			int32 Max = TNumericLimits<int32>::Min();

			for (int32 i = 0; i < BranchNumberAttribute.Num(); ++i)
			{
				Min = FMath::Min(Min, BranchNumberAttribute[i]);
				Max = FMath::Max(Max, BranchNumberAttribute[i]);
			}

			// Allow up to 8192 empty slots (~32KB) in the array table before falling back to TMap.
			// Array lookup has better cache locality; 32KB of overhead is negligible per plant.
			constexpr int32 ArrayTableLimit = 8192;
			const int32 Range = Max - Min;
			if (Range <= BranchNumberAttribute.Num() + ArrayTableLimit)
			{
				FArrayTable ArrayTable;
				ArrayTable.Table.Init(INDEX_NONE, Range + 1);
				ArrayTable.Offset = Min;

				for (int32 i = 0; i < BranchNumberAttribute.Num(); ++i)
				{
					const int32 BranchNumber = BranchNumberAttribute[i];
					ArrayTable.Table[BranchNumber - ArrayTable.Offset] = i;
				}

				Table.Emplace<FArrayTable>(MoveTemp(ArrayTable));
			}
			else
			{
				FMapTable MapTable;
				MapTable.Reserve(BranchNumberAttribute.Num());
				for (int32 i = 0; i < BranchNumberAttribute.Num(); ++i)
				{
					MapTable.Add(BranchNumberAttribute[i], i);
				}

				Table.Emplace<FMapTable>(MoveTemp(MapTable));
			}
		}

		int32 Find(int32 BranchNumber) const
		{
			if (!ensure(IsValid()))
			{
				return INDEX_NONE;
			}

			if (Table.IsType<FArrayTable>())
			{
				const FArrayTable& ArrayTable = Table.Get<FArrayTable>();
				const int32 TableIndex = BranchNumber - ArrayTable.Offset;
				return ArrayTable.Table.IsValidIndex(TableIndex) ? ArrayTable.Table[TableIndex] : INDEX_NONE;
			}
			else
			{
				const FMapTable& MapTable = Table.Get<FMapTable>();
				const int32* BranchIndex = MapTable.Find(BranchNumber);
				return BranchIndex ? *BranchIndex : INDEX_NONE;
			}
		}

		bool IsValid() const
		{
			return !Table.IsType<FEmptyVariantState>();
		}
	};

	inline int32 GetBranchParentIndex(
		FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		int32 BranchIndex
	)
	{
		if (!BranchParentNumberAttribute.IsValidIndex(BranchIndex) || !BranchNumberToIndex.IsValid())
		{
			return INDEX_NONE;
		}

		return BranchNumberToIndex.Find(BranchParentNumberAttribute[BranchIndex]);
	}

	// Slow variant of GetBranchParentIndex. Prefer GetBranchParentIndex with a FBranchNumberToIndexTable when a FBranchNumberToIndexTable is already available.
	// The difference is in how the parent branch index is resolved:
	//   GetBranchParentIndex        — takes a pre-built FBranchNumberToIndexTable and resolves the parent in O(1).
	//   GetBranchParentIndex (Slow) — takes the raw BranchNumberAttribute and resolves the parent via a linear Find(),
	//                                 making repeated calls O(N) per lookup.
	// Use this variant only when constructing the lookup table upfront is not practical (e.g. one-off lookups
	// in contexts where the table is not already available and building it would be wasteful).
	inline int32 GetBranchParentIndex(
		FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		FBranchNumberAttributeConstView BranchNumberAttribute,
		int32 BranchIndex
	)
	{
		return BranchParentNumberAttribute.IsValidIndex(BranchIndex)
			? BranchNumberAttribute.Find(BranchParentNumberAttribute[BranchIndex])
			: INDEX_NONE;
	}

	inline TArray<int32> GetPlantBranchIndices(
		PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute,
		int32 PlantNumber
	)
	{
		TArray<int32> OutBranchIndices;

		for (int32 i = 0; i < BranchPlantNumberAttribute.Num(); i++)
		{
			if (BranchPlantNumberAttribute[i] == PlantNumber)
			{
				OutBranchIndices.Add(i);
			}
		}

		return OutBranchIndices;
	}

	inline bool IsTrunk(FBranchParentNumberAttributeConstView BranchParentNumberAttribute, int32 BranchIndex)
	{
		return BranchParentNumberAttribute.IsValidIndex(BranchIndex) && BranchParentNumberAttribute[BranchIndex] == 0;
	}

	inline TArray<int32> GetTrunkIndices(FBranchParentNumberAttributeConstView BranchParentNumberAttribute)
	{
		TArray<int32> TrunkIndicesResult;
		for (int32 i = 0; i < BranchParentNumberAttribute.Num(); i++)
		{
			if (IsTrunk(BranchParentNumberAttribute, i))
			{
				TrunkIndicesResult.Add(i);
			}
		}
		return TrunkIndicesResult;
	}

	inline FVector3f GetPlantRootPoint(
		FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute,
		FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		FBranchPointsAttributeConstView BranchPointsAttribute,
		FPointPositionAttributeConstView PointPositionAttribute,
		int32 PlantNumber
	)
	{
		for (int32 BranchIndex : GetPlantBranchIndices(BranchPlantNumberAttribute, PlantNumber))
		{
			if (IsTrunk(BranchParentNumberAttribute, BranchIndex))
			{
				const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
				if (BranchPoints.Num() > 0 && PointPositionAttribute.IsValidIndex(BranchPoints[0]))
				{
					return PointPositionAttribute[BranchPoints[0]];
				}
			}
		}
		return FVector3f::ZeroVector;
	}

	inline TArray<int32> GetUniquePlantNumbers(PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute)
	{
		TSet<int32> PlantNumberElements;
		for (int32 i = 0; i < BranchPlantNumberAttribute.Num(); i++)
		{
			PlantNumberElements.Add(BranchPlantNumberAttribute[i]);
		}
		return PlantNumberElements.Array();
	}

	enum class EForEachResult
	{
		Break    = 0,
		Continue = 1
	};

	// Walks the branches in hierarchical order (e.g. start with the trunk and recursivly visit the child branches)
	template <typename StateType>
	void RecursiveWalkBranches(
		FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		FBranchChildrenAttributeConstView BranchChildrenAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		const StateType& RootBranchState,
		const TFunction<EForEachResult(int32, StateType&)>& Body
	)
	{
		if (!ValidateAttributeCollection(BranchParentNumberAttribute, BranchChildrenAttribute))
		{
			return;
		}

		const TFunction<EForEachResult(int32, const StateType&)> WalkBranches_Recursive = [&](int32 BranchIndex, const StateType& State) -> EForEachResult
		{
			StateType BranchState = State;
			if (Body(BranchIndex, BranchState) == EForEachResult::Break)
			{
				return EForEachResult::Break;
			}

			const TArray<int32>& BranchChildren = BranchChildrenAttribute[BranchIndex];
			for (int32 ChildBranchNumber : BranchChildren)
			{
				const int32 ChildBranchIndex = BranchNumberToIndex.Find(ChildBranchNumber);
				if (!ensure(ChildBranchIndex != INDEX_NONE))
				{
					continue;
				}

				if (BranchNumberToIndex.Find(BranchParentNumberAttribute[ChildBranchIndex]) != BranchIndex)
				{
					continue;
				}

				if (WalkBranches_Recursive(ChildBranchIndex, BranchState) == EForEachResult::Break)
				{
					return EForEachResult::Break;
				}
			}

			return EForEachResult::Continue;
		};

		for (const int32 TrunkIndex : GetTrunkIndices(BranchParentNumberAttribute))
		{
			if (WalkBranches_Recursive(TrunkIndex, RootBranchState) == EForEachResult::Break)
			{
				return;
			}
		}
	}

	inline void RecursiveWalkBranches(
		FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		FBranchChildrenAttributeConstView BranchChildrenAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		const TFunction<EForEachResult(int32 BranchIndex)>& Body
	)
	{
		RecursiveWalkBranches<int32>(
			BranchParentNumberAttribute,
			BranchChildrenAttribute,
			BranchNumberToIndex,
			INDEX_NONE,
			[&](int32 BranchIndex, int32&) -> EForEachResult
			{
				if (Body(BranchIndex) == EForEachResult::Break)
				{
					return EForEachResult::Break;
				}
				return EForEachResult::Continue;
			}
		);
	}

	// Walks the branches in reversed hierarchical order (e.g. first visit all leaf nodes, then their parents, and so on, visiting the tunk last)
	inline void RecursiveWalkBranches_Reversed(
		FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		FBranchChildrenAttributeConstView BranchChildrenAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		const TFunction<EForEachResult(int32 BranchIndex)>& Body
	)
	{
		TArray<int32> BranchIndices;
		BranchIndices.Reserve(BranchParentNumberAttribute.Num());

		RecursiveWalkBranches(
			BranchParentNumberAttribute,
			BranchChildrenAttribute,
			BranchNumberToIndex,
			[&](int32 BranchIndex) -> EForEachResult
			{
				BranchIndices.Add(BranchIndex);
				return EForEachResult::Continue;
			}
		);

		for (int32 i = BranchIndices.Num() - 1; i >= 0; --i)
		{
			if (Body(BranchIndices[i]) == EForEachResult::Break)
			{
				break;
			}
		}
	}

	inline TArray<int32> GetBranchPointChildBranchIndices(
		PV::FBranchChildrenAttributeConstView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		int32 BranchIndex,
		int32 BranchPointIndex
	)
	{
		if (!ValidateAttributeCollection(BranchChildrenAttribute, BranchParentNumberAttribute, BranchPointsAttribute))
		{
			return TArray<int32>();
		}

		if (!BranchPointsAttribute.IsValidIndex(BranchIndex))
		{
			return TArray<int32>();
		}

		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
		if (!BranchPoints.IsValidIndex(BranchPointIndex))
		{
			return TArray<int32>();
		}

		const int32 PointIndex = BranchPoints[BranchPointIndex];
		const TArray<int32>& BranchChildren = BranchChildrenAttribute[BranchIndex];

		TArray<int32> ChildrenIndices;
		ChildrenIndices.Reserve(BranchChildren.Num());

		for (const int32 BranchChild : BranchChildren)
		{
			const int32 ChildIndex = BranchNumberToIndex.Find(BranchChild);
			if (ChildIndex == INDEX_NONE)
			{
				continue;
			}

			const int32 ParentIndex = GetBranchParentIndex(BranchParentNumberAttribute, BranchNumberToIndex, ChildIndex);
			if (ParentIndex != BranchIndex)
			{
				continue;
			}

			const TArray<int32>& ChildBranchPoints = BranchPointsAttribute[ChildIndex];
			if (ChildBranchPoints.Num() == 0)
			{
				continue;
			}

			const int32 FirstChildPointIndex = ChildBranchPoints[0];
			if (FirstChildPointIndex == PointIndex)
			{
				ChildrenIndices.Add(ChildIndex);
			}
		}

		return ChildrenIndices;
	}

	struct FRecursiveWalkBranchPointsParams
	{
		int32 BranchIndex;
		int32 BranchPointIndex; // The index of the point in the branch points attribute
		int32 PointIndex;	    // The global point index of this point
		int32 ParentPointIndex; // The global point index of this points parent point (INDEX_NONE for the first point on the trunk)
		const TArray<int32>& ChildBranchIndices;
		const TArray<int32>& ChildPointIndices;
	};

	template <typename StateType, bool bWalkAxillaryFirst = false>
	inline EForEachResult RecursiveWalkBranchPoints(
		PV::FBranchChildrenAttributeConstView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		int32 BranchIndex,
		const StateType& RootPointState,
		const TFunction<EForEachResult(const FRecursiveWalkBranchPointsParams& Params, StateType& PointState)>& Body
	)
	{
		if (!ValidateAttributeCollection(BranchChildrenAttribute, BranchParentNumberAttribute, BranchPointsAttribute))
		{
			return EForEachResult::Break;
		}

		const TFunction<EForEachResult(int32, int32, int32, const StateType&)> WalkPoints = [&](int32 BranchIndex, int32 BranchPointIndex, int32 ParentPointIndex, const StateType& ParentState) -> EForEachResult
		{
			StateType PointState = ParentState;
			
			const TArray<int32>& CurrentBranchPoints = BranchPointsAttribute[BranchIndex];
			const int32 PointIndex = CurrentBranchPoints[BranchPointIndex];

			const TArray<int32> ChildBranchIndices = GetBranchPointChildBranchIndices(
				BranchChildrenAttribute,
				BranchParentNumberAttribute,
				BranchNumberToIndex,
				BranchPointsAttribute,
				BranchIndex,
				BranchPointIndex
			);

			TArray<int32> ChildPointIndices;
			ChildPointIndices.SetNum(ChildBranchIndices.Num());
			for (int32 i = 0; i < ChildBranchIndices.Num(); ++i)
			{
				const int32 ChildBranchIndex = ChildBranchIndices[i];
				ChildPointIndices[i] = BranchPointsAttribute[ChildBranchIndex].Num() > 1 ? BranchPointsAttribute[ChildBranchIndex][1] : INDEX_NONE;
			}

			if (Body({ BranchIndex, BranchPointIndex, PointIndex, ParentPointIndex, ChildBranchIndices, ChildPointIndices }, PointState) == EForEachResult::Break)
			{
				return EForEachResult::Break;
			}

			struct FChildPoint
			{
				int32 BranchIndex;
				int32 BranchPointIndex;
			};
			TArray<FChildPoint, TInlineAllocator<3>> Children;
			Children.Reserve(1 + ChildBranchIndices.Num());

			if constexpr (bWalkAxillaryFirst)
			{
				for (int32 ChildBranchIndex : ChildBranchIndices)
				{
					Children.Emplace(ChildBranchIndex, 1);
				}

				if (BranchPointIndex < CurrentBranchPoints.Num() - 1)
				{
					Children.Emplace(BranchIndex, BranchPointIndex + 1);
				}
			}
			else
			{
				if (BranchPointIndex < CurrentBranchPoints.Num() - 1)
				{
					Children.Emplace(BranchIndex, BranchPointIndex + 1);
				}

				for (int32 ChildBranchIndex : ChildBranchIndices)
				{
					Children.Emplace(ChildBranchIndex, 1);
				}
			}

			for (const auto& [ChildBranchIndex, ChildBranchPointIndex] : Children)
			{
				if (WalkPoints(ChildBranchIndex, ChildBranchPointIndex, PointIndex, PointState) == EForEachResult::Break)
				{
					return EForEachResult::Break;
				}
			}

			return EForEachResult::Continue;
		};

		return WalkPoints(BranchIndex, 0, INDEX_NONE, RootPointState);
	}

	template<bool bWalkAxillaryFirst = false>
	inline EForEachResult RecursiveWalkBranchPoints(
		PV::FBranchChildrenAttributeConstView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		int32 BranchIndex,
		const TFunction<EForEachResult(const FRecursiveWalkBranchPointsParams& Params)>& Body
	)
	{
		return RecursiveWalkBranchPoints<int32, bWalkAxillaryFirst>(
			BranchChildrenAttribute,
			BranchParentNumberAttribute,
			BranchNumberToIndex,
			BranchPointsAttribute,
			BranchIndex,
			0,
			[&](const FRecursiveWalkBranchPointsParams& Params, int32&) -> EForEachResult
			{
				return Body(Params);
			}
		);
	}

	template <typename StateType, bool bWalkAxillaryBranchesFirst = false>
	void RecursiveWalkPlantPoints(
		PV::FBranchChildrenAttributeConstView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		const StateType& RootPointState,
		const TFunction<EForEachResult(const FRecursiveWalkBranchPointsParams& Params, StateType& PointState)>& Body
	)
	{
		for (const int32 TrunkIndex : GetTrunkIndices(BranchParentNumberAttribute))
		{
			auto Result = RecursiveWalkBranchPoints<StateType, bWalkAxillaryBranchesFirst>(
				BranchChildrenAttribute,
				BranchParentNumberAttribute,
				BranchNumberToIndex,
				BranchPointsAttribute,
				TrunkIndex,
				RootPointState,
				Body
			);
			if (Result == EForEachResult::Break)
			{
				return;
			}
		}
	}

	template <bool bWalkAxillaryBranchesFirst = false>
	inline void RecursiveWalkPlantPoints(
		PV::FBranchChildrenAttributeConstView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const FBranchNumberToIndexTable& BranchNumberToIndex,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		const TFunction<EForEachResult(const FRecursiveWalkBranchPointsParams& Params)>& Body
	)
	{
		RecursiveWalkPlantPoints<int32, bWalkAxillaryBranchesFirst>(
			BranchChildrenAttribute,
			BranchParentNumberAttribute,
			BranchNumberToIndex,
			BranchPointsAttribute,
			INDEX_NONE,
			[&](const FRecursiveWalkBranchPointsParams& Params, int32&)
			{
				return Body(Params);
			}
		);
	}

	inline void ForEachUniquePointOnBranches(
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		const TFunction<EForEachResult(int32 BranchIndex, int32 BranchPointIndex, int32 PointIndex)>& Body
	)
	{
		if (!ValidateAttributeCollection(BranchPointsAttribute, BranchParentNumberAttribute))
		{
			return;
		}

		for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
		{
			const bool bSkipFirstPoint = !IsTrunk(BranchParentNumberAttribute, BranchIndex);

			const int32 StartBranchPointIndex = bSkipFirstPoint ? 1 : 0;
			const TArray<int32>& PointIndices = BranchPointsAttribute[BranchIndex];
			for (int32 BranchPointIndex = StartBranchPointIndex; BranchPointIndex < PointIndices.Num(); ++BranchPointIndex)
			{
				if (Body(BranchIndex, BranchPointIndex, PointIndices[BranchPointIndex]) == EForEachResult::Break)
				{
					return;
				}
			}
		}
	}

	inline void ForEachPlant(
		PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute,
		const TFunction<EForEachResult(int32 PlantNumber)>& Body
	)
	{
		for (int32 PlantNumber : GetUniquePlantNumbers(BranchPlantNumberAttribute))
		{
			if (Body(PlantNumber) == EForEachResult::Break)
			{
				return;
			}
		}
	}

	inline void ForEachPlant(
		const FManagedArrayCollection& Collection,
		const TFunction<EForEachResult(int32 PlantNumber)>& Body
	)
	{
		if (auto BranchPlantNumberAttribute = PV::FBranchPlantNumberAttribute::FindAttribute(Collection))
		{
			ForEachPlant(BranchPlantNumberAttribute, Body);
		}
	}

	inline void ForEachPlantPoint(
		PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		int32 PlantNumber,
		const TFunction<EForEachResult(int32 BranchIndex, int32 PointIndex)>& Body
	)
	{
		if (!ValidateAttributeCollection(BranchPlantNumberAttribute, BranchPointsAttribute, BranchParentNumberAttribute))
		{
			return;
		}

		const TArray<int32> BranchIndices = GetPlantBranchIndices(BranchPlantNumberAttribute, PlantNumber);
		for (int32 BranchIndex : BranchIndices)
		{
			const bool bIsTrunk = IsTrunk(BranchParentNumberAttribute, BranchIndex);
			for (int32 i = bIsTrunk ? 0 : 1; i < BranchPointsAttribute[BranchIndex].Num(); ++i)
			{
				if (Body(BranchIndex, BranchPointsAttribute[BranchIndex][i]) == EForEachResult::Break)
				{
					return;
				}
			}
		}
	}

	inline void ForEachPlantPoint(
		const FManagedArrayCollection& Collection,
		int32 PlantNumber,
		const TFunction<EForEachResult(int32 BranchIndex, int32 PointIndex)>& Body
	)
	{
		const PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute = FBranchPlantNumberAttribute::FindAttribute(Collection);
		const PV::FBranchPointsAttributeConstView BranchPointsAttribute = FBranchPointsAttribute::FindAttribute(Collection);
		const PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = FBranchParentNumberAttribute::FindAttribute(Collection);
		ForEachPlantPoint(BranchPlantNumberAttribute, BranchPointsAttribute, BranchParentNumberAttribute, PlantNumber, Body);
	}
}