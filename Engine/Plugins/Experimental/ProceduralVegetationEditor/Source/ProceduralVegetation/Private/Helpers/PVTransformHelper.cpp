// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PVTransformHelper.h"

void PV::Transform::TransformPoint(
	const FTransformPointAttributeCollection& Attributes,
	int32 BranchIndex,
	int32 PointIndex,
	const FVector3f& Translation,
	const FQuat4f& Rotation
)
{
	if (Attributes.PointPosition.IsValid() && Attributes.PointPosition.IsValidIndex(PointIndex))
	{
		Attributes.PointPosition[PointIndex] += Translation;
	}

	if (Attributes.BudDirection.IsValid() && Attributes.BudDirection.IsValidIndex(PointIndex))
	{
		PV::FBudDirectionView BudDirections = Attributes.BudDirection[PointIndex];
		BudDirections.Apical = Rotation.RotateVector(BudDirections.Apical);
		BudDirections.Axillary = Rotation.RotateVector(BudDirections.Axillary);
		BudDirections.GuideCurve = Rotation.RotateVector(BudDirections.GuideCurve);
		BudDirections.UpVector = Rotation.RotateVector(BudDirections.UpVector);
	}
}
