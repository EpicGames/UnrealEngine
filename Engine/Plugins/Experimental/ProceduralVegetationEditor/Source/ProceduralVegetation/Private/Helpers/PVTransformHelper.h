// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Utils/PVAttributes.h"
#include "Helpers/PVPlantTraversalHelper.h"

namespace PV::Transform
{
/*
* Attributes required to traverse a tree and rotate the points around its parent point.
*/
struct FRotateBranchPointsAttributeCollection
{
	PV::FPointPositionAttributeView PointPosition;
	PV::FBudDirectionAttributeView BudDirection;
	PV::FBranchChildrenAttributeConstView BranchChildren;
	PV::FBranchParentNumberAttributeConstView BranchParentNumber;
	PV::FBranchNumberAttributeConstView BranchNumber;
	PV::FBranchPointsAttributeConstView BranchPoints;

	FRotateBranchPointsAttributeCollection() = default;
	FRotateBranchPointsAttributeCollection(FManagedArrayCollection& Collection)
		: PointPosition(PV::FPointPositionAttribute::FindAttribute(Collection))
		, BudDirection(PV::FBudDirectionAttribute::FindAttribute(Collection))
		, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(Collection))
		, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(Collection))
		, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(Collection))
		, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(Collection))
	{
	}

	bool IsValid() const
	{
		return PointPosition
			&& BudDirection
			&& BranchChildren
			&& BranchParentNumber
			&& BranchNumber
			&& BranchPoints;
	}
};

struct FTransformPointAttributeCollection
{
	PV::FPointPositionAttributeView PointPosition;
	PV::FBudDirectionAttributeView BudDirection;

	FTransformPointAttributeCollection() = default;
	FTransformPointAttributeCollection(FManagedArrayCollection& Collection)
		: PointPosition(PV::FPointPositionAttribute::FindAttribute(Collection))
		, BudDirection(PV::FBudDirectionAttribute::FindAttribute(Collection))
	{
	}

	FTransformPointAttributeCollection(const FRotateBranchPointsAttributeCollection& Other)
		: PointPosition(Other.PointPosition)
		, BudDirection(Other.BudDirection)
	{
	}

	FTransformPointAttributeCollection(
		PV::FPointPositionAttributeView InPointPositionAttribute, 
		PV::FBudDirectionAttributeView InBudDirectionAttribute)
		: PointPosition(InPointPositionAttribute)
		, BudDirection(InBudDirectionAttribute)
	{
	}
};

PROCEDURALVEGETATION_API extern void TransformPoint(
	const FTransformPointAttributeCollection& Attributes,
	int32 BranchIndex,
	int32 PointIndex,
	const FVector3f& Translation,
	const FQuat4f& Rotation
);

using FRotateBranchPointParams = PV::PlantTraversalHelper::FRecursiveWalkBranchPointsParams;

struct FRotateBranchPointResult
{
	FQuat4f Rotation;
	bool bAbsoluteRotation;  // If true, then the rotation of the parent point will be ignored when rotating this point.
};

/**
 * Recursively rotates all points in the branch hierarchy starting from the specified BranchIndex.
 * 
 * Parent rotations are multiplied into child transforms ensuring that child points inherit all 
 * parent rotations (unless bAbsoluteRotation is set to true). The behavior mirrors bone 
 * transform accumulation in a skeletal hierarchy.
 *
 * This function performs a depth-first traversal of the branch hierarchy:
 *  1. It walks all points in the current branch in order.
 *  2. For each point, it invokes RotatePointsBody, allowing the caller to define
 *     a local rotation for that point.
 *  3. The computed rotation is accumulated with the parent’s transform, ensuring
 *     that rotations propagate hierarchically (similar to skeletal animation).
 *     Each point is effectively rotated around its parent pivot (or TrunkPivot
 *     for the root).
 *  4. After finishing all points in the current branch, the function recursively
 *     processes any axillary (child) branches originating from points in this branch.
 *
 * @param Attributes       The attributes requried to traverse the tree and update the point rotations.
 * @param BranchIndex      Index of the branch to start traversal from.
 * @param RotatePointsBody Callback invoked for each point to compute its rotation.
 * @param TrunkPivot       Pivot used for the root branch.
 *
 * @return true if the traversal and rotation succeeded; false otherwise.
 */
template<typename StateType>
inline bool RecursiveRotateBranchPoints(
	const FRotateBranchPointsAttributeCollection& Attributes,
	int32 BranchIndex,
	const StateType& RootPointState,
	const TFunction<void(const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result, StateType& State)>& RotatePointsBody,
	const FVector3f& TrunkPivot = FVector3f::ZeroVector
)
{
	using namespace PV::PlantTraversalHelper;

	if (!Attributes.IsValid())
	{
		return false;
	}

	struct FPointState
	{
		FVector3f  ParentPivotPoint;
		FVector3f  ParentIdentityPosition;
		FMatrix44f TransformMatrix;
		StateType  UserState;
	};

	const FPointState DefaultPointState = {
		FVector3f::ZeroVector,
		TrunkPivot,
		FTranslationMatrix44f(TrunkPivot),
		RootPointState
	};

	RecursiveWalkBranchPoints<FPointState, false>(
		Attributes.BranchChildren,
		Attributes.BranchParentNumber,
		Attributes.BranchNumber,
		Attributes.BranchPoints,
		BranchIndex,
		DefaultPointState,
		[&](const FRecursiveWalkBranchPointsParams& InParams, FPointState& PointState)
		{
			FRotateBranchPointResult Result = {
				.Rotation = FQuat4f::Identity,
				.bAbsoluteRotation = false,
			};
			RotatePointsBody(InParams, Result, PointState.UserState);

			const FVector3f& PointPosition = Attributes.PointPosition[InParams.PointIndex];
			const FVector3f RelativePosition = PointPosition - PointState.ParentIdentityPosition;

			const FMatrix44f LocalRotationMatrix = Result.Rotation.ToMatrix();
			const FMatrix44f LocalTranslationMatrix = FTranslationMatrix44f(RelativePosition);
			const FMatrix44f LocalTransformMatrix = LocalTranslationMatrix * LocalRotationMatrix;

			if (Result.bAbsoluteRotation)
			{
				const FMatrix44f TranslationOnly = FTranslationMatrix44f(PointState.TransformMatrix.GetOrigin() + LocalTransformMatrix.GetOrigin());
				PointState.TransformMatrix = LocalRotationMatrix * TranslationOnly;
			}
			else
			{
				PointState.TransformMatrix = LocalTransformMatrix * PointState.TransformMatrix;
			}

			PointState.ParentIdentityPosition = PointPosition;

			const FVector3f PointTranslation = PointState.TransformMatrix.GetOrigin() - PointPosition;
			const FQuat4f PointRotation = PointState.TransformMatrix.ToQuat();
			TransformPoint(Attributes, InParams.BranchIndex, InParams.PointIndex, PointTranslation, PointRotation);

			return EForEachResult::Continue;
		}
	);

	return true;
}

inline bool RecursiveRotateBranchPoints(
	const FRotateBranchPointsAttributeCollection& Attributes,
	int32 BranchIndex,
	const TFunction<void(const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result)>& RotatePointsBody,
	const FVector3f& TrunkPivot = FVector3f::ZeroVector
)
{
	return RecursiveRotateBranchPoints<int32>(
		Attributes, 
		BranchIndex, 
		0,
		[&](const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result, int32&) { RotatePointsBody(Params, Result); },
		TrunkPivot
	);
}

template<typename StateType>
inline bool RecursiveRotatePlantPoints(
	const FRotateBranchPointsAttributeCollection& Attributes,
	const StateType& RootPointState,
	const TFunction<void(const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result, StateType& State)>& RotatePointsBody,
	const FVector3f& TrunkPivot = FVector3f::ZeroVector
)
{
	if (!Attributes.BranchParentNumber.IsValid())
	{
		return false;
	}

	for (const int32 TrunkIndex : PV::PlantTraversalHelper::GetTrunkIndices(Attributes.BranchParentNumber))
	{
		const auto Result = RecursiveRotateBranchPoints<StateType>(
			Attributes,
			TrunkIndex,
			RootPointState,
			RotatePointsBody,
			TrunkPivot
		);

		if (!Result)
		{
			return false;
		}
	}

	return true;
}

inline bool RecursiveRotatePlantPoints(
	const FRotateBranchPointsAttributeCollection& Attributes,
	const TFunction<void(const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result)>& RotatePointsBody,
	const FVector3f& TrunkPivot = FVector3f::ZeroVector
)
{
	return RecursiveRotatePlantPoints<int32>(
		Attributes,
		0,
		[&](const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result, int32&) { RotatePointsBody(Params, Result); },
		TrunkPivot
	);
}

};