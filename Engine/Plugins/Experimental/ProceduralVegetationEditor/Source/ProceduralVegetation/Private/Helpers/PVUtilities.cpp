// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Helpers/PVUtilities.h"
#include "Helpers/PVAttributesHelper.h"
#include "ProceduralVegetationModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/Package.h"
#include "Components/MeshComponent.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"
#include "Implementations/PVFoliage.h"
#include "Misc/FileHelper.h"
#include "Nodes/PVPresetLoaderSettings.h"
#include "Rendering/SkeletalMeshRenderData.h"
 
namespace PV::Utilities
{
#if WITH_EDITOR
	static TAutoConsoleVariable<bool> CVarEnablePVDebugMode(
		TEXT("PV.DebugMode.Enabled"),
		false,
		TEXT("Enables debug mode for the Procedural Vegetation Editor"), 
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* ConsoleVariable)
			{
				const bool bNewValue = ConsoleVariable->GetBool();

				TArray<UClass*> PVSettingsClasses;
				GetDerivedClasses(UPVBaseSettings::StaticClass(), PVSettingsClasses);

				for (UClass* PVSettingsClass : PVSettingsClasses)
				{
					UPVBaseSettings* BaseSettings = PVSettingsClass->GetDefaultObject<UPVBaseSettings>();
					check(BaseSettings != nullptr);
					if (BaseSettings->bOnlyExposeInDebugMode)
					{
						BaseSettings->bExposeToLibrary = bNewValue;
					}
				}
			})
	);
#endif
 
	bool DebugModeEnabled()
	{
#if WITH_EDITOR
		return CVarEnablePVDebugMode.GetValueOnAnyThread();
#else
		return false;
#endif
	}
 
	FString FormatLongPackageNameErrorCode(const FPackageName::EErrorCode ErrorCode)
	{
		switch (ErrorCode)
		{
		case FPackageName::EErrorCode::PackageNameUnknown:
			return "Unknown Export Path error";
		case FPackageName::EErrorCode::PackageNameEmptyPath:
			return "Empty Export Path";
		case FPackageName::EErrorCode::PackageNamePathNotMounted:
			return "Mount Point for Export Path is not valid";
		case FPackageName::EErrorCode::PackageNamePathIsMemoryOnly:
			return "Export Path exists in memory only. ";
		case FPackageName::EErrorCode::PackageNameSpacesNotAllowed:
			return "Spaces are not allowed in Export Path";	
		case FPackageName::EErrorCode::PackageNameContainsInvalidCharacters:
			return FString::Printf(TEXT("Export Path contains one of \n%s\n Invalid Characters"), TEXT(R"(\\:*?\"<>|' ,.&!~\n\r\t@#)"));
		case FPackageName::EErrorCode::LongPackageNames_PathTooShort:
			return "Export Path is too small to be a valid path";
		case FPackageName::EErrorCode::LongPackageNames_PathWithNoStartingSlash:
			return "Export Path has to start with a slash [/]";
		case FPackageName::EErrorCode::LongPackageNames_PathWithTrailingSlash:
			return "Export Path cannot end with a slash [/]";
		case FPackageName::EErrorCode::LongPackageNames_PathWithDoubleSlash:
			return "Export Path contains a double slash [/]";
		}
		return FString();
	}
 
	bool ValidateAssetPathAndName(const FString& AssetName, const FString& Path, UClass* InClass, FString& OutError)
	{
		if (FPackageName::EErrorCode ErrorCode; !FPackageName::IsValidLongPackageName(Path, false, &ErrorCode))
        {
			OutError =  FormatLongPackageNameErrorCode(ErrorCode);
        	return false;
        }
 
		if (FText Reason; !IsFileNameValid(FName(AssetName), Reason))
        {
			OutError = Reason.ToString();
        	return false;
        }
 
		return true;
	}
 
	bool IsFileNameValid(FName FileName, FText& Reason)
	{
		FileName.IsValidObjectName(Reason);
		FFileHelper::IsFilenameValidForSaving(FileName.ToString(), Reason);
 
		return Reason.IsEmpty();
	}
 
	bool DoesConflictingPackageExist(const FString& LongPackageName, UClass* AssetClass)
	{
		UPackage* Package = FindPackage(nullptr, *LongPackageName);
		if (IsValid(Package))
		{
			UObject* ExistingAsset = Package->FindAssetInPackage();
			return !ExistingAsset || ExistingAsset->GetClass() != AssetClass;
		}
		else
		{
			TArray<FAssetData> OutAssets;
			IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
			AssetRegistry.GetAssetsByPackageName(*LongPackageName, OutAssets, true);
 
			if (OutAssets.Num() == 0)
			{
				return false;
			}
 
			if (OutAssets.Num() != 1
				|| OutAssets[0].AssetClassPath != AssetClass->GetClassPathName())
			{
				return true;
			}
		}
 
		return false;
	}
 
	bool PackageExists(const FString& LongPackageName, UClass* AssetClass)
	{
		UPackage* Package = FindPackage(nullptr, *LongPackageName);
		if (IsValid(Package))
		{
			UObject* ExistingAsset = Package->FindAssetInPackage();
			return ExistingAsset && ExistingAsset->GetClass() == AssetClass;
		}
		else
		{
			TArray<FAssetData> OutAssets;
			IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
			AssetRegistry.GetAssetsByPackageName(*LongPackageName, OutAssets, true);
 
			if (OutAssets.Num() == 0)
			{
				return false;
			}
 
			if (OutAssets[0].AssetClassPath == AssetClass->GetClassPathName())
			{
				return true;
			}
		}
 
		return false;
	}
 
	int32 GetMeshTriangles(const FString InMeshPath)
	{
		UStaticMesh* StaticFoliageMesh = LoadObject<UStaticMesh>(nullptr, *InMeshPath, {}, LOAD_NoWarn | LOAD_Quiet);
		USkeletalMesh* SkeletalFoliageMesh = LoadObject<USkeletalMesh>(nullptr, *InMeshPath, {}, LOAD_NoWarn | LOAD_Quiet);
 
		int32 NumTriangles = 0;
		
		if (StaticFoliageMesh)
		{
			NumTriangles = StaticFoliageMesh->GetNumTriangles(0);
		}
		else if (SkeletalFoliageMesh)
		{
			FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalFoliageMesh->GetResourceForRendering();
			if (SkelMeshRenderData && SkelMeshRenderData->LODRenderData.Num() > 0)
			{
				const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[0];
				NumTriangles = LODData.GetTotalFaces();
			}
 
			return NumTriangles;
		}
 
		return NumTriangles;
	}
 
	FLinearColor GetRandomHueColor(float Alpha)
	{
		Alpha = FMath::Clamp(Alpha, 0.f, 1.f);
		const float HueDeg = 240.f * Alpha;
		return FLinearColor(HueDeg, 1, 1).HSVToLinearRGB();
	};
 
	bool IsValidGrowthData(const FManagedArrayCollection& Collection)
	{
		if (!Collection.HasGroup(PV::GroupNames::PointGroup) 
			|| !Collection.HasGroup(PV::GroupNames::BranchGroup))
		{
			// Early out if PointGroup or BranchGroup is missing
			return false;
		}

		Facades::FPointFacade PointFacade = Facades::FPointFacade(Collection);
		Facades::FBranchFacade BranchFacade = Facades::FBranchFacade(Collection);
	
		return PointFacade.IsValid() && BranchFacade.IsValid();
	}

	template <typename T>
	static FORCEINLINE T LerpValue(const T& A, const T& B, float Alpha)
	{
		if constexpr (TIsIntegral<T>::Value)
		{
			const float Val = FMath::Lerp(static_cast<float>(A), static_cast<float>(B), Alpha);
			return static_cast<T>(FMath::RoundToInt(Val));
		}
		else
		{
			return FMath::Lerp(A, B, Alpha);
		}
	}

	template <typename T>
	TArray<T> LerpArrayElements(const TArray<T>& InArray1, const TArray<T>& InArray2, const float Alpha)
	{
		TArray<T> OutArray;
		OutArray.SetNumUninitialized(FMath::Min(InArray1.Num(), InArray2.Num()));

		for (int32 i = 0; i < FMath::Min(InArray1.Num(), InArray2.Num()); i++)
		{
			OutArray[i] = LerpValue(InArray1[i], InArray2[i], Alpha);
		}

		return OutArray;
	}

	int32 AddInterpolatedPointToCollection(FManagedArrayCollection& OutCollection, int32 BranchIndex, const int32 CurrentPointIndex, const int32 NextPointIndex,
	                           const float Alpha, const int32 BudNumber, bool CalculatePixelIndex /*= false*/)
	{
		Facades::FPointFacade PointFacade(OutCollection);

		const FVector3f CurrentPointPosition = PointFacade.GetPosition(CurrentPointIndex);
		const FVector3f NextPointPosition = PointFacade.GetPosition(NextPointIndex);
		const FVector3f Position = FMath::Lerp(CurrentPointPosition, NextPointPosition, Alpha);

		const float CurrentPointLengthFromRoot = PointFacade.GetLengthFromRoot(CurrentPointIndex);
		const float NextPointLengthFromRoot = PointFacade.GetLengthFromRoot(NextPointIndex);
		const float LengthFromRoot = FMath::Lerp(CurrentPointLengthFromRoot, NextPointLengthFromRoot, Alpha);

		const auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(OutCollection);
		const auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(OutCollection);
		const auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(OutCollection);
		const auto BudDirectionAttribute = PV::FBudDirectionAttribute::GetAttribute(OutCollection);
		const auto PointPositionAttribute = PV::FPointPositionAttribute::GetAttribute(OutCollection);
		const auto BudDevelopmentAttribute = PV::FBudDevelopmentAttribute::GetAttribute(OutCollection);
		
		const int32 CurrentBranchPointIndex = BranchPointsAttribute.IsValidIndex(BranchIndex)
			? BranchPointsAttribute[BranchIndex].IndexOfByKey(CurrentPointIndex)
			: INDEX_NONE;
		
		const int32 NextBranchPointIndex = BranchPointsAttribute.IsValidIndex(BranchIndex)
			? BranchPointsAttribute[BranchIndex].IndexOfByKey(NextPointIndex)
			: INDEX_NONE;
		
		check(CurrentBranchPointIndex != INDEX_NONE);
		check(NextBranchPointIndex != INDEX_NONE);

		const float CurrentPointScale = AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, 
				BranchIndex, CurrentBranchPointIndex);
		
		const float NextPointScale = AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, 
			BranchIndex, NextBranchPointIndex);
		
		const float PointScale = FMath::Lerp(CurrentPointScale, NextPointScale, Alpha);
		
		float CurrentSeedPScale = 0.f;
		PointFacade.GetSeedPScale(CurrentPointIndex, CurrentSeedPScale);
		float NextSeedPScale = 0.f;
		PointFacade.GetSeedPScale(NextPointIndex, NextSeedPScale);
		const float SeedPScale = FMath::Lerp(CurrentSeedPScale, NextSeedPScale, Alpha);
		
		float CurrentSeedPScaleRatio = 1.f;
		PointFacade.GetSeedPScaleRatio(CurrentPointIndex, CurrentSeedPScaleRatio);
		float NextSeedPScaleRatio = 1.f;
		PointFacade.GetSeedPScaleRatio(NextPointIndex, NextSeedPScaleRatio);
		const float SeedPScaleRatio = FMath::Lerp(CurrentSeedPScaleRatio, NextSeedPScaleRatio, Alpha);

		const float CurrentPointLengthFromSeed = PointFacade.GetLengthFromSeed(CurrentPointIndex);
		const float NextPointLengthFromSeed = PointFacade.GetLengthFromSeed(NextPointIndex);
		const float LengthFromSeed = FMath::Lerp(CurrentPointLengthFromSeed, NextPointLengthFromSeed, Alpha);

		const TArray<float>& CurrentPointBudLightDetected = PointFacade.GetBudLightDetected(CurrentPointIndex);
		const TArray<float>& NextPointBudLightDetected = PointFacade.GetBudLightDetected(NextPointIndex);
		const TArray<float> BudLightDetected = LerpArrayElements(CurrentPointBudLightDetected, NextPointBudLightDetected, Alpha);

		const TArray<int>& CurrentPointBudDevelopment = AttributesHelper::GetBudDevelopment(BudDevelopmentAttribute, BranchPointsAttribute,BranchParentNumberAttribute,
			BranchIndex, CurrentBranchPointIndex);
		
		const TArray<int>& NextPointBudDevelopment = AttributesHelper::GetBudDevelopment(BudDevelopmentAttribute, BranchPointsAttribute,BranchParentNumberAttribute,
			BranchIndex, NextBranchPointIndex);
		const TArray<int> BudDevelopment = LerpArrayElements(CurrentPointBudDevelopment, NextPointBudDevelopment, Alpha);

		const TArray<float>& CurrentPointBudLateralMeristem = PointFacade.GetBudLateralMeristem(CurrentPointIndex);
		const TArray<float>& NextPointBudLateralMeristem = PointFacade.GetBudLateralMeristem(NextPointIndex);
		const TArray<float> BudLateralMeristem = LerpArrayElements(CurrentPointBudLateralMeristem, NextPointBudLateralMeristem, Alpha);

		const TArray<float>& CurrentPointBudHormoneLevels = PointFacade.GetBudHormoneLevels(CurrentPointIndex);
		const TArray<float>& NextPointBudHormoneLevels = PointFacade.GetBudHormoneLevels(NextPointIndex);
		const TArray<float> BudHormoneLevels = LerpArrayElements(CurrentPointBudHormoneLevels, NextPointBudHormoneLevels, Alpha);

		const TArray<FVector3f> CurrentPointBudDirections = AttributesHelper::GetBudDirection(BudDirectionAttribute, BranchPointsAttribute, BranchParentNumberAttribute, 
			PointPositionAttribute, BranchIndex, CurrentBranchPointIndex);
		
		const TArray<FVector3f>& NextPointBudDirections = AttributesHelper::GetBudDirection(BudDirectionAttribute, BranchPointsAttribute, BranchParentNumberAttribute, 
			PointPositionAttribute, BranchIndex, NextBranchPointIndex);
		
		const TArray<FVector3f> BudDirections = LerpArrayElements(CurrentPointBudDirections, NextPointBudDirections, Alpha);

		TArray<int> CurrentPointBudStatus;
		PointFacade.GetBudStatus(CurrentPointIndex, CurrentPointBudStatus);
		TArray<int> NextPointBudStatus;
		PointFacade.GetBudStatus(NextPointIndex, NextPointBudStatus);
		TArray<int> BudStatus;
		BudStatus.SetNum(CurrentPointBudStatus.Num());

		check(BudStatus.Num() == 10);

		// 0_ApicalMeristem
		BudStatus[0] = 0;
		// 1_Codominant
		BudStatus[1] = 0;
		// 2_Axillary
		BudStatus[2] = 1;
		// 3_Seed
		BudStatus[3] = 0;
		// 4_Dormant
		BudStatus[4] = 0;
		// 5_Triggered
		BudStatus[5] = 1;
		// 6_NumTriggered
		BudStatus[6] = 1;
		// 7_Inactive
		BudStatus[7] = 0;
		// 8_BrokenTip
		BudStatus[8] = 0;
		// 9_Broken
		BudStatus[9] = 0;

		const int32 NewIndex = PointFacade.AddElements(1);
		PointFacade.SetPosition(NewIndex, Position);
		PointFacade.SetLengthFromRoot(NewIndex, LengthFromRoot);
		PointFacade.SetPointScale(NewIndex, PointScale);
		PointFacade.SetSeedPScale(NewIndex, SeedPScale);
		PointFacade.SetSeedPScaleRatio(NewIndex, SeedPScaleRatio);
		PointFacade.SetLengthFromSeed(NewIndex, LengthFromSeed);
		PointFacade.SetBudLightDetected(NewIndex, BudLightDetected);
		PointFacade.SetBudDevelopment(NewIndex, BudDevelopment);
		PointFacade.SetBudLateralMeristem(NewIndex, BudLateralMeristem);
		PointFacade.SetBudNumber(NewIndex, BudNumber);
		PointFacade.SetBudHormoneLevels(NewIndex, BudHormoneLevels);
		PointFacade.SetBudDirections(NewIndex, BudDirections);
		PointFacade.SetBudStatus(NewIndex, BudStatus);

		if (CalculatePixelIndex)
		{
			auto NjordPixelIndexAttribute = FPointNjordPixelIndexAttribute::FindAttribute(OutCollection);
			
			if (NjordPixelIndexAttribute.IsValid())
			{
				NjordPixelIndexAttribute[NewIndex] = FMath::Lerp(NjordPixelIndexAttribute[CurrentPointIndex], NjordPixelIndexAttribute[NextPointIndex], Alpha);
			}
		}

		return NewIndex;
	}

	bool DoesAssetExist(const FSoftObjectPath& InPath)
	{
		if (InPath.IsNull())
		{
			return false;
		}
		
		const FAssetData AssetData = IAssetRegistry::Get()->GetAssetByObjectPath(InPath);
		return AssetData.IsValid();
	}
}
