// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVCarve.h"

#include "Facades/PVAttributesNames.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"
#include "Helpers/PVAttributesHelper.h"

namespace
{
	static void MarkSubtreeForRemoval(
		const int32 BranchIndex,
		const PV::Facades::FBranchFacade& BranchFacadeSource,
		const TMap<int32, int32>& BranchNumbersToBranchIDs,
		TArray<bool>& BranchesToRemove,
		TArray<bool>& PointsToRemove,
		const bool bSkipFirstPoint = false)
	{
		BranchesToRemove[BranchIndex] = true;
		const TArray<int32>& BranchPoints = BranchFacadeSource.GetPoints(BranchIndex);

		const int32 StartingIndex = bSkipFirstPoint? 1: 0;
		
		for (int32 i = StartingIndex; i < BranchPoints.Num(); ++i)
		{
			PointsToRemove[BranchPoints[i]] = true;
		}

		for (const int32& ChildBranchNumber : BranchFacadeSource.GetImmediateChildren(BranchIndex))
		{
			const int32* ChildIndexPtr = BranchNumbersToBranchIDs.Find(ChildBranchNumber);
			if (ChildIndexPtr && !BranchesToRemove[*ChildIndexPtr])
			{
				MarkSubtreeForRemoval(*ChildIndexPtr, BranchFacadeSource, BranchNumbersToBranchIDs, BranchesToRemove,
					PointsToRemove);
			}
		}
	}
}

void FPVCarve::ApplyCarve(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection,
                          const ECarveBasis CarveBasis, const float Carve)
{
	if (Carve == 0.0f)
	{
		return;
	}

	if (CarveBasis == ECarveBasis::FromBottom)
	{
		CarveFromBottom(OutCollection, SourceCollection, Carve);
	}
	else
	{
		CarveFromTop(OutCollection, SourceCollection, Carve, CarveBasis);
	}
}

void FPVCarve::UpdatePointScales(PV::Facades::FPointFacade& PointFacadeOut, const PV::Facades::FPointFacade& PointFacadeSource,
                                 const TArray<int>& BranchPoints, const float LastPointScale, const int32 LastPointIndex,
                                 const float CarveRatio, const int32 EndIndex, const float FirstPointTargetPScale, const bool InSkipFirstPoint)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVCarve::UpdatePointScales);

	const float OldMax = FirstPointTargetPScale;
	const float NewMax = OldMax * CarveRatio;
	const float Ratio = NewMax / OldMax;
	const float OldMin = PointFacadeSource.GetPointScale(LastPointIndex);
	const float NewMin = LastPointScale * Ratio;

	const int32 StartingIndex = InSkipFirstPoint? 1: 0;
	
	for (int32 i = StartingIndex; i <= EndIndex; ++i)
	{
		const float UpdatedScale = FMath::GetMappedRangeValueClamped(
			FVector2f(OldMin, OldMax),
			FVector2f(NewMin, NewMax),
			PointFacadeSource.GetPointScale(BranchPoints[i]));
		PointFacadeOut.ModifyPointScales()[BranchPoints[i]] = UpdatedScale;
	}
}

void FPVCarve::RecomputeAttributes(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection,
                                   const TMap<int32, int32>& BranchesNewIDsToOldIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVCarve::RecomputeAttributes);

	const PV::Facades::FBranchFacade BranchFacadeSource(SourceCollection);
	const PV::Facades::FPointFacade PointFacadeSource(SourceCollection);

	const PV::Facades::FBranchFacade BranchFacadeOut(OutCollection);
	const PV::Facades::FPlantFacade PlantFacadeOut(OutCollection);
	PV::Facades::FPointFacade PointFacadeOut(OutCollection);

	for (const int32& PlantNumber : PlantFacadeOut.GetPlantNumbers())
	{
		// Remap Ethylene levels for each branch and the whole plant
		for (const int32& BranchIndex : PlantFacadeOut.GetBranchIndices(PlantNumber))
		{
			const TArray<int32>& BranchPoints = BranchFacadeOut.GetPoints(BranchIndex);
			const int32 NumBranchPointsCurrent = BranchPoints.Num();
			const int32 OldBranchIndex = BranchesNewIDsToOldIDs.Contains(BranchIndex)
				? BranchesNewIDsToOldIDs[BranchIndex]
				: BranchIndex;
			const TArray<int32>& SourceBranchPoints = BranchFacadeSource.GetPoints(OldBranchIndex);

			for (int32 Idx = 0; Idx < NumBranchPointsCurrent; ++Idx)
			{
				const float RelativePointIndex = (static_cast<float>(Idx) / NumBranchPointsCurrent) * SourceBranchPoints.Num();
				const int32 OldCurrentPointIndex = FMath::Clamp(FMath::CeilToInt(RelativePointIndex), 0, SourceBranchPoints.Num() - 1);
				const int32 OldPreviousPointIndex = OldCurrentPointIndex == 0
					? 0
					: OldCurrentPointIndex - 1;
				float BlendValue = RelativePointIndex - FMath::FloorToInt(RelativePointIndex);

				const TArray<float>& BudHormoneLevelsAtOldPreviousPointIndex = PointFacadeSource.GetBudHormoneLevels(
					SourceBranchPoints[OldPreviousPointIndex]);
				check(BudHormoneLevelsAtOldPreviousPointIndex.Num() >= 5);
				float EthyleneLevelAtOldPreviousPointIndex = BudHormoneLevelsAtOldPreviousPointIndex[4];

				const TArray<float>& BudHormoneLevelsAtOldCurrentPointIndex = PointFacadeSource.GetBudHormoneLevels(
					SourceBranchPoints[OldCurrentPointIndex]);
				check(BudHormoneLevelsAtOldCurrentPointIndex.Num() >= 5);
				float EthyleneLevelAtOldCurrentPointIndex = BudHormoneLevelsAtOldCurrentPointIndex[4];

				const float NewEthyleneLevel = FMath::Lerp(EthyleneLevelAtOldPreviousPointIndex, EthyleneLevelAtOldCurrentPointIndex, BlendValue);
				TArray<float> CurrentBudHormoneLevels = PointFacadeOut.GetBudHormoneLevels(BranchPoints[Idx]);
				check(CurrentBudHormoneLevels.Num() >= 4);
				CurrentBudHormoneLevels[4] = NewEthyleneLevel;
				PointFacadeOut.SetBudHormoneLevels(BranchPoints[Idx], CurrentBudHormoneLevels);
			}
		}
	}
}

void FPVCarve::CarveFromTop(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection, const float Carve,
                            const ECarveBasis CarveBasis)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVCarve::CarveFromTop);

	const float CarveDistance = 1.0f - Carve;

	const PV::Facades::FBranchFacade BranchFacadeSource(SourceCollection);
	const PV::Facades::FPointFacade PointFacadeSource(SourceCollection);
	const PV::Facades::FPlantFacade PlantFacadeSource(SourceCollection);

	PV::Facades::FBranchFacade BranchFacadeOut(OutCollection);
	PV::Facades::FPointFacade PointFacadeOut(OutCollection);
	PV::Facades::FFoliageFacade FoliageFacadeOut(OutCollection);

	if (PointFacadeSource.GetElementCount() == 0 || BranchFacadeSource.GetElementCount() == 0)
	{
		return;
	}

	const int32 NumOfBranchesInSource = BranchFacadeSource.GetElementCount();
	TArray<bool> PointsToRemove;
	PointsToRemove.Init(false, PointFacadeSource.GetElementCount());
	TArray<bool> BranchesToRemove;
	BranchesToRemove.Init(false, NumOfBranchesInSource);
	

	TMap<int32, int32> BranchNumbersToBranchIDs;
	TMap<int32, float> BranchNumbersToLengthFromRoots;
	ComputeMetadata(BranchNumbersToBranchIDs, BranchNumbersToLengthFromRoots, BranchFacadeSource, PointFacadeSource);

	TArray<float> CarveValues;
	CarveValues.Reserve(PointFacadeSource.GetElementCount());

	for (int32 i = 0; i < PointFacadeSource.GetElementCount(); ++i)
	{
		switch (CarveBasis)
		{
		case ECarveBasis::LengthFromRoot:
			CarveValues.Add(PointFacadeSource.GetLengthFromRoot(i));
			break;

		case ECarveBasis::FromBottom:
			CarveValues.Add(PointFacadeSource.GetLengthFromRoot(i));
			break;

		case ECarveBasis::ZPosition:
			CarveValues.Add(PointFacadeSource.GetPosition(i).Z);
			break;

		case ECarveBasis::Radius:
			CarveValues.Add(PointFacadeSource.GetPointScale(i));
			break;
		}
	}

	const float MinCarveValue = *Algo::MinElement(CarveValues);
	const float MaxCarveValue = *Algo::MaxElement(CarveValues);

	FVector2f MappedRange(0.0f, 1.0f);
	if (CarveBasis == ECarveBasis::Radius)
	{
		MappedRange = FVector2f(1.0f, 0.0f);
	}

	for (int32 i = 0; i < PointFacadeSource.GetElementCount(); ++i)
	{
		CarveValues[i] = FMath::GetMappedRangeValueClamped(
			FVector2f(MinCarveValue, MaxCarveValue),
			MappedRange,
			CarveValues[i]);
	}

	for (const int32& PlantNumber : PlantFacadeSource.GetPlantNumbers())
	{
		for (const int32& BranchIndex : PlantFacadeSource.GetBranchIndices(PlantNumber))
		{
			// Compute branches, foliage instances and points to remove
			if (BranchesToRemove[BranchIndex])
			{
				continue;
			}

			const TArray<int32>& BranchPoints = BranchFacadeSource.GetPoints(BranchIndex);
			const int32 NumOfPointsInCurrentBranch = BranchPoints.Num();

			if (NumOfPointsInCurrentBranch == 0)
			{
				BranchesToRemove[BranchIndex] = true;
				continue;
			}

			//  Find Source point Pscales
			// (For non-trunk branches)
			const int32 SourceBudNumber = BranchFacadeSource.GetBranchSourceBudNumber(BranchIndex);

			const int32* RootPointIndexPtr = BranchPoints.FindByPredicate([&PointFacadeSource, &SourceBudNumber](const int32& PointIndex)
				{
					return PointFacadeSource.GetBudNumber(PointIndex) == SourceBudNumber;
				});

			// Safety check
			checkf(RootPointIndexPtr != nullptr, TEXT("Crave: No point with matching bud number found"));
			check(BranchPoints[0] == *RootPointIndexPtr);
			
			const float FirstPointTargetPScale = PointFacadeSource.GetPointScale(*RootPointIndexPtr);
			const bool IsTrunk = PlantFacadeSource.IsTrunkIndex(BranchIndex);
			
			int EndIndex = 1;
			while (EndIndex < NumOfPointsInCurrentBranch && (CarveValues[BranchPoints[EndIndex]] < CarveDistance + 0.0001f))
			{
				EndIndex++;
			}
			EndIndex--;
			
			if (EndIndex == NumOfPointsInCurrentBranch - 1)
			{
				const int32 LastPointIndex = BranchPoints[EndIndex];
				const float LastPointScale = PointFacadeSource.GetPointScale(LastPointIndex);
				UpdatePointScales(PointFacadeOut, PointFacadeSource, BranchPoints, LastPointScale, LastPointIndex, 1.0f, EndIndex,
					FirstPointTargetPScale, !IsTrunk);
				continue;
			}

			// Remove all branches in the subtree beyond the last kept point
			for (const int32& BranchNumber : BranchFacadeSource.GetImmediateChildren(BranchIndex))
			{
				if (BranchNumbersToLengthFromRoots.Contains(BranchNumber) && BranchNumbersToLengthFromRoots[BranchNumber] > PointFacadeSource.
					GetLengthFromRoot(BranchPoints[EndIndex]))
				{
					const int32 Index = BranchNumbersToBranchIDs[BranchNumber];
					const TArray<int32>& ChildBranchPoints = BranchFacadeSource.GetPoints(Index);
					const bool bSkipFirstPoint = ChildBranchPoints.Num() > 0
						&& BranchPoints.IsValidIndex(EndIndex + 1)
						&& ChildBranchPoints[0] == BranchPoints[EndIndex + 1];

					MarkSubtreeForRemoval(Index, BranchFacadeSource, BranchNumbersToBranchIDs, BranchesToRemove, PointsToRemove, bSkipFirstPoint);
				}
			}

			// Keep next point to adjust its position, remove all others
			++EndIndex;
			for (int32 j = EndIndex + 1; j < NumOfPointsInCurrentBranch; ++j)
			{
				PointsToRemove[BranchPoints[j]] = true;
			}

			TArray<int32> UpdatedBranchPoints = TArray(&BranchPoints[0], EndIndex + 1);
			BranchFacadeOut.SetPoints(BranchIndex, UpdatedBranchPoints);

			// Adjust last point
			const int32 LastPointIndex = BranchPoints[EndIndex];
			const int32 PreviousPointIndex = BranchPoints[EndIndex - 1];
			const FVector3f& LastPointPosition = PointFacadeSource.GetPosition(LastPointIndex);
			const FVector3f& PreviousPointPosition = PointFacadeSource.GetPosition(PreviousPointIndex);
			const float LastPointCarveValue = CarveValues[LastPointIndex];
			const float PreviousPointCarveValue = CarveValues[PreviousPointIndex];
			const float BlendValue = FMath::Clamp(
				FMath::GetMappedRangeValueClamped(
					FVector2f(PreviousPointCarveValue, LastPointCarveValue),
					FVector2f(0.0f, 1.0f),
					CarveDistance),
				0.1f,
				1.0f);
			const FVector3f Position = FMath::Lerp(PreviousPointPosition, LastPointPosition, BlendValue);
			const float LengthFromRoot = FMath::Lerp(PointFacadeSource.GetLengthFromRoot(PreviousPointIndex),
				PointFacadeSource.GetLengthFromRoot(LastPointIndex),
				BlendValue);
			const float PointScale = FMath::Lerp(PointFacadeSource.GetPointScale(PreviousPointIndex),
				PointFacadeSource.GetPointScale(LastPointIndex),
				BlendValue);
			PointFacadeOut.ModifyPositions()[LastPointIndex] = Position;
			PointFacadeOut.ModifyLengthFromRoots()[LastPointIndex] = LengthFromRoot;
			PointFacadeOut.ModifyLengthFromSeeds()[LastPointIndex] = LengthFromRoot;
			PointFacadeOut.ModifyPointScales()[LastPointIndex] = PointScale;
			PointsToRemove[LastPointIndex] = false;

			// Update point scales
			const float CarveRatio = (EndIndex + BlendValue) / NumOfPointsInCurrentBranch;
			UpdatePointScales(PointFacadeOut, PointFacadeSource, BranchPoints, PointScale, LastPointIndex, CarveRatio, EndIndex,
				FirstPointTargetPScale, !IsTrunk);
		}
	}
	TArray<bool> FoliageInstancesToRemove;
	RemoveEntriesAndRecomputeAttributes(OutCollection, SourceCollection, PointsToRemove, BranchesToRemove, FoliageInstancesToRemove);
}

void FPVCarve::CarveFromBottom(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection, const float Carve)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVCarve::CarveFromBottom);

	const PV::Facades::FBranchFacade BranchFacadeSource(SourceCollection);
	const PV::Facades::FPointFacade PointFacadeSource(SourceCollection);
	const PV::Facades::FFoliageFacade FoliageFacadeSource(SourceCollection);
	const PV::Facades::FPlantFacade PlantFacadeSource(SourceCollection);

	PV::Facades::FBranchFacade BranchFacadeOut(OutCollection);
	PV::Facades::FPointFacade PointFacadeOut(OutCollection);

	if (PointFacadeSource.GetElementCount() == 0 || BranchFacadeSource.GetElementCount() == 0)
	{
		return;
	}

	TArray<bool> PointsToRemove;
	PointsToRemove.Init(false, PointFacadeSource.GetElementCount());
	TArray<bool> BranchesToRemove;
	BranchesToRemove.Init(false, BranchFacadeSource.GetElementCount());

	TMap<int32, int32> BranchNumbersToBranchIDs;
	TMap<int32, float> BranchNumbersToLengthFromRoots;
	ComputeMetadata(BranchNumbersToBranchIDs, BranchNumbersToLengthFromRoots, BranchFacadeSource, PointFacadeSource);

	// Carve along trunk based on length from root
	// Compute branches, foliage instances and points to remove
	for (const TMap<int32, int32> PlantNumbersToTrunkIDs = PlantFacadeSource.GetPlantNumbersToTrunkIndicesMap();
	     const TPair<int32, int32> Pair : PlantNumbersToTrunkIDs)
	{
		const int32 PlantNumber = Pair.Key;
		const int32 TrunkIndex = Pair.Value;
		const TArray<int32>& TrunkPoints = BranchFacadeSource.GetPoints(TrunkIndex);
		const int32 NumOfPointsInTrunk = TrunkPoints.Num();

		if (NumOfPointsInTrunk < 2)
		{
			BranchesToRemove[TrunkIndex] = true;
			continue;
		}

		const float LengthFromRootOfFirstPoint = PointFacadeSource.GetLengthFromRoot(TrunkPoints[0]);
		const float LengthFromRootOfLastPoint = PointFacadeSource.GetLengthFromRoot(TrunkPoints[NumOfPointsInTrunk - 1]);
		const float CarveLengthFromRoot = FMath::GetMappedRangeValueClamped(FVector2f(0.0f, 1.0f),
			FVector2f(LengthFromRootOfFirstPoint, LengthFromRootOfLastPoint), Carve);

		int EndIndex = 0;
		while (EndIndex < NumOfPointsInTrunk && PointFacadeSource.GetLengthFromRoot(TrunkPoints[EndIndex]) < CarveLengthFromRoot)
		{
			EndIndex++;
		}
		EndIndex--;

		// Remove all branches in the subtree before current point
		for (const int32& BranchNumber : BranchFacadeSource.GetImmediateChildren(TrunkIndex))
		{
			if (BranchNumbersToLengthFromRoots.Contains(BranchNumber) && BranchNumbersToLengthFromRoots[BranchNumber] < CarveLengthFromRoot)
			{
				const int32 Index = BranchNumbersToBranchIDs[BranchNumber];

				// Do not remove the child's first point if it is the fused point with the kept new base point
				// (TrunkPoints[EndIndex]); otherwise the new base is deleted and the re-grounded trunk floats above the origin.
				const TArray<int32>& ChildBranchPoints = BranchFacadeSource.GetPoints(Index);
				
				const bool bSkipFirstPoint = ChildBranchPoints.Num() > 0 && ChildBranchPoints[0] == TrunkPoints[EndIndex];

				MarkSubtreeForRemoval(Index, BranchFacadeSource, BranchNumbersToBranchIDs, BranchesToRemove, PointsToRemove, bSkipFirstPoint);
			}
		}

		// Keep current point to adjust its position, remove all others
		for (int32 j = 0; j < EndIndex; ++j)
		{
			PointsToRemove[TrunkPoints[j]] = true;
		}

		TArray<int32> UpdatedTrunkPoints = TArray(&TrunkPoints[EndIndex], NumOfPointsInTrunk - EndIndex);
		BranchFacadeOut.SetPoints(TrunkIndex, UpdatedTrunkPoints);

		// Re-grounding dropped the old trunk base point; keep branchSourceBudNumber on the new base.
		BranchFacadeOut.SetBranchSourceBudNumber(TrunkIndex, PointFacadeSource.GetBudNumber(UpdatedTrunkPoints[0]));

		// Adjust End point
		const int32 CurrentPointIndex = TrunkPoints[EndIndex];
		const int32 NextPointIndex = TrunkPoints[EndIndex + 1];
		const FVector3f& PositionOfCurrentPoint = PointFacadeSource.GetPosition(CurrentPointIndex);
		const FVector3f& PositionOfNextPoint = PointFacadeSource.GetPosition(NextPointIndex);
		const float LengthFromRootOfCurrentPoint = PointFacadeSource.GetLengthFromRoot(CurrentPointIndex);
		const float LengthFromRootOfNextPoint = PointFacadeSource.GetLengthFromRoot(NextPointIndex);

		const float BlendValue = FMath::Clamp(
			FMath::GetMappedRangeValueClamped(
				FVector2f(LengthFromRootOfCurrentPoint, LengthFromRootOfNextPoint),
				FVector2f(0.0f, 1.0f),
				CarveLengthFromRoot),
			0.1f,
			1.0f);
		const FVector3f Position = FMath::Lerp(PositionOfCurrentPoint, PositionOfNextPoint, BlendValue);
		const float PointScale = FMath::Lerp(PointFacadeSource.GetPointScale(CurrentPointIndex),
			PointFacadeSource.GetPointScale(NextPointIndex),
			BlendValue);
		PointFacadeOut.ModifyPositions()[CurrentPointIndex] = Position;
		PointFacadeOut.ModifyLengthFromRoots()[CurrentPointIndex] = CarveLengthFromRoot;
		PointFacadeOut.ModifyLengthFromSeeds()[CurrentPointIndex] = CarveLengthFromRoot;
		PointFacadeOut.ModifyPointScales()[CurrentPointIndex] = PointScale;

		// Move all points down and recompute length from roots
		const FVector3f Delta(0.0f, 0.0f, Position.Z);
		
		TSet<int32> VisitedPoints;

		for (const int32& BranchIndex : PlantFacadeSource.GetBranchIndices(PlantNumber))
		{
			if (BranchesToRemove[BranchIndex])
			{
				continue;
			}

			for (const int32& PointIndex : BranchFacadeOut.GetPoints(BranchIndex))
			{
				if (VisitedPoints.Contains(PointIndex))
					continue;

				PointFacadeOut.ModifyPositions()[PointIndex] = PointFacadeOut.GetPosition(PointIndex) - Delta;

				PointFacadeOut.ModifyLengthFromRoots()[PointIndex] = FMath::Max(PointFacadeOut.GetLengthFromRoot(PointIndex) - CarveLengthFromRoot,
					0.0f);
				PointFacadeOut.ModifyLengthFromSeeds()[PointIndex] = FMath::Max(PointFacadeOut.GetLengthFromSeed(PointIndex) - CarveLengthFromRoot,
					0.0f);

				VisitedPoints.Add(PointIndex);
			}
		}
	}
	
	TArray<bool> FoliageInstancesToRemove;
	RemoveEntriesAndRecomputeAttributes(OutCollection, SourceCollection, PointsToRemove, BranchesToRemove, FoliageInstancesToRemove);
}

void FPVCarve::ComputeMetadata(TMap<int32, int32>& OutBranchNumbersToBranchIDs, TMap<int32, float>& OutBranchNumbersToLengthFromRoots,
                               const PV::Facades::FBranchFacade& BranchFacadeSource,
                               const PV::Facades::FPointFacade& PointFacadeSource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVCarve::ComputeMetadata);

	const int32 NumOfBranchesInSource = BranchFacadeSource.GetElementCount();
	OutBranchNumbersToBranchIDs.Reserve(NumOfBranchesInSource);
	OutBranchNumbersToLengthFromRoots.Reserve(NumOfBranchesInSource);

	for (int32 BranchIndex = 0; BranchIndex < NumOfBranchesInSource; ++BranchIndex)
	{
		const int32 BranchNumber = BranchFacadeSource.GetBranchNumber(BranchIndex);
		OutBranchNumbersToBranchIDs.Add(BranchNumber, BranchIndex);

		const TArray<int32>& BranchPoints = BranchFacadeSource.GetPoints(BranchIndex);
		check(BranchPoints.Num() > 0);

		if (BranchPoints.Num() > 0)
		{
			const int32 FirstPointIndex = BranchFacadeSource.GetPoints(BranchIndex)[0];
			const float BranchLengthFromRoot = PointFacadeSource.GetLengthFromRoot(FirstPointIndex);
			OutBranchNumbersToLengthFromRoots.Add(BranchNumber, BranchLengthFromRoot);
		}
		else
		{
			OutBranchNumbersToLengthFromRoots.Add(BranchNumber, 0.0f);
		}
	}
}

void FPVCarve::RemoveEntriesAndRecomputeAttributes(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection,
                                                   TArray<bool>& PointsToRemove, TArray<bool>& BranchesToRemove,
                                                   TArray<bool>& FoliageInstancesToRemove)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVCarve::RemoveEntriesAndRecomputeAttributes);

	PV::Facades::FRemoveEntriesResult RemoveEntriesResult = PV::Facades::FTreeFacade::RemoveEntriesAndReIndexAttributes(
		OutCollection, PointsToRemove, BranchesToRemove, FoliageInstancesToRemove);

	TMap<int32, int32> BranchesNewIDsToOldIDs;
	BranchesNewIDsToOldIDs.Reserve(RemoveEntriesResult.BranchesOldIDsToNewIDs.Num());
	for (const TPair<int32, int32>& Pair : RemoveEntriesResult.BranchesOldIDsToNewIDs)
	{
		BranchesNewIDsToOldIDs.Add(Pair.Value, Pair.Key);
	}

	RecomputeAttributes(OutCollection, SourceCollection, BranchesNewIDsToOldIDs);
}