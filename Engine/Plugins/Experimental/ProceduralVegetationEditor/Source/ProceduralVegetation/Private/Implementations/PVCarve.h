// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "PVCarve.generated.h"

UENUM(BlueprintType)
enum class ECarveBasis : uint8
{
	LengthFromRoot UMETA(DisplayName = "Length from root"),
	FromBottom UMETA(DisplayName = "From Bottom"),
	ZPosition UMETA(DisplayName = "Z Position"),
	Radius UMETA(DisplayName = "Radius")
};

USTRUCT()
struct FPVCarveParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Carve", meta=(Tooltip="Reference rule used to trim the plant.\n\nLengthFromRoot: distance measured from the plant's root/trunk base (along the branch graph). FromBottom: relative height starting from the plant's lowest point. ZPosition: absolute world-space Z. Radius: branch thickness."))
	ECarveBasis CarveBasis = ECarveBasis::LengthFromRoot;

	UPROPERTY(EditAnywhere, Category="Carve", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="How much of the structure to remove (0 = none, 1 = aggressive).\n\nControls how much of the structure is removed according to the chosen Carve Basis. Lower values keep more geometry; higher values trim more."))
	float Carve = 0.0f;
};

struct FPVCarve
{
	static void ApplyCarve(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection, const ECarveBasis CarveBasis,
	                       const float Carve);

	static void UpdatePointScales(PV::Facades::FPointFacade& PointFacadeOut, const PV::Facades::FPointFacade& PointFacadeSource,
	                              const TArray<int>& BranchPoints, const float LastPointScale, const int32 LastPointIndex, const float CarveRatio,
	                              const int32 EndIndex, const float FirstPointTargetPScale, const bool InSkipFirstPoint);

	static void RecomputeAttributes(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection,
	                                const TMap<int32, int32>& BranchesNewIDsToOldIDs);

	static void CarveFromTop(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection, const float Carve,
	                         const ECarveBasis CarveBasis);

	static void CarveFromBottom(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection, const float Carve);

	static void ComputeMetadata(TMap<int32, int32>& OutBranchNumbersToBranchIDs, TMap<int32, float>& OutBranchNumbersToLengthFromRoots,
	                            const PV::Facades::FBranchFacade& BranchFacadeSource, const PV::Facades::FPointFacade& PointFacadeSource);

	static void RemoveEntriesAndRecomputeAttributes(FManagedArrayCollection& OutCollection, const FManagedArrayCollection& SourceCollection,
	                                                TArray<bool>& PointsToRemove, TArray<bool>& BranchesToRemove,
	                                                TArray<bool>& FoliageInstancesToRemove);
};
