// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVFoliage.h"

#include "ProceduralVegetationModule.h"
#include "Utils/PVFloatRamp.h"
#include "Algo/RandomShuffle.h"
#include "Math/StaticSpatialIndex.h"

#include "Facades/PVAttributesNames.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVPointFacade.h"
#include "Helpers/PVDistributionHelper.h"
#include "Helpers/PVPhyllotaxyHelper.h"
#include "Helpers/PVUtilities.h"

using namespace PV::DistributionConditionUtils;
using PV::DistributionConditionUtils::FConditionAttributes;

namespace PVFoliage
{
	void RecomputeBranchFoliageIds(PV::Facades::FFoliageFacade& FoliageFacade)
	{
		// Re-calculate Foliage IDs for each branch
		TMap<int32, TArray<int32>> BranchIndexToFoliageIDs;
		for (int32 i = 0; i < FoliageFacade.NumFoliageEntries(); ++i)
		{
			const int32 BranchIndex = FoliageFacade.GetFoliageEntry(i).BranchId;
			TArray<int32>& IDs = BranchIndexToFoliageIDs.FindOrAdd(BranchIndex);
			IDs.Add(i);
		}
		for (const TPair<int32, TArray<int32>>& Pair : BranchIndexToFoliageIDs)
		{
			int32 BranchId = Pair.Key;
			const TArray<int32>& FoliageIDs = Pair.Value;
			FoliageFacade.SetFoliageIdsArray(BranchId, FoliageIDs);
		}
	}

	TArray<PV::DistributionHelper::FAttachmentPoint> FilterOverlappingPoints(
		PV::FFoliagePivotPointAttributeConstView PivotPointsAttribute,
		const TArray<PV::DistributionHelper::FAttachmentPoint>& AttachmentPoints,
		float ChainMaskDistance
	)
	{
		const int32 NumExistingPoints = PivotPointsAttribute.Num();
		if (NumExistingPoints == 0 || ChainMaskDistance <= 0)
		{
			return AttachmentPoints;
		}

		using FSpatialIndex = TStaticSpatialIndexRTree<
			int32,
			FStaticSpatialIndex::TNodeSorterNoSort<FStaticSpatialIndex::FSpatialIndexProfile3D>,
			FStaticSpatialIndex::FSpatialIndexProfile3D
		>;

		const float MaskRadius = ChainMaskDistance * 0.5f;

		TArray<TPair<FBox, int32>> IndexElements;
		IndexElements.Reserve(NumExistingPoints);
		for (int32 i = 0; i < NumExistingPoints; ++i)
		{
			const FVector Center(PivotPointsAttribute[i]);
			IndexElements.Emplace(FBox(Center - MaskRadius, Center + MaskRadius), i);
		}

		FSpatialIndex SpatialIndex;
		SpatialIndex.Init(MoveTemp(IndexElements));

		TArray<PV::DistributionHelper::FAttachmentPoint> FilteredAttachmentPoints;
		FilteredAttachmentPoints.Reserve(AttachmentPoints.Num());

		for (const auto& AP : AttachmentPoints)
		{
			bool bOverlaps = false;

			// The sphere query prunes the tree to candidates whose AABB intersects
			// the query sphere. The exact sphere-sphere check below rejects the
			// false positives that arise at AABB corners.
			SpatialIndex.ForEachIntersectingElement(
				FStaticSpatialIndex::FSphere(FVector(AP.Position), MaskRadius),
				[&](int32 PointIndex) -> bool
				{
					if (FVector3f::DistSquared(AP.Position, PivotPointsAttribute[PointIndex]) < FMath::Square(ChainMaskDistance))
					{
						bOverlaps = true;
						return false; // stops iteration early
					}
					return true;
				});

			if (!bOverlaps)
			{
				FilteredAttachmentPoints.Add(AP);
			}
		}

		return FilteredAttachmentPoints;
	}

	void DistributeFoliageEntries(
		FManagedArrayCollection& OutCollection,
		const TArray<FPVFoliageInfo>& FoliageInfos,
		const TArray<PV::DistributionHelper::FAttachmentPoint>& AttachmentPoints,
		const FPVDistributionConditionParams& DistributionConditions,
		FRandomStream& RandomStream,
		float ChainMaskDistance)
	{
		PV::Facades::FFoliageFacade FoliageFacadeOutput(OutCollection);

		const int32 FoliageInfosIndexOffset = FoliageFacadeOutput.AppendFoliageInfos(FoliageInfos);
		auto IsValidCandidate = [&](int32 CandidateIndex)
			{
				return FoliageInfos.IsValidIndex(CandidateIndex);
			};

		auto GetCandidateValue = [&](int32 CandidateIndex, EPVDistributionCondition Condition) -> float
			{
				return ConditionPicker::GetAttributeValue(FoliageInfos[CandidateIndex].Attributes, Condition);
			};
		
		PV::FFoliagePivotPointAttributeAccessor PivotPointsAttribute(OutCollection);

		const TArray<PV::DistributionHelper::FAttachmentPoint> FilteredAttachmentPoints = PVFoliage::FilterOverlappingPoints(
			PivotPointsAttribute.GetConstView(),
			AttachmentPoints,
			ChainMaskDistance
		);

		for (const auto& AttachmentPoint : FilteredAttachmentPoints)
		{
			if (FMath::Abs(AttachmentPoint.Scale) <= UE_KINDA_SMALL_NUMBER)
			{
				continue;
			}

			const int32 PickedFoliageIndex = ConditionPicker::PickCandidateIndex(
				FoliageInfos.Num(),
				DistributionConditions,
				AttachmentPoint.ConditionSample,
				RandomStream,
				GetCandidateValue,
				IsValidCandidate);

			if (PickedFoliageIndex == INDEX_NONE)
			{
				continue;
			}

			if (!FoliageInfos[PickedFoliageIndex].bUseAsMask && PV::Utilities::DoesAssetExist(FoliageInfos[PickedFoliageIndex].Mesh.ToSoftObjectPath()))
			{
				const int32 NewIndex = PivotPointsAttribute.AddElements(1);
				FoliageFacadeOutput.SetFoliageEntry(NewIndex, {
					.NameId = PickedFoliageIndex + FoliageInfosIndexOffset,
					.BranchId = AttachmentPoint.BranchIndex,
					.PivotPoint = AttachmentPoint.Position,
					.UpVector = AttachmentPoint.UpDirection,
					.NormalVector = AttachmentPoint.NormalDirection,
					.Scale = AttachmentPoint.Scale,
					.LengthFromRoot = AttachmentPoint.LengthFromRoot,
					.ConditionUpAlignment = AttachmentPoint.ConditionSample.UpAlignment,
					.ConditionTip = AttachmentPoint.ConditionSample.Tip,
					.ConditionLight = AttachmentPoint.ConditionSample.Light,
					.ConditionScale = AttachmentPoint.ConditionSample.Scale,
					.ConditionHealth = AttachmentPoint.ConditionSample.Health,
					.ConditionHeight = AttachmentPoint.ConditionSample.Height,
					.ConditionGeneration = AttachmentPoint.ConditionSample.Generation
				});
			}
		}

		RecomputeBranchFoliageIds(FoliageFacadeOutput);
	}

	FNormalizedPointCaches BuildNormalizedPointCaches(
		FManagedArrayCollection& OutCollection,
		const FPVDistributionConditionParams& DistributionConditions,
		TArray<float> PScalesOverride = {})
	{
		FNormalizedPointCaches NormalizedPointCaches;
		FNormalizationInputs Inputs;
		Inputs.bComputePScales    = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Scale);
		Inputs.bComputeLight      = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Light);
		Inputs.bComputeHealth     = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Health);
		Inputs.bComputeHeight     = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Height);
		Inputs.bComputeGeneration = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Generation);
		Inputs.PScalesOverride    = MoveTemp(PScalesOverride);

		BuildNormalizedPointCaches(OutCollection, Inputs, NormalizedPointCaches);
		return NormalizedPointCaches;
	}
};

void FPVFoliage::DistributeFoliageWithHormoneBasedSettings(
	FManagedArrayCollection& OutCollection,
	const FManagedArrayCollection& FoliageCollection,
	const FHormoneSettings::FDistributionSettings& DistributionSettings,
	const FHormoneSettings::FScaleSettings& ScaleSettings,
	const FHormoneSettings::FAxilSettings& AxilSettings,
	const FHormoneSettings::FPhyllotaxySettings& PhyllotaxySettings,
	const FPVDistributionVectorParams& VectorSettings,
	const FPVDistributionConditionParams& DistributionConditions,
	const int32 RandomSeed,
	float ChainMaskDistance,
	float TrunkOffset
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVFoliage::DistributeFoliageWithHormoneBasedSettings);

	if (!(DistributionSettings.InstanceSpacingRamp && ScaleSettings.ScaleRamp && AxilSettings.AxilAngleRamp))
	{
		return;
	}

	PV::Facades::FFoliageFacade FoliagePaletteFacade(FoliageCollection);
	if (const int32 NumFoliageMeshes = FoliagePaletteFacade.NumFoliageInfo(); NumFoliageMeshes == 0)
	{
		UE_LOGF(LogProceduralVegetation, Warning, "There are no foliage meshes available in input for distribution");
		return;
	}

	TArray<float> PScalesOverride;
	if (DistributionConditions.IsActiveCondition(EPVDistributionCondition::Scale))
	{
		auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(OutCollection);
		auto BudHormoneLevels    = PV::FBudHormoneLevelsAttribute::GetAttribute(OutCollection);
		PScalesOverride = PV::HormoneDistributionHelper::RemapPScalesToEthyleneThreshold(
			PointScaleAttribute,
			BudHormoneLevels,
			DistributionSettings.EthyleneThreshold);
	}

	const FNormalizedPointCaches NormalizedPointCaches = PVFoliage::BuildNormalizedPointCaches(
		OutCollection, DistributionConditions, MoveTemp(PScalesOverride));

	FRandomStream RandomStream;
	RandomStream.Initialize(RandomSeed);

	TArray<PV::DistributionHelper::FAttachmentPoint> AttachmentPoints;
	ComputeHormoneBasedAttachmentPoints(
		OutCollection,
		NormalizedPointCaches,
		DistributionSettings,
		ScaleSettings,
		AxilSettings,
		PhyllotaxySettings,
		VectorSettings,
		RandomStream,
		TrunkOffset,
		AttachmentPoints
	);

	PVFoliage::DistributeFoliageEntries(
		OutCollection,
		FoliagePaletteFacade.GetFoliageInfos(),
		AttachmentPoints,
		DistributionConditions,
		RandomStream,
		ChainMaskDistance
	);
}

void FPVFoliage::DistributeFoliageWithParametricSettings(
	FManagedArrayCollection& OutCollection,
	const FManagedArrayCollection& FoliageCollection,
	const FParametricSettings::FSpacingSettings& SpacingSettings,
	const FParametricSettings::FPhyllotaxySettings& PhyllotaxySettings,
	const FParametricSettings::FAngleSettings& AngleSettings,
	const FParametricSettings::FScaleSettings& ScaleSettings,
	const FPVDistributionVectorParams& VectorSettings,
	const FPVDistributionConditionParams& DistributionConditions,
	const int32 RandomSeed,
	float ChainMaskDistance,
	float TrunkOffset
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVFoliage::DistributeFoliageWithParametricSettings);

	if (!(SpacingSettings.SpacingRamp && AngleSettings.AxilAngleRamp && ScaleSettings.ScaleRamp))
	{
		return;
	}

	PV::Facades::FFoliageFacade FoliagePaletteFacade(FoliageCollection);
	if (const int32 NumFoliageMeshes = FoliagePaletteFacade.NumFoliageInfo(); NumFoliageMeshes == 0)
	{
		UE_LOGF(LogProceduralVegetation, Warning, "There are no foliage meshes available in input for distribution");
		return;
	}

	TArray<float> PScalesOverride;
	if (DistributionConditions.IsActiveCondition(EPVDistributionCondition::Scale))
	{
		auto BranchPointsAttribute       = PV::FBranchPointsAttribute::GetAttribute(OutCollection);
		auto PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::GetAttribute(OutCollection);
		auto PointScaleAttribute          = PV::FPointScaleAttribute::GetAttribute(OutCollection);
		PScalesOverride = PV::ParametricDistributionHelper::RemapPScalesToGradientRange(
			BranchPointsAttribute,
			PointLengthFromRootAttribute,
			PointScaleAttribute,
			SpacingSettings.RelativeStart,
			SpacingSettings.RelativeEnd);
	}

	const FNormalizedPointCaches NormalizedPointCaches = PVFoliage::BuildNormalizedPointCaches(
		OutCollection, DistributionConditions, MoveTemp(PScalesOverride));

	FRandomStream RandomStream;
	RandomStream.Initialize(RandomSeed);

	TArray<PV::DistributionHelper::FAttachmentPoint> AttachmentPoints;
	ComputeAttachmentPoints(
		OutCollection,
		NormalizedPointCaches,
		SpacingSettings,
		PhyllotaxySettings,
		AngleSettings,
		ScaleSettings,
		VectorSettings,
		RandomSeed,
		TrunkOffset,
		AttachmentPoints
	);

	PVFoliage::DistributeFoliageEntries(
		OutCollection,
		FoliagePaletteFacade.GetFoliageInfos(),
		AttachmentPoints,
		DistributionConditions,
		RandomStream,
		ChainMaskDistance
	);
}
