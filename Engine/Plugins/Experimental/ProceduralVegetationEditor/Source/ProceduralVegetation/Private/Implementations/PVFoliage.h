// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PVHormoneDistributionHelper.h"
#include "Helpers/PVParametricDistributionHelper.h"

struct FPVDistributionConditionParams;

using namespace PV::ParametricDistributionHelper;
using namespace PV::HormoneDistributionHelper;

struct FPVFoliage
{
	static void DistributeFoliageWithHormoneBasedSettings(
		FManagedArrayCollection& OutCollection,
		const FManagedArrayCollection& FoliageCollection,
		const FHormoneSettings::FDistributionSettings& DistributionSettings,
		const FHormoneSettings::FScaleSettings& ScaleSettings,
		const FHormoneSettings::FAxilSettings& AxilSettings,
		const FHormoneSettings::FPhyllotaxySettings& PhyllotaxySettings,
		const FPVDistributionVectorParams& VectorSettings,
		const FPVDistributionConditionParams& DistributionConditions,
		const int32 RandomSeed,
		float ChainMaskDistance,
		float TrunkOffset
	);

	static void DistributeFoliageWithParametricSettings(
		FManagedArrayCollection& OutCollection,
		const FManagedArrayCollection& FoliageCollection,
		const FParametricSettings::FSpacingSettings& SpacingSettings,
		const FParametricSettings::FPhyllotaxySettings& PhyllotaxySettings,
		const FParametricSettings::FAngleSettings& AngleSettings,
		const FParametricSettings::FScaleSettings& ScaleSettings,
		const FPVDistributionVectorParams& VectorSettings,
		const FPVDistributionConditionParams& DistributionConditions,
		const int32 RandomSeed,
		float ChainMaskDistance,
		float TrunkOffset
	);
};
