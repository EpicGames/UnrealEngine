// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrafter.h"

#include "ProceduralVegetationModule.h"

#include "Algo/RandomShuffle.h"

#include "DataTypes/PVGraftInfo.h"

#include "Facades/PVAttributesNames.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVMetaInfoFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"

#include "Helpers/PVAttributesHelper.h"
#include "Helpers/PVFoliageJSONHelper.h"
#include "Helpers/PVUtilities.h"

#include "Utils/PVFloatRamp.h"

using namespace PV::DistributionConditionUtils;
using namespace PV::ParametricDistributionHelper;
using PV::DistributionConditionUtils::FConditionAttributes;

namespace PVGrafter
{
	FNormalizedPointCaches BuildNormalizedPointCaches(
		FManagedArrayCollection& OutCollection,
		const FPVDistributionConditionParams& DistributionConditions,
		TArray<float> PScalesOverride = {}
	)
	{
		FNormalizedPointCaches NormalizedPointCaches;
		FNormalizationInputs Inputs;
		Inputs.bComputePScales = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Scale);
		Inputs.bComputeLight = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Light);
		Inputs.bComputeHealth = DistributionConditions.IsActiveCondition(EPVDistributionCondition::Health);
		Inputs.PScalesOverride = MoveTemp(PScalesOverride);

		BuildNormalizedPointCaches(
			OutCollection,
			Inputs,
			NormalizedPointCaches);

		return NormalizedPointCaches;
	}

	void RecomputeLightDetection(FManagedArrayCollection& OutCollection)
	{
		auto PointHullGradientAttribute = PV::FPointHullGradientAttribute::AddAttribute(OutCollection);
		auto PointGroundGradientAttribute = PV::FPointGroundGradientAttribute::AddAttribute(OutCollection);
		auto BudLightDetectedAttribute = PV::FBudLightDetectedAttribute::FindAttribute(OutCollection);
		auto BudStatusAttribute = PV::FBudStatusAttribute::FindAttribute(OutCollection);
		auto PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(OutCollection);
		auto BranchPointsAttribute = PV::FBranchPointsAttribute::FindAttribute(OutCollection);
		auto BranchChildrenAttribute = PV::FBranchChildrenAttribute::FindAttribute(OutCollection);
		auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(OutCollection);
		auto BranchNumberAttribute = PV::FBranchNumberAttribute::FindAttribute(OutCollection);

		if (!ensure(PV::ValidateAttributeCollection(
			PointHullGradientAttribute,
			PointGroundGradientAttribute,
			BudLightDetectedAttribute,
			BudStatusAttribute,
			PointPositionAttribute,
			BranchPointsAttribute,
			BranchChildrenAttribute,
			BranchParentNumberAttribute,
			BranchNumberAttribute
		)))
		{
			return;
		}

		PV::AttributesHelper::ComputePointGroundGradient({
			PointGroundGradientAttribute,
			PointPositionAttribute
		});

		PV::AttributesHelper::ComputePointHullGradient({
			PointHullGradientAttribute,
			PointPositionAttribute,
			BranchPointsAttribute,
			BranchParentNumberAttribute
		});

		PV::AttributesHelper::EstimateBudLightDetected({
			PointHullGradientAttribute,
			PointGroundGradientAttribute,
			BudStatusAttribute,
			BudLightDetectedAttribute,
			BranchPointsAttribute,
			BranchChildrenAttribute,
			BranchParentNumberAttribute,
			BranchNumberAttribute
		});
	}

	void DistributeGrafts(
		FManagedArrayCollection& OutCollection,
		TArray<FManagedArrayCollection>& SourceGraftCollections,
		const TArray<PV::DistributionHelper::FAttachmentPoint>& AttachmentPoints,
		const FPVDistributionConditionParams& DistributionConditions,
		FRandomStream& RandomStream,
		bool bRecomputeLightDetected
	)
	{
		auto BudNumberAttribute = PV::FPointBudNumberAttribute::GetAttribute(OutCollection);
		auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(OutCollection);
		auto PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::GetAttribute(OutCollection);
		if (!ensure(PV::ValidateAttributeCollection(
			BudNumberAttribute,
			BranchPointsAttribute,
			PointLengthFromRootAttribute
		)))
		{
			return;
		}

		if (!ensure(BudNumberAttribute.Num() > 0))
		{
			return;
		}

		// We extract all individual plants from the graft source as separate collections for simplicity
		TArray<FPVGraftInfo> CandidateGraftData;
		TArray<FManagedArrayCollection> GraftSourcesExtracted;
		for (FManagedArrayCollection& Source : SourceGraftCollections)
		{
			const PV::Facades::FMetaInfoFacade MetaInfoFacade(Source);
			const FPVGraftInfo GraftAttributes = MetaInfoFacade.GetGraftAttributes();
			if (!MetaInfoFacade.GraftEntryUseAsMask())
			{
				TArray<FManagedArrayCollection> IndividualPlants = FPVGrafter::ExtractIndividualPlants(Source);
				for (FManagedArrayCollection& IndividualPlant : IndividualPlants)
				{
					GraftSourcesExtracted.Add(MoveTemp(IndividualPlant));
					CandidateGraftData.Add(GraftAttributes);
				}
			}
			else
			{
				GraftSourcesExtracted.AddDefaulted();
				CandidateGraftData.Add(GraftAttributes);
			}
		}

		check(GraftSourcesExtracted.Num() > 0);

		const auto IsValidCandidate = [&CandidateGraftData](int32 CandidateIndex) -> bool
		{
			return CandidateGraftData.IsValidIndex(CandidateIndex);
		};

		auto GetCandidateValue = [&](const int32 CandidateIndex, const EPVDistributionCondition Condition) -> float
			{
				return ConditionPicker::GetAttributeValue(CandidateGraftData[CandidateIndex].Attributes, Condition);
			};

		using FBranchSegmentKey = TPair<int32, int32>;
		struct FInterpolatedPointValue
		{
			float PointAlpha;
			int32 AttachmentPointIndex;
		};

		TMap<FBranchSegmentKey, TArray<FInterpolatedPointValue>> InterpolatedPoints;
		InterpolatedPoints.Reserve(AttachmentPoints.Num());

		const auto FindExistingSegmentPoint = [&InterpolatedPoints](const FBranchSegmentKey& BranchSegmentKey, float PointAlpha)->int32
		{
			constexpr float MinPointAlphaDist = 0.01f;

			if (FMath::IsNearlyEqual(PointAlpha, 0.f, MinPointAlphaDist))
			{
				return BranchSegmentKey.Key;
			}
			
			if (FMath::IsNearlyEqual(PointAlpha, 1.f, MinPointAlphaDist))
			{
				return BranchSegmentKey.Value;
			}

			TArray<FInterpolatedPointValue>* InterpolatedPointValues = InterpolatedPoints.Find(BranchSegmentKey);

			const FInterpolatedPointValue* ExistingValue = InterpolatedPointValues 
				? InterpolatedPointValues->FindByPredicate([&](const auto& X) { return FMath::IsNearlyEqual(X.PointAlpha, PointAlpha, MinPointAlphaDist); })
				: nullptr;
			if (ExistingValue)
			{
				return ExistingValue->AttachmentPointIndex;
			}
			
			return INDEX_NONE;
		};

		for (const PV::DistributionHelper::FAttachmentPoint& AttachmentPoint : AttachmentPoints)
		{
			const int32 PickedCandidateIndex = ConditionPicker::PickCandidateIndex(
				GraftSourcesExtracted.Num(),
				DistributionConditions,
				AttachmentPoint.ConditionSample,
				RandomStream,
				GetCandidateValue,
				IsValidCandidate
			);

			if (PickedCandidateIndex == INDEX_NONE)
			{
				continue;
			}
			
			if (CandidateGraftData[PickedCandidateIndex].bUseAsMask)
			{
				continue;
			}

			int32 AttachmentPointIndex = INDEX_NONE;

			const int32 MaxBudNumber = *Algo::MaxElement(BudNumberAttribute);
			int32 CurrentBudNumber = MaxBudNumber + 1;
			
			if (AttachmentPoint.bIsTipInstance)
			{
				AttachmentPointIndex = AttachmentPoint.TipPointIndex;
			}
			else
			{
				// PointIndexA/B are always ordered root->tip by the distribution helpers, so PointAlpha
				// shares one reference frame per segment. The key is intentionally NOT orientation-normalized:
				// merging (A,B) with (B,A) would compare alphas measured in opposite directions.
				const auto BranchSegmentKey = FBranchSegmentKey(AttachmentPoint.PointIndexA, AttachmentPoint.PointIndexB);
				AttachmentPointIndex = FindExistingSegmentPoint(BranchSegmentKey, AttachmentPoint.PointAlpha);
				
				if (AttachmentPointIndex == INDEX_NONE)
				{
					AttachmentPointIndex = PV::Utilities::AddInterpolatedPointToCollection(
						OutCollection,
						AttachmentPoint.BranchIndex,
						AttachmentPoint.PointIndexA,
						AttachmentPoint.PointIndexB,
						AttachmentPoint.PointAlpha,
						CurrentBudNumber);

					CurrentBudNumber++;

					InterpolatedPoints.FindOrAdd(BranchSegmentKey).Add({ AttachmentPoint.PointAlpha, AttachmentPointIndex });

					auto& Points = BranchPointsAttribute[AttachmentPoint.BranchIndex];
					Points.Add(AttachmentPointIndex);
					Points.Sort([PointLengthFromRootAttribute](const int32& A, const int32& B)
					{
						return PointLengthFromRootAttribute[A] < PointLengthFromRootAttribute[B];
					});
				}
			}

			FPVGrafter::FPVGraftAttributes Attributes
			{
				.BranchIndex = AttachmentPoint.BranchIndex,
				.PointIndex = AttachmentPointIndex,
				.CurrentBudNumber = CurrentBudNumber,
				.AttachmentPoint = AttachmentPoint.Position,
				.UpVector = AttachmentPoint.UpDirection,
				.NormalVector = AttachmentPoint.NormalDirection,
				.Scale = AttachmentPoint.Scale,
				.LengthFromRoot = AttachmentPoint.LengthFromRoot,
				.bIsTipInstance = AttachmentPoint.bIsTipInstance
			};

			check(GraftSourcesExtracted.Num() > 0);
			const FManagedArrayCollection& PickedSourceGraft = GraftSourcesExtracted[PickedCandidateIndex];
			FPVGrafter::SetGraft(OutCollection, PickedSourceGraft, Attributes);
		}

		if (bRecomputeLightDetected)
		{
			PVGrafter::RecomputeLightDetection(OutCollection);
		}
	}
}

template <typename Fn>
void FPVGrafter::ForEachGraftPointExcludingRoot(const int32 NumPoints, const int32 RootIndex, Fn&& Func)
{
	for (int32 SourceIndex = 0, TargetIndex = 0; SourceIndex < NumPoints; SourceIndex++)
	{
		if (SourceIndex == RootIndex)
		[[likely]]
		{
			continue;
		}

		Func(SourceIndex, TargetIndex);
		TargetIndex++;
	}
}


FTransform FPVGrafter::GetPointTransform(const FVector3f AttachmentPoint, const FVector3f UpVector, const FVector3f NormalVector, const float Scale)
{
	const FVector UpVectorN = FVector(UpVector).GetSafeNormal();
	if (UpVectorN.IsNearlyZero())
	{
		return FTransform(FQuat::Identity, FVector(AttachmentPoint), FVector(Scale));
	}

	const FVector NormalVectorN = FVector(NormalVector).GetSafeNormal();

	FVector OrthogonalizedNormalVectorN =
		(NormalVectorN - FVector::DotProduct(NormalVectorN, UpVectorN) * UpVectorN).GetSafeNormal();
	if (OrthogonalizedNormalVectorN.IsNearlyZero())
	{
		FVector TempVector;
		UpVectorN.FindBestAxisVectors(TempVector, OrthogonalizedNormalVectorN);
	}

	const FQuat RotationQuat = FQuat(FRotationMatrix::MakeFromYZ(OrthogonalizedNormalVectorN, UpVectorN));
	return FTransform(RotationQuat, FVector(AttachmentPoint), FVector(Scale));
}

void FPVGrafter::RescaleBranchForGraftingOnTip(FManagedArrayCollection& OutCollection, const int32 BranchIndex, const int32 BranchTipPointIndex,
                                               const float GraftTrunkLength)
{
	PV::Facades::FPointFacade PointFacade(OutCollection);
	PV::Facades::FBranchFacade BranchFacade(OutCollection);

	const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
	if (BranchPoints.Num() < 2)
	{
		return;
	}

	TArray<int32> SortedPointIndices = BranchPoints;
	SortedPointIndices.Sort([PointFacade](const int32& A, const int32& B)
		{
			return PointFacade.GetLengthFromRoot(A) < PointFacade.GetLengthFromRoot(B);
		});

	const int32 RootPointIndex = SortedPointIndices[0];

	const float RootPointLFR = PointFacade.GetLengthFromRoot(RootPointIndex);
	const float TipPointLFR = PointFacade.GetLengthFromRoot(BranchTipPointIndex);
	const float BranchLength = TipPointLFR - RootPointLFR;

	const float CombinedLengths = BranchLength + GraftTrunkLength;
	if (CombinedLengths <= KINDA_SMALL_NUMBER)
	{
		return;
	}

	const float Ratio = BranchLength / CombinedLengths;
	const float RootPointScale = PointFacade.GetPointScale(RootPointIndex);
	const float TipPointScale = PointFacade.GetPointScale(BranchTipPointIndex);
	const float NewTipPointScale = FMath::Lerp(RootPointScale, TipPointScale, Ratio);

	int32 i = 1;
	while (SortedPointIndices[i - 1] != BranchTipPointIndex)
	{
		const int32 PtIndex = SortedPointIndices[i];
		const float PtScale = PointFacade.GetPointScale(PtIndex);

		const float NewPtScale = FMath::GetMappedRangeValueClamped(
			FVector2f(RootPointScale, TipPointScale),
			FVector2f(RootPointScale, NewTipPointScale),
			PtScale);

		TArray<float> BudLateralMeristem = PointFacade.GetBudLateralMeristem(PtIndex);
		if (ensure(BudLateralMeristem.Num() > 0))
		{
			BudLateralMeristem[0] = NewPtScale * 0.01f; // Multiply with 0.01 to convert from cm -> m
			PointFacade.SetBudLateralMeristem(PtIndex, BudLateralMeristem);
		}
		PointFacade.SetPointScale(PtIndex, NewPtScale);
		i++;
	}
}

void FPVGrafter::SetGraft(FManagedArrayCollection& OutCollection,
                          const FManagedArrayCollection& SourceGraftCollection,
                          const FPVGraftAttributes& InAttributes)
{
	// Transform and set points
	PV::Facades::FPointFacade PointFacadeOut(OutCollection);
	PV::Facades::FPointFacade PointFacadeSourceGraft(SourceGraftCollection);
	const PV::Facades::FBranchFacade BranchFacadeSourceGraft(SourceGraftCollection);

	int32 SourceGraftTrunkBranchIndex = INDEX_NONE;
	for (int32 i = 0; i < BranchFacadeSourceGraft.GetElementCount(); ++i)
	{
		if (BranchFacadeSourceGraft.IsTrunk(i))
		{
			SourceGraftTrunkBranchIndex = i;
			break;
		}
	}
	check(SourceGraftTrunkBranchIndex != INDEX_NONE)

	const TArray<int32>& SourceGraftTrunkPoints = BranchFacadeSourceGraft.GetPoints(SourceGraftTrunkBranchIndex);
	check(SourceGraftTrunkPoints.Num() > 0);
	const int32 SourceGraftTrunkRootPointIndex = SourceGraftTrunkPoints[0];

	// Compute graft trunk length
	float GraftTrunkLength = 0;
	for (int32 PtIndex : SourceGraftTrunkPoints)
	{
		GraftTrunkLength = FMath::Max(GraftTrunkLength, PointFacadeSourceGraft.GetLengthFromRoot(PtIndex));
	}
	GraftTrunkLength *= InAttributes.Scale;

	// Get max bud age from source graft and add to all points' bud age in the source skeleton
	const int32 MaxBudAgeInSourceGraft = PointFacadeSourceGraft.GetMaxBudAge();
	PointFacadeOut.AddToBudAgeForAllPoints(MaxBudAgeInSourceGraft);

	const TManagedArray<int32>& SourceSkeletonBudNumbers = PointFacadeOut.GetBudNumbersAttribute();
	check(SourceSkeletonBudNumbers.Num() > 0);
	int32 MaxBudNumber = InAttributes.CurrentBudNumber;
	const TManagedArray<int32>& SourceGraftBudNumbers = PointFacadeSourceGraft.GetBudNumbersAttribute();
	TArray<int32> TransformedBudNumbers;
	const int32 NumOfPointsInSourceGraft = PointFacadeSourceGraft.GetElementCount();
	// We will ignore the main trunk's root point when adding points to skeleton
	const int32 NumberOfPointsToAddToSkeleton = NumOfPointsInSourceGraft - 1;
	TransformedBudNumbers.SetNum(NumberOfPointsToAddToSkeleton);
	TMap<int32, int32> SourceGraftOldToNewBudNumbers;
	SourceGraftOldToNewBudNumbers.Reserve(NumberOfPointsToAddToSkeleton);
	int32 SourcePointIndex = 0;
	int32 TargetPointIndex = 0;
	while (SourcePointIndex < NumOfPointsInSourceGraft)
	{
		const int32 OldBudNumber = SourceGraftBudNumbers[SourcePointIndex];
		const int32 NewBudNumber = OldBudNumber + MaxBudNumber + 1;
		SourceGraftOldToNewBudNumbers.Add(OldBudNumber, NewBudNumber);
		if (SourcePointIndex != SourceGraftTrunkRootPointIndex)
		[[likely]]
		{
			TransformedBudNumbers[TargetPointIndex] = NewBudNumber;
			TargetPointIndex++;
		}
		SourcePointIndex++;
	}

	const int32 PointFacadeOutStartIndex = PointFacadeOut.GetElementCount();
	PointFacadeOut.SetBudNumbersFromIndex(TransformedBudNumbers, PointFacadeOutStartIndex);

	const TManagedArray<FVector3f>& PointPositions = PointFacadeSourceGraft.GetPositions();
	const FVector GraftRootPosition = FVector(PointPositions[SourceGraftTrunkRootPointIndex]);
	const FTransform Transform = GetPointTransform(InAttributes.AttachmentPoint, InAttributes.UpVector, InAttributes.NormalVector,
		InAttributes.Scale);
	TArray<FVector3f> TransformedPoints;
	TransformedPoints.SetNum(NumberOfPointsToAddToSkeleton);

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				const FVector SourcePointPosition = FVector(PointPositions[SourcePointIndex]);
				const FVector SourcePositionWrtRoot = SourcePointPosition - GraftRootPosition;
				const FVector RotatedScaledOffsetPosition = Transform.TransformPosition(SourcePositionWrtRoot);
				TransformedPoints[TargetPointIndex] = FVector3f(RotatedScaledOffsetPosition);
			});

	PointFacadeOut.SetPositionsFromIndex(TransformedPoints, PointFacadeOutStartIndex);

	if (InAttributes.bIsTipInstance)
	{
		RescaleBranchForGraftingOnTip(OutCollection, InAttributes.BranchIndex, InAttributes.PointIndex, GraftTrunkLength);
	}

	const TManagedArray<float>& SourceGraftPointScales = PointFacadeSourceGraft.GetPointScales();
	TArray<float> TransformedPointScales;
	TransformedPointScales.SetNum(NumberOfPointsToAddToSkeleton);

	const float ParentPointPointScale = PointFacadeOut.GetPointScale(InAttributes.PointIndex);
	float SourceGraftRootPointScale = SourceGraftPointScales[SourceGraftTrunkRootPointIndex];

	if (FMath::IsNearlyZero(SourceGraftRootPointScale))
	[[unlikely]]
	{
		SourceGraftRootPointScale = 1;
	}
	const float ParentToGraftRootPointScaleRatio = ParentPointPointScale / SourceGraftRootPointScale;

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				TransformedPointScales[TargetPointIndex] =
					SourceGraftPointScales[SourcePointIndex] * ParentToGraftRootPointScaleRatio;
			});

	PointFacadeOut.SetPointScalesFromIndex(TransformedPointScales, PointFacadeOutStartIndex);
	
	TArray<float> TransformedSeedPScales;
	TransformedSeedPScales.SetNum(NumberOfPointsToAddToSkeleton);

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				float SeedPScaleValue = 0.f;
				PointFacadeSourceGraft.GetSeedPScale(SourcePointIndex, SeedPScaleValue);
				TransformedSeedPScales[TargetPointIndex] = SeedPScaleValue * ParentToGraftRootPointScaleRatio;
			});

	PointFacadeOut.SetSeedPScalesFromIndex(TransformedSeedPScales, PointFacadeOutStartIndex);
	
	TArray<float> TransformedSeedPScaleRatios;
	TransformedSeedPScaleRatios.SetNum(NumberOfPointsToAddToSkeleton);

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				float SeedPScaleRatioValue = 1.f;
				PointFacadeSourceGraft.GetSeedPScaleRatio(SourcePointIndex, SeedPScaleRatioValue);
				TransformedSeedPScaleRatios[TargetPointIndex] = SeedPScaleRatioValue;
			});

	PointFacadeOut.SetSeedPScaleRatiosFromIndex(TransformedSeedPScaleRatios, PointFacadeOutStartIndex);
	
	const TManagedArray<float>& LFRs = PointFacadeSourceGraft.GetLengthFromRootsArray();
	TArray<float> TransformedLFRs;
	TransformedLFRs.SetNum(NumberOfPointsToAddToSkeleton);

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				TransformedLFRs[TargetPointIndex] = ((LFRs[SourcePointIndex] - LFRs[SourceGraftTrunkRootPointIndex]) * InAttributes.Scale)
					+ InAttributes.LengthFromRoot;
			});

	PointFacadeOut.SetLFRsFromIndex(TransformedLFRs, PointFacadeOutStartIndex);

	const TManagedArray<float>& LengthFromSeeds = PointFacadeSourceGraft.GetLengthFromSeedsArray();
	TArray<float> TransformedLengthFromSeeds;
	TransformedLengthFromSeeds.SetNum(NumberOfPointsToAddToSkeleton);

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				TransformedLengthFromSeeds[TargetPointIndex] = ((LengthFromSeeds[SourcePointIndex] - LengthFromSeeds[SourceGraftTrunkRootPointIndex])
					* InAttributes.Scale) + InAttributes.LengthFromRoot;
			});

	PointFacadeOut.SetLengthFromSeedsArrayFromIndex(TransformedLengthFromSeeds, PointFacadeOutStartIndex);

	const TManagedArray<TArray<float>>& BudLightDetectedArrays = PointFacadeSourceGraft.GetBudLightDetectedArrays();

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				PointFacadeOut.SetBudLightDetected(TargetPointIndex + PointFacadeOutStartIndex, BudLightDetectedArrays[SourcePointIndex]);
			});


	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				const TArray<int>& BudDevelopmentValuesInGraftPoint = PointFacadeSourceGraft.GetBudDevelopment(SourcePointIndex);
				const TArray<int>& BudDevelopmentValuesInAttachmentPoint = PointFacadeOut.GetBudDevelopment(InAttributes.PointIndex);
				const int32 NumOfValuesInSource = BudDevelopmentValuesInGraftPoint.Num();
				TArray<int> TransformedValues;
				TransformedValues.SetNum(NumOfValuesInSource);
				check(NumOfValuesInSource == 6);

				// 0_Generation
				TransformedValues[0] = BudDevelopmentValuesInGraftPoint[0] + BudDevelopmentValuesInAttachmentPoint[0];

				// 1_BudAge
				// Max bud age from source graft has already been added to all the points in the source skeleton
				TransformedValues[1] = BudDevelopmentValuesInGraftPoint[1];

				// 2_BranchAge
				TransformedValues[2] = BudDevelopmentValuesInGraftPoint[2] + BudDevelopmentValuesInAttachmentPoint[2];

				// 3_AgeSenescence
				TransformedValues[3] = 0;

				// 4_LightSenescence
				TransformedValues[4] = 0;

				// 5_RelativeBudAge
				TransformedValues[5] = TransformedValues[0];

				PointFacadeOut.SetBudDevelopment(TargetPointIndex + PointFacadeOutStartIndex, TransformedValues);
			});

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				const TArray<float>& BudHormoneLevelsInGraftPoint = PointFacadeSourceGraft.GetBudHormoneLevels(SourcePointIndex);
				// keeping them the same as original graft for now
				// TODO: recompute and update
				PointFacadeOut.SetBudHormoneLevels(TargetPointIndex + PointFacadeOutStartIndex, BudHormoneLevelsInGraftPoint);
			});

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				const TArray<FVector3f>& BudDirectionsInGraftPoint = PointFacadeSourceGraft.GetBudDirection(SourcePointIndex);
				const int32 NumOfValues = BudDirectionsInGraftPoint.Num();
				TArray<FVector3f> TransformedValues;
				TransformedValues.SetNum(NumOfValues);

				for (int32 j = 0; j < NumOfValues; ++j)
				{
					TransformedValues[j] = FVector3f(Transform.TransformVectorNoScale(FVector(BudDirectionsInGraftPoint[j])));
				}
				PointFacadeOut.SetBudDirections(TargetPointIndex + PointFacadeOutStartIndex, TransformedValues);
			});

	const TManagedArray<TArray<float>>& BudLateralMeristemArrays = PointFacadeSourceGraft.GetBudLateralMeristemArrays();

	// Index 4 (ParentDot) is preserved under the pure-rotation Transform applied to bud directions, except for the
	// graft trunk's first non-root point in the non-tip case; that exception is patched after BudDirections are written.
	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				TArray<float> LateralMeristemValues = BudLateralMeristemArrays[SourcePointIndex];
				check(LateralMeristemValues.Num() > 0);
				LateralMeristemValues[0] *= ParentToGraftRootPointScaleRatio;
				LateralMeristemValues[5] = TransformedLFRs[TargetPointIndex];
				PointFacadeOut.SetBudLateralMeristem(TargetPointIndex + PointFacadeOutStartIndex, LateralMeristemValues);
			});

	// Patch ParentDot for the graft trunk's first non-root point in the non-tip case.
	if (!InAttributes.bIsTipInstance && SourceGraftTrunkPoints.Num() > 1)
	{
		const int32 SourceFirstNonRootTrunkPointIndex = SourceGraftTrunkPoints[1];
		const int32 TargetFirstNonRootTrunkPointIndex = PointFacadeOutStartIndex
			+ SourceFirstNonRootTrunkPointIndex
			- (SourceFirstNonRootTrunkPointIndex > SourceGraftTrunkRootPointIndex ? 1 : 0);

		const FVector3f TransformedApical = PointFacadeOut.GetBudDirection(TargetFirstNonRootTrunkPointIndex)[PV::Facades::BudDirectionsApical];
		const FVector3f HostApical = PointFacadeOut.GetBudDirection(InAttributes.PointIndex)[PV::Facades::BudDirectionsApical];

		TArray<float> LateralMeristemValues = PointFacadeOut.GetBudLateralMeristem(TargetFirstNonRootTrunkPointIndex);
		LateralMeristemValues[4] = FVector3f::DotProduct(HostApical, TransformedApical);
		PointFacadeOut.SetBudLateralMeristem(TargetFirstNonRootTrunkPointIndex, LateralMeristemValues);
	}

	ForEachGraftPointExcludingRoot(NumOfPointsInSourceGraft, SourceGraftTrunkRootPointIndex,
		[&](const int32 SourcePointIndex, const int32 TargetPointIndex)
			{
				TArray<int> BudStatusInGraftPoint;
				PointFacadeSourceGraft.GetBudStatus(SourcePointIndex, BudStatusInGraftPoint);
				PointFacadeOut.SetBudStatus(TargetPointIndex + PointFacadeOutStartIndex, BudStatusInGraftPoint);
			});

	// Transform and set branches
	PV::Facades::FBranchFacade BranchFacadeOut(OutCollection);

	const int32 NumOfBranchesInSource = BranchFacadeSourceGraft.GetElementCount();
	const int32 NumOfBranchesInTarget = BranchFacadeOut.GetElementCount();

	int32 NumOfBranchesToAdd = NumOfBranchesInSource - (InAttributes.bIsTipInstance
		? 1
		: 0);
	BranchFacadeOut.AddElements(NumOfBranchesToAdd);

	int32 SourceIndex = 0;
	int32 TargetIndex = NumOfBranchesInTarget;
	while (SourceIndex < NumOfBranchesInSource)
	{
		TArray<int32> SourceBranchPoints = BranchFacadeSourceGraft.GetPoints(SourceIndex);

		const int32 NumOfPointsInSource = SourceBranchPoints.Num();
		TArray<int32> TransformedBranchPoints;
		TransformedBranchPoints.Reserve(NumOfPointsInSource);

		for (int32 j = 0; j < NumOfPointsInSource; ++j)
		{
			if (const int32 BranchPoint = SourceBranchPoints[j];
				BranchPoint < SourceGraftTrunkRootPointIndex)
			{
				TransformedBranchPoints.Add(PointFacadeOutStartIndex + BranchPoint);
			}
			else if (BranchPoint > SourceGraftTrunkRootPointIndex)
			{
				TransformedBranchPoints.Add(PointFacadeOutStartIndex + BranchPoint - 1);
			}
		}

		if (SourceIndex == SourceGraftTrunkBranchIndex)
		[[unlikely]]
		{
			if (InAttributes.bIsTipInstance)
			{
				TArray<int32> TargetBranchPoints = BranchFacadeOut.GetPoints(InAttributes.BranchIndex);
				TargetBranchPoints.Append(TransformedBranchPoints);
				BranchFacadeOut.SetPoints(InAttributes.BranchIndex, TargetBranchPoints);
			}
			else
			{
				TransformedBranchPoints.Insert(InAttributes.PointIndex, 0);
				BranchFacadeOut.SetPoints(TargetIndex, TransformedBranchPoints);
				TargetIndex++;
			}
		}
		else
		[[likely]]
		{
			BranchFacadeOut.SetPoints(TargetIndex, TransformedBranchPoints);
			TargetIndex++;
		}

		SourceIndex++;
	}


	check(NumOfBranchesInTarget > 0);
	const int32 BranchFacadeOutStartIndex = NumOfBranchesInTarget;
	const int32 MaxBranchNumberInTarget = *Algo::MaxElement(BranchFacadeOut.GetBranchNumbers());
	const int32 BranchNumberBeingAttachedToInTarget = BranchFacadeOut.GetBranchNumber(InAttributes.BranchIndex);
	const TManagedArray<int32>& BranchNumbers = BranchFacadeSourceGraft.GetBranchNumbers();
	TMap<int32, int32> SourceOldBranchNumbersToNewBranchNumbers;
	SourceOldBranchNumbersToNewBranchNumbers.Reserve(NumOfBranchesInSource);

	for (int32 i = 0; i < NumOfBranchesInSource; ++i)
	{
		const int32 NewBranchNumber = (i == SourceGraftTrunkBranchIndex && InAttributes.bIsTipInstance)
			? BranchNumberBeingAttachedToInTarget
			: BranchNumbers[i] + MaxBranchNumberInTarget;
		SourceOldBranchNumbersToNewBranchNumbers.Add(BranchNumbers[i], NewBranchNumber);
	}

	TArray<int32> TransformedBranchNumbers;
	TransformedBranchNumbers.SetNum(NumOfBranchesToAdd);
	SourceIndex = 0;
	TargetIndex = 0;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			TransformedBranchNumbers[TargetIndex] = SourceOldBranchNumbersToNewBranchNumbers[BranchNumbers[SourceIndex]];
			TargetIndex++;
		}

		SourceIndex++;
	}

	BranchFacadeOut.SetBranchNumbersFromIndex(TransformedBranchNumbers, BranchFacadeOutStartIndex);

	const TManagedArray<int32>& ParentBranchNumbers = BranchFacadeSourceGraft.GetParentBranchNumbers();
	TArray<int32> TransformedParentBranchNumbers;
	TransformedParentBranchNumbers.SetNum(NumOfBranchesToAdd);
	SourceIndex = 0;
	TargetIndex = 0;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			TransformedParentBranchNumbers[TargetIndex] = ParentBranchNumbers[SourceIndex] == 0
				? BranchNumberBeingAttachedToInTarget
				: SourceOldBranchNumbersToNewBranchNumbers[ParentBranchNumbers[SourceIndex]];
			TargetIndex++;
		}

		SourceIndex++;
	}

	BranchFacadeOut.SetParentBranchNumbersFromIndex(TransformedParentBranchNumbers, BranchFacadeOutStartIndex);

	const TManagedArray<int32>& BranchHierarchyNumbers = BranchFacadeSourceGraft.GetBranchHierarchyNumbers();
	TArray<int32> TransformedBranchHierarchyNumbers;
	TransformedBranchHierarchyNumbers.SetNum(NumOfBranchesToAdd);
	const int32 BranchHierarchyNumberOfTarget = BranchFacadeOut.GetBranchHierarchyNumber(InAttributes.BranchIndex);
	SourceIndex = 0;
	TargetIndex = 0;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			TransformedBranchHierarchyNumbers[TargetIndex] = BranchHierarchyNumbers[SourceIndex] + BranchHierarchyNumberOfTarget;
			TargetIndex++;
		}

		SourceIndex++;
	}

	BranchFacadeOut.SetBranchHierarchyNumbersFromIndex(TransformedBranchHierarchyNumbers, BranchFacadeOutStartIndex);

	const TManagedArray<int32>& BranchSourceBudNumbers = BranchFacadeSourceGraft.GetBranchSourceBudNumbers();
	TArray<int32> TransformedBranchSourceBudNumbers;
	TransformedBranchSourceBudNumbers.SetNum(NumOfBranchesToAdd);

	SourceIndex = 0;
	TargetIndex = 0;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			const int32 ParentPointSourceBudNumber = PointFacadeOut.GetBudNumber(InAttributes.PointIndex);
			const int32 SourceBudNumber = BranchSourceBudNumbers[SourceIndex];
			const int32 NewSourceBudNumber = SourceBudNumber == 1
				? ParentPointSourceBudNumber
				: SourceGraftOldToNewBudNumbers[SourceBudNumber];
			TransformedBranchSourceBudNumbers[TargetIndex] = NewSourceBudNumber;
			TargetIndex++;
		}

		SourceIndex++;
	}

	BranchFacadeOut.SetBranchSourceBudNumbersFromIndex(TransformedBranchSourceBudNumbers, BranchFacadeOutStartIndex);


	SourceIndex = 0;
	TargetIndex = NumOfBranchesInTarget;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			const TArray<int32>& SourceBranchChildren = BranchFacadeSourceGraft.GetChildren(SourceIndex);
			const int32 NumOfChildrenInSource = SourceBranchChildren.Num();
			TArray<int32> TransformedChildren;
			TransformedChildren.Reserve(NumOfChildrenInSource);

			for (int32 j = 0; j < NumOfChildrenInSource; ++j)
			{
				if (const int32* NewNumber = SourceOldBranchNumbersToNewBranchNumbers.Find(SourceBranchChildren[j]))
				{
					TransformedChildren.Add(*NewNumber);
				}
			}

			BranchFacadeOut.SetChildren(TargetIndex, TransformedChildren);
			TargetIndex++;
		}

		SourceIndex++;
	}

	TArray<int32> HostBranchChildren = BranchFacadeOut.GetChildren(InAttributes.BranchIndex);

	if (InAttributes.bIsTipInstance)
	{
		// The graft trunk merged into the host branch, so the trunk's source children
		// become new children of the host branch.
		const TArray<int32>& SourceTrunkChildren = BranchFacadeSourceGraft.GetChildren(SourceGraftTrunkBranchIndex);
		for (const int32 SourceChildBranchNumber : SourceTrunkChildren)
		{
			if (const int32* NewNumber = SourceOldBranchNumbersToNewBranchNumbers.Find(SourceChildBranchNumber))
			{
				HostBranchChildren.Add(*NewNumber);
			}
		}
	}
	else
	{
		// The graft trunk is a new branch; add it as a direct child of the host branch
		const int32 GraftTrunkNewBranchNumber =
			SourceOldBranchNumbersToNewBranchNumbers[BranchNumbers[SourceGraftTrunkBranchIndex]];
		HostBranchChildren.Add(GraftTrunkNewBranchNumber);
		
		// Also add all descendants of the graft trunk
		const TArray<int32>& SourceTrunkDescendants = BranchFacadeSourceGraft.GetChildren(SourceGraftTrunkBranchIndex);
		for (const int32 SourceDescendantBranchNumber : SourceTrunkDescendants)
		{
			if (const int32* NewNumber = SourceOldBranchNumbersToNewBranchNumbers.Find(SourceDescendantBranchNumber))
			{
				HostBranchChildren.Add(*NewNumber);
			}
		}
	}

	BranchFacadeOut.SetChildren(InAttributes.BranchIndex, MoveTemp(HostBranchChildren));
	
	// Update all ancestors of the host branch to include the new graft branches as descendants
	const TArray<int32>& HostAncestorBranchNumbers = BranchFacadeOut.GetParents(InAttributes.BranchIndex);
	if (!HostAncestorBranchNumbers.IsEmpty())
	{
		const TManagedArray<int32>& AllBranchNumbers = BranchFacadeOut.GetBranchNumbers();
		TMap<int32, int32> BranchNumberToIndex;
		BranchNumberToIndex.Reserve(AllBranchNumbers.Num());
		for (int32 i = 0; i < AllBranchNumbers.Num(); ++i)
		{
			BranchNumberToIndex.Add(AllBranchNumbers[i], i);
		}
	
		for (const int32 AncestorBranchNumber : HostAncestorBranchNumbers)
		{
			if (AncestorBranchNumber == 0)
			{
				continue;
			}
			if (const int32* AncestorIndex = BranchNumberToIndex.Find(AncestorBranchNumber))
			{
				TArray<int32> AncestorChildren = BranchFacadeOut.GetChildren(*AncestorIndex);
				AncestorChildren.Append(TransformedBranchNumbers);
				BranchFacadeOut.SetChildren(*AncestorIndex, MoveTemp(AncestorChildren));
			}
		}
	}


	TArray<int32> HostAncestorPrefix = BranchFacadeOut.GetParents(InAttributes.BranchIndex);
	HostAncestorPrefix.Add(BranchFacadeOut.GetBranchNumber(InAttributes.BranchIndex));
	SourceIndex = 0;
	TargetIndex = NumOfBranchesInTarget;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			const TArray<int32>& SourceBranchParents = BranchFacadeSourceGraft.GetParents(SourceIndex);
			const int32 NumOfParentsInSource = SourceBranchParents.Num();
			TArray<int32> TransformedParents = HostAncestorPrefix;

			for (int32 j = 0; j < NumOfParentsInSource; ++j)
			{
				if (SourceBranchParents[j] != 0 
					&& SourceOldBranchNumbersToNewBranchNumbers.Contains(SourceBranchParents[j]))
				{
					TransformedParents.Add(SourceOldBranchNumbersToNewBranchNumbers[SourceBranchParents[j]]);
				}
			}

			BranchFacadeOut.SetParents(TargetIndex, TransformedParents);
			TargetIndex++;
		}

		SourceIndex++;
	}

	const int32 PlantNumberOfTargetBranch = BranchFacadeOut.GetBranchPlantNumber(InAttributes.BranchIndex);
	SourceIndex = 0;
	TargetIndex = NumOfBranchesInTarget;
	while (SourceIndex < NumOfBranchesInSource)
	{
		if (!(InAttributes.bIsTipInstance && SourceIndex == SourceGraftTrunkBranchIndex))
		[[likely]]
		{
			BranchFacadeOut.SetBranchPlantNumber(TargetIndex, PlantNumberOfTargetBranch);
			TargetIndex++;
		}

		SourceIndex++;
	}
}

bool FPVGrafter::RemoveFoliageDataIfPresent(FManagedArrayCollection& OutCollection)
{
	bool bRemovedFoliageData = false;
	PV::Facades::FBranchFacade BranchFacade(OutCollection);

	if (OutCollection.HasGroup(PV::GroupNames::FoliageNamesGroup) || OutCollection.HasGroup(PV::GroupNames::FoliageGroup))
	{
		if (OutCollection.HasGroup(PV::GroupNames::FoliageNamesGroup))
			OutCollection.RemoveGroup(PV::GroupNames::FoliageNamesGroup);

		if (OutCollection.HasGroup(PV::GroupNames::FoliageGroup))
			OutCollection.RemoveGroup(PV::GroupNames::FoliageGroup);

		for (int32 i = 0; i < BranchFacade.GetElementCount(); ++i)
		{
			BranchFacade.SetBranchFoliageIDs(i, {});
		}

		bRemovedFoliageData = true;
	}

	return bRemovedFoliageData;
}

TArray<FManagedArrayCollection> FPVGrafter::ExtractIndividualPlants(const FManagedArrayCollection& InCollection)
{
	const PV::Facades::FPlantFacade PlantFacadeSourceGraft(InCollection);

	TArray<int32> PlantNumbersInGraft = PlantFacadeSourceGraft.GetPlantNumbers();
	TArray<FManagedArrayCollection> IndividualPlants;
	if (PlantNumbersInGraft.Num() > 1)
	{
		IndividualPlants.Reserve(PlantNumbersInGraft.Num());
		for (const int32 PlantNumber : PlantNumbersInGraft)
		{
			FManagedArrayCollection NewCollection;
			InCollection.CopyTo(&NewCollection);
			PV::Facades::FBranchFacade BranchFacade(NewCollection);
			TArray<int> BranchesToRemove;
			BranchesToRemove.Reserve(BranchFacade.GetElementCount());
			for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
			{
				int32 PlantNumberOfBranch;
				BranchFacade.GetPlantNumber(BranchIndex, PlantNumberOfBranch);
				if (PlantNumberOfBranch != PlantNumber)
				{
					BranchesToRemove.Add(BranchIndex);
				}
			}

			PV::Facades::FTreeFacade::RemoveBranches(BranchFacade, BranchesToRemove, NewCollection);
			IndividualPlants.Add(NewCollection);
		}
	}
	else
	{
		IndividualPlants.Add(InCollection);
	}

	return IndividualPlants;
}

void FPVGrafter::DistributeGraftWithHormoneBasedSettings(
	FManagedArrayCollection& OutCollection,
	TArray<FManagedArrayCollection>& SourceGraftCollections,
	const FHormoneSettings::FDistributionSettings& DistributionSettings,
	const FHormoneSettings::FPhyllotaxySettings& PhyllotaxySettings,
	const FHormoneSettings::FScaleSettings& ScaleSettings,
	const FHormoneSettings::FAxilSettings& AxilSettings,
	const FPVDistributionVectorParams& VectorSettings,
	const FPVDistributionConditionParams& DistributionConditions,
	const int32 RandomSeed,
	const bool bRecomputeLightDetected)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVGrafter::DistributeGraftWithHormoneBasedSettings);

	if (!(DistributionSettings.InstanceSpacingRamp && ScaleSettings.ScaleRamp && AxilSettings.AxilAngleRamp))
	{
		return;
	}

	FRandomStream RandomStream;
	RandomStream.Initialize(RandomSeed);

	// Only need to do this temporarily since legacy skeleton has inconsistent data in children
	// TODO: Remove this once support for legacy skeletons is dropped
	PV::Facades::FBranchFacade BranchFacadeOut(OutCollection);
	BranchFacadeOut.RecomputeBranchChildren();
	BranchFacadeOut.RecomputeBranchParents();

	TArray<float> PScalesOverride;
	if (DistributionConditions.IsActiveCondition(EPVDistributionCondition::Scale))
	{
		auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(OutCollection);
		auto BudHormoneLevels    = PV::FBudHormoneLevelsAttribute::GetAttribute(OutCollection);
		PScalesOverride = PV::HormoneDistributionHelper::RemapPScalesToEthyleneThreshold(
			PointScaleAttribute,
			BudHormoneLevels,
			DistributionSettings.EthyleneThreshold);
	}

	const FNormalizedPointCaches NormalizedPointCaches = PVGrafter::BuildNormalizedPointCaches(
		OutCollection, DistributionConditions, MoveTemp(PScalesOverride));

	TArray<PV::DistributionHelper::FAttachmentPoint> AttachmentPoints;
	ComputeHormoneBasedAttachmentPoints(
		OutCollection,
		NormalizedPointCaches,
		DistributionSettings,
		ScaleSettings,
		AxilSettings,
		PhyllotaxySettings,
		VectorSettings,
		RandomStream,
		0.0f,
		AttachmentPoints);
	
	PVGrafter::DistributeGrafts(
		OutCollection,
		SourceGraftCollections,
		AttachmentPoints,
		DistributionConditions,
		RandomStream,
		bRecomputeLightDetected
	);
}

void FPVGrafter::DistributeGraftWithParametricSettings(FManagedArrayCollection& OutCollection,
                                                       TArray<FManagedArrayCollection>& SourceGraftCollections,
                                                       const FParametricSettings::FSpacingSettings& SpacingSettings,
                                                       const FParametricSettings::FPhyllotaxySettings& PhyllotaxySettings,
                                                       const FParametricSettings::FAngleSettings& AngleSettings,
                                                       const FParametricSettings::FScaleSettings& ScaleSettings,
                                                       const FPVDistributionVectorParams& VectorSettings,
                                                       const FPVDistributionConditionParams& DistributionConditions,
                                                       const int32 RandomSeed,
                                                       const bool bRecomputeLightDetected)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PVGrafter::DistributeGraftWithParametricSettings);

	FRandomStream RandomStream;
	RandomStream.Initialize(RandomSeed);

	TArray<float> PScalesOverride;
	if (DistributionConditions.IsActiveCondition(EPVDistributionCondition::Scale))
	{
		auto BranchPointsAttribute        = PV::FBranchPointsAttribute::GetAttribute(OutCollection);
		auto PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::GetAttribute(OutCollection);
		auto PointScaleAttribute          = PV::FPointScaleAttribute::GetAttribute(OutCollection);
		PScalesOverride = PV::ParametricDistributionHelper::RemapPScalesToGradientRange(
			BranchPointsAttribute,
			PointLengthFromRootAttribute,
			PointScaleAttribute,
			SpacingSettings.RelativeStart,
			SpacingSettings.RelativeEnd);
	}

	const FNormalizedPointCaches NormalizedPointCaches = PVGrafter::BuildNormalizedPointCaches(
		OutCollection, DistributionConditions, MoveTemp(PScalesOverride));

	TArray<PV::DistributionHelper::FAttachmentPoint> AttachmentPoints;
	ComputeAttachmentPoints(
		OutCollection,
		NormalizedPointCaches,
		SpacingSettings,
		PhyllotaxySettings,
		AngleSettings,
		ScaleSettings,
		VectorSettings,
		RandomSeed,
		0.0f,
		AttachmentPoints
	);

	PVGrafter::DistributeGrafts(
		OutCollection,
		SourceGraftCollections,
		AttachmentPoints,
		DistributionConditions,
		RandomStream,
		bRecomputeLightDetected
	);
}
