// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/PVAttributes.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PVHormoneDistributionHelper.h"
#include "Helpers/PVParametricDistributionHelper.h"

using namespace PV::ParametricDistributionHelper;
using namespace PV::HormoneDistributionHelper;

struct FPVDistributionConditionParams;
struct FPVFloatRamp;

struct FPVGrafter
{
	struct FPVGraftAttributes
	{
		int32 BranchIndex;
		int32 PointIndex;
		int32 CurrentBudNumber;
		FVector3f AttachmentPoint;
		FVector3f UpVector;
		FVector3f NormalVector;
		float Scale;
		float LengthFromRoot;
		bool bIsTipInstance;
	};

	static FTransform GetPointTransform(const FVector3f AttachmentPoint, const FVector3f UpVector, const FVector3f NormalVector,
	                                    const float Scale);

	static void RescaleBranchForGraftingOnTip(FManagedArrayCollection& OutCollection, const int32 BranchIndex, const int32 BranchTipPointIndex,
	                                          const float GraftTrunkLength);

	template <typename Fn>
	static void ForEachGraftPointExcludingRoot(const int32 NumPoints, const int32 RootIndex, Fn&& Func);

	static void SetGraft(FManagedArrayCollection& OutCollection,
	                     const FManagedArrayCollection& SourceGraftCollection, const FPVGraftAttributes& InAttributes);

	static bool RemoveFoliageDataIfPresent(FManagedArrayCollection& OutCollection);

	static TArray<FManagedArrayCollection> ExtractIndividualPlants(const FManagedArrayCollection& InCollection);

	static void DistributeGraftWithHormoneBasedSettings(
		FManagedArrayCollection& OutCollection,
		TArray<FManagedArrayCollection>& SourceGraftCollections,
		const FHormoneSettings::FDistributionSettings& DistributionSettings,
		const FHormoneSettings::FPhyllotaxySettings& PhyllotaxySettings,
		const FHormoneSettings::FScaleSettings& ScaleSettings,
		const FHormoneSettings::FAxilSettings& AxilSettings,
		const FPVDistributionVectorParams& VectorSettings,
		const FPVDistributionConditionParams& DistributionConditions,
		const int32 RandomSeed,
		const bool bRecomputeLightDetected);

	static void DistributeGraftWithParametricSettings(
		FManagedArrayCollection& OutCollection,
		TArray<FManagedArrayCollection>& SourceGraftCollections,
		const FParametricSettings::FSpacingSettings& SpacingSettings,
		const FParametricSettings::FPhyllotaxySettings& PhyllotaxySettings,
		const FParametricSettings::FAngleSettings& AngleSettings,
		const FParametricSettings::FScaleSettings& ScaleSettings,
		const FPVDistributionVectorParams& VectorSettings,
		const FPVDistributionConditionParams& DistributionConditions,
		const int32 RandomSeed,
		const bool bRecomputeLightDetected);
};
