// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGravity.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"

void FPVGravity::GeneratePhototropicData(const FPVGravityParams& InGravityParams, const PV::Facades::FBranchFacade& InBranchFacade,
                                         const PV::Facades::FPointFacade& InPointFacade, TArray<FVector3f>& OutPhototropicDirections)
{
	FVector3f PrevVec = FVector3f::UpVector;

	OutPhototropicDirections.Init(PrevVec, InPointFacade.GetElementCount());

	for (int BranchIndex = 0; BranchIndex < InBranchFacade.GetElementCount(); BranchIndex++)
	{
		const TArray<int32>& BranchPoints = InBranchFacade.GetPoints(BranchIndex);

		for (int i = BranchPoints.Num() - 1; i > 0; i--)
		{
			const auto CurrentPointIndex = BranchPoints[i];
			const TArray<FVector3f>& BudDirections = InPointFacade.GetBudDirection(CurrentPointIndex);

			if (BudDirections.Num() > PV::Facades::BudDirectionsLightSubOptimal)
			{
				const FVector3f& LightOptimal = BudDirections[PV::Facades::BudDirectionsLightOptimalIndex];
				const FVector3f& LightSubOptimal = BudDirections[PV::Facades::BudDirectionsLightSubOptimal];

				FVector3f PhototropicDirection = FMath::Lerp(LightOptimal, LightSubOptimal, InGravityParams.PhototropicBias);
				PhototropicDirection.Normalize();

				if (PhototropicDirection.Length() < 0.001)
				{
					PhototropicDirection = PrevVec;
				}
				else
				{
					PrevVec = PhototropicDirection;
				}

				if (OutPhototropicDirections.Num() > CurrentPointIndex)
				{
					OutPhototropicDirections[CurrentPointIndex] = PhototropicDirection;
				}
			}
		}
	}
}

void FPVGravity::ApplyGravity(const FPVGravityParams& InGravityParams, FManagedArrayCollection& OutCollection)
{
	const PV::Facades::FBranchFacade BranchFacade(OutCollection);
	const PV::Facades::FPointFacade PointFacade(OutCollection);
	const PV::Facades::FPlantFacade PlantFacade(OutCollection);

	if (!PointFacade.IsValid() || !BranchFacade.IsValid() || !PlantFacade.IsValid())
	{
		return;
	}

	TArray<FVector3f> OutPhototropicDirection;

	if (InGravityParams.Mode == EGravityMode::Phototropic)
	{
		GeneratePhototropicData(InGravityParams, BranchFacade, PointFacade, OutPhototropicDirection);
	}

	for (const int32 TrunkIndex : PlantFacade.GetTrunkIndices())
	{
		ApplyGravity(TrunkIndex, InGravityParams, OutPhototropicDirection, OutCollection);
	}
}

// Please note that LFR (length from root) for both skeleton points and foliage points are not being scaled by
// 100 at the moment, which is OK for now since they're only being compared relative to each other for now,
// however if they're used in any computation in the future in any meaningful way where other quantities are
// involved they will end up giving erroneous results.
void FPVGravity::ApplyGravity(
	const int32 BranchIndex,
	const FPVGravityParams& GravitySettings,
	const TArray<FVector3f>& PhototropicDirections,
	FManagedArrayCollection& OutCollection,
	FQuat4f TotalDownForce,
	FVector3f PreviousPosition
)
{
	PVE_OUTER_LOOP_DEBUG_CHECK(return);
	
	const auto CalculateDownForce = [](const FVector3f InRelativeDirection, const int InPointIndex, const float InPScaleGradient,
	                                          const FPVGravityParams& InGravityParams,
	                                          const TArray<FVector3f>& InPhototropicDirections) -> FQuat4f
		{
			const FVector3f PhototropicDirection = InPhototropicDirections.Num() > InPointIndex
				? InPhototropicDirections[InPointIndex]
				: FVector3f::UpVector;
			
			const FVector3f GravityDirection = InGravityParams.Mode == EGravityMode::Phototropic
				? PhototropicDirection
				: InGravityParams.Direction;

			float YDot = 1.0f - FMath::Abs(FVector3f::DotProduct(InRelativeDirection, GravityDirection));
			YDot = FMath::GetMappedRangeValueClamped(FVector2f(0, 1),
				FVector2f(FMath::Max(0.1f, 1 - InGravityParams.AngleCorrection), 1),
				YDot);

			const float GravityImpact = (InGravityParams.Gravity / 10.0f) * (1 - InPScaleGradient) * YDot;
			const FVector3f GravityForce = FMath::Lerp(InRelativeDirection, GravityDirection, GravityImpact).GetUnsafeNormal();
			return FQuat4f::FindBetween(InRelativeDirection, GravityForce);
		};

	PV::Facades::FBranchFacade BranchFacade = PV::Facades::FBranchFacade(OutCollection);
	PV::Facades::FPointFacade PointFacade = PV::Facades::FPointFacade(OutCollection);
	PV::Facades::FBudVectorsFacade BudVectorFacade = PV::Facades::FBudVectorsFacade(OutCollection);

	const TManagedArray<float>& PointScales = PointFacade.GetPointScales();
	if (PointScales.Num() == 0)
	{
		return;
	}
	
	const float MinScale = *Algo::MinElement(PointScales);
	const float MaxScale = *Algo::MaxElement(PointScales);

	TManagedArray<FVector3f>& PointPositions = PointFacade.ModifyPositions();
	TManagedArray<TArray<FVector3f>>& PointBudDirections = BudVectorFacade.ModifyBudDirections();

	const TArray<int32>& CurrentBranchPoints = BranchFacade.GetPoints(BranchIndex);
	const TArray<int32>& CurrentBranchChildren = BranchFacade.GetChildren(BranchIndex);
	const TManagedArray<int32>& BranchNumbers = BranchFacade.GetBranchNumbers();

	TArray<int32> ChildrenIndices;
	ChildrenIndices.Reserve(CurrentBranchChildren.Num());
	for (const int32 Child : CurrentBranchChildren)
	{
		const int32 ChildIndex = BranchNumbers.Find(Child);

		if (const int32 ParentIndex = BranchFacade.GetParentIndex(ChildIndex);
			ChildIndex != INDEX_NONE && ParentIndex == BranchIndex)
		{
			ChildrenIndices.Add(ChildIndex);
		}
	}

	if (TotalDownForce == FQuat4f::Identity)
	[[unlikely]]
	{
		PreviousPosition = PointPositions[CurrentBranchPoints[0]];
	}

	for (int32 Index = 1; Index < CurrentBranchPoints.Num(); ++Index)
	{
		const int32 PointIndex = CurrentBranchPoints[Index];
		const int32 PreviousPointIndex = CurrentBranchPoints[Index - 1];

		FVector3f& Position = PointPositions[PointIndex];
		const FVector3f& PivotPosition = PointPositions[PreviousPointIndex];

		TArray<FVector3f>& CurrentBudDirections = PointBudDirections[PointIndex];

		const float LengthFromRoot = PointFacade.GetLengthFromRoot(PointIndex);
		const float PreviousLengthFromRoot = PointFacade.GetLengthFromRoot(PreviousPointIndex);

		const FVector3f RelativeDirection = TotalDownForce * (Position - PreviousPosition);
		const FVector3f Tangent = RelativeDirection.GetUnsafeNormal();

		const FQuat4f DownForce = CalculateDownForce(
			Tangent,
			PointIndex,
			FMath::GetRangePct(MinScale, MaxScale, PointFacade.GetPointScale(PointIndex)),
			GravitySettings,
			PhototropicDirections
		);
		TotalDownForce = DownForce * TotalDownForce;

		PVE_LOOP_DEBUG_PARAMS_START()
			PVE_LOOP_DEBUG_POINT_PARAM("Pivot", PivotPosition);
			PVE_LOOP_DEBUG_DIRECTION_PARAM("Tangent", Position, Tangent);
			PVE_LOOP_DEBUG_DIRECTION_PARAM("Updated Relative Direction", Position, DownForce * RelativeDirection);
			PVE_LOOP_DEBUG_TEXT_PARAM("Length from Root", Position, FText::FromString(FString::Printf(TEXT("%.2f"), LengthFromRoot)));
			PVE_LOOP_DEBUG_VECTOR_PARAM("Offset", Position, PivotPosition + DownForce * RelativeDirection);
		PVE_LOOP_DEBUG_PARAMS_END()

		PVE_LOOP_DEBUG_STEP(return);

		PreviousPosition = Position;
		Position = PivotPosition + DownForce * RelativeDirection;
		
		for (const int32 ChildIndex : ChildrenIndices)
		{
			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(ChildIndex);
			if (BranchPoints.Num() == 0)
			{
				continue;
			}

			const int32 FirstPointIndex = BranchPoints[0];
			if (const float ChildLengthFromRoot = PointFacade.GetLengthFromRoot(FirstPointIndex);
				ChildLengthFromRoot <= LengthFromRoot && ChildLengthFromRoot > PreviousLengthFromRoot)
			[[unlikely]]
			{
				ApplyGravity(
					ChildIndex,
					GravitySettings,
					PhototropicDirections,
					OutCollection,
					TotalDownForce,
					PreviousPosition
				);
			}
		}

		for (int32 i = 0; i < CurrentBudDirections.Num(); ++i)
		{
			if (i == PV::Facades::BudDirectionsLightOptimalIndex || i == PV::Facades::BudDirectionsLightSubOptimal)
			{
				continue;
			}

			CurrentBudDirections[i] = TotalDownForce * CurrentBudDirections[i];
		}
	}
}
