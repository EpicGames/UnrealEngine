// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "Helpers/PVUtilities.h"

#include "PVGravity.generated.h"

namespace PV::Facades
{
	class FPointFacade;
	class FBranchFacade;
}

UENUM()
enum class EGravityMode : uint8
{
	Gravity,
	Phototropic
};

USTRUCT()
struct FPVGravityParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Gravity", meta=(Tooltip="Gravity (downward pull) or Phototropic (growth toward light/up).\n\nGravity = branches bend in the `Direction` vector (default downward). Phototropic = branches bend upward toward the sky."))
	EGravityMode Mode = EGravityMode::Gravity;

	UPROPERTY(EditAnywhere, Category="Gravity", meta=(ClampMin=0.0f, ClampMax=10.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Strength of the gravity / phototropic effect.\n\nHigher values yield more pronounced curvature. 0 = no effect."))
	float Gravity = 0;

	UPROPERTY(EditAnywhere, Category="Gravity", meta=(EditCondition = "Mode == EGravityMode::Gravity", Tooltip="Direction vector for the gravity pull.\n\nWorld-space gravity direction. Default is straight down. Use custom directions for stylized effects (e.g. sideways wind-blown look)."))
	FVector3f Direction = FVector3f::DownVector;

	UPROPERTY(EditAnywhere, Category="Gravity", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preserves the original branch angle during bending.\n\nBias that resists over-bending by maintaining the original branching angle. Higher = branches keep more of their original orientation. 0 = full bend; 1 = no bend at all (overrides Gravity)."))
	float AngleCorrection = 0;

	// UPROPERTY(EditAnywhere, Category="Gravity", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, EditCondition = "Mode == EGravityMode::Phototropic", Tooltip="Bias toward light-facing growth.\n\nBias between optimal light direction and shadow avoidance.  0 Being Light optimal, 1 being shadow avoidance"))
	UPROPERTY()
	float PhototropicBias = 0;
};

struct FPVGravity
{
	static void ApplyGravity(const FPVGravityParams& InGravityParams, FManagedArrayCollection& OutCollection);
	
private :

	static void GeneratePhototropicData(const FPVGravityParams& InGravityParams, const PV::Facades::FBranchFacade& InBranchFacade, const PV::Facades::FPointFacade& InPointFacade, TArray<FVector3f>& OutPhototropicDirections);

	static void ApplyGravity(const int32 BranchIndex, const FPVGravityParams& GravitySettings, const TArray<FVector3f>& PhototropicDirections, FManagedArrayCollection& OutCollection,
		FQuat4f TotalDownForce = FQuat4f::Identity, FVector3f PreviousPosition = FVector3f::ZeroVector);
};
