// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerData.h"
#include "ProceduralVegetationModule.h"

void FPVBudDevelopment::SetData(const TArray<int>& InArray)
{
	if (ensure(InArray.Num() == 6))
	{
		Generation = InArray[0];
		BudAge = InArray[1];
		BranchAge = InArray[2];
		AgeSenescense = InArray[3];
		LightSenescense = InArray[4];
		RelativeBudAge = InArray[5];
	}
}

void FPVBudDevelopment::ResetAge()
{
	BudAge = 0;
	BranchAge = 0;
	RelativeBudAge = 0;
}

void FPVBudDirection::SetData(const TArray<FVector3f>& InArray)
{
	if (ensure(InArray.Num() == 6))
	{
		Apical = FVector(InArray[0]);
		Axillary = FVector(InArray[1]);
		LightOptimal = FVector(InArray[2]);
		LightSubOptimal = FVector(InArray[3]);
		CurveGuide = FVector(InArray[4]);
		UpVector = FVector(InArray[5]);
	}
}

void FPVBudHormoneLevels::SetData(const TArray<float>& InArray)
{
	if (ensure(InArray.Num() == 6))
	{
		Apical = InArray[0];
		Axillary = InArray[1];
		AxillaryInhibition = InArray[2];
		Radical = InArray[3];
		Ethylene = InArray[4];
		Cytokinin = InArray[5];
	}
}

void FPVBudLateralMeristem::SetData(const TArray<float>& InArray)
{
	if (ensure(InArray.Num() == 7))
	{
		LateralMeristem = InArray[0];
		Multiplier = InArray[1];
		Inactive = InArray[2];
		Davinci = InArray[3];
		ParentDot = InArray[4];
		RootDistance = InArray[5];
		Degredation = InArray[6];
	}
}

void FPVBudLightDetected::SetData(const TArray<float>& InArray)
{
	if (ensure(InArray.Num() == 4))
	{
		Available = InArray[0];
		Resource = InArray[1];
		Branch = InArray[2];
		Collision = InArray[3];
	}
}

void FPVBudStatus::SetData(const TArray<int>& InArray)
{
	if (ensure(InArray.Num() == 10))
	{
		ApicalMeristem = InArray[0];
		Codominant = InArray[1];
		Axillary = InArray[2];
		Seed = InArray[3];
		Dormant = InArray[4];
		Triggered = InArray[5];
		NumTriggered = InArray[6];
		Inactive = InArray[7];
		BrokenTip = InArray[8];
		Broken = InArray[9];
	}
}

void UPVGrowerPrimitive::AddBranchBud(UPVGrowerPoint* Point)
{
	if (Point && !BranchBuds.Contains(Point->GetBudNumber()))
	{
		BranchBuds.Add(Point->GetBudNumber());
		Point->RefCount++;
	}
}

void UPVGrowerPrimitive::InsertBranchBud(UPVGrowerPoint* Point, int32 Index)
{
	if (Point && !BranchBuds.Contains(Point->GetBudNumber()))
	{
		BranchBuds.Insert(Point->GetBudNumber(),Index);
		Point->RefCount++;
	}
}

void UPVGrowerData::AddPoint(TObjectPtr<UPVGrowerPoint> Point, TObjectPtr<UPVGrowerPrimitive> Primitive, bool bAddToBranch /*= true*/)
{
	MaxBudNumber += 1;
	Point->Bud.BudNumber = MaxBudNumber;
	if (bAddToBranch)
	{
		Primitive->AddBranchBud(Point);
	}
	
	Point->Primitive = Primitive;
	AddPoint(Point, MaxBudNumber);
}

void UPVGrowerData::AddPoint(TObjectPtr<UPVGrowerPoint> Point, int32 BudNumber)
{
	if (ensureMsgf(!BudNumberIndexMap.Contains(BudNumber), TEXT("A Point already exist with same BudNumber %i, BudNumber must be unique.s"), BudNumber))
	{
		Points.Add(Point);
		BudNumberIndexMap.Add(BudNumber , Points.Num() - 1);
	}
}

void UPVGrowerData::RemovePoint(TObjectPtr<UPVGrowerPrimitive> Primitive, int32 BudNumber)
{
	if (TObjectPtr<UPVGrowerPoint> Point = GetPoint(BudNumber))
	{
		if (Primitive)
		{
			Primitive->BranchBuds.Remove(Point->GetBudNumber());
		}

		Point->RefCount--;

		if (Point->RefCount <= 0)
		{
			RemovePoint(Point);
		}
		else if (Primitive)
		{
			// Point is shared with another primitive (e.g. inserted as a source bud via
			// InsertBranchBud). Sever the bidirectional neighbor links between Point and
			// every point that belongs to this Primitive: remove Point from Neighbor->Neighbors
			// AND remove Neighbor from Point->Neighbors so the graph stays consistent.
			Point->Neighbors.RemoveAll([&](TObjectPtr<UPVGrowerPoint> Neighbor)
			{
				if (Neighbor && Neighbor->Primitive == Primitive)
				{
					Neighbor->Neighbors.Remove(Point);
					return true;
				}
				return false;
			});
		}
	}
	else
	{
		UE_LOGF(LogProceduralVegetation, Warning, "Cannot find Point with BudNumber %i while removing the point", BudNumber);
	}
}

void UPVGrowerData::RemovePoint(TObjectPtr<UPVGrowerPoint> Point)
{
	for (TObjectPtr<UPVGrowerPoint> Neighbor : Point->Neighbors)
	{
		Neighbor->Neighbors.Remove(Point);
	}

	if (ensure(BudNumberIndexMap.Contains(Point->GetBudNumber())))
	{
		int32 Index = BudNumberIndexMap[Point->GetBudNumber()];
		int LastIndex = Points.Num() - 1;

		if (Index != LastIndex)
		{
			TObjectPtr<UPVGrowerPoint> LastPoint = Points[LastIndex];
			Points.Swap(Index, LastIndex);

			BudNumberIndexMap[LastPoint->Bud.BudNumber] = Index;
		}

		Points.RemoveAt(LastIndex);
		BudNumberIndexMap.Remove(Point->GetBudNumber());
	}
}

int32 UPVGrowerData::GetPointIndex(int32 BudNumber)
{
	if (!BudNumberIndexMap.Contains(BudNumber))
	{
		return INDEX_NONE;
	}
	
	return BudNumberIndexMap[BudNumber];
}

TObjectPtr<UPVGrowerPoint> UPVGrowerData::GetPoint(int32 BudNumber) const
{
	if (!BudNumberIndexMap.Contains(BudNumber))
	{
		return nullptr;
	}
	
	return Points[BudNumberIndexMap[BudNumber]];
}

TObjectPtr<UPVGrowerPoint> UPVGrowerData::GetPointFromIndex(int32 PointIndex) const
{
	if (Points.IsValidIndex(PointIndex))
	{
		return Points[PointIndex];
	}

	return nullptr;
}

int32 UPVGrowerData::GetPrimitiveIndex(int32 BranchNumber) const
{
	return Primitives.IndexOfByPredicate([BranchNumber](UPVGrowerPrimitive* Primitive)
	{
		return Primitive->BranchNumber == BranchNumber;
	});
}

void UPVGrowerData::AddPrimitive(TObjectPtr<UPVGrowerPrimitive> Primitive)
{
	MaxBranchNumber += 1;
	Primitive->BranchNumber = MaxBranchNumber;
	AddPrimitive(Primitive, MaxBranchNumber);
}

void UPVGrowerData::AddPrimitive(TObjectPtr<UPVGrowerPrimitive> Primitive,const int32 BranchNumber)
{
	//check(!Primitives.Contains(Primitive));
	if (ensureMsgf(!PrimitiveNumberIndexMap.Contains(BranchNumber), TEXT("A Primitive already exist with same branch Number %i, BranchNumber must be unique.s"), BranchNumber))
	{
		Primitives.Add(Primitive);
		PrimitiveNumberIndexMap.Add(BranchNumber , Primitives.Num() - 1);
	}
	//check(Primitives.Num() == PrimitiveNumberIndexMap.Num());
}

UPVGrowerPrimitive* UPVGrowerData::GetPrimitive(int32 BranchNumber) const
{
	if (!PrimitiveNumberIndexMap.Contains(BranchNumber))
	{
		return nullptr;
	}
	
	return Primitives[PrimitiveNumberIndexMap[BranchNumber]];
}

void UPVGrowerData::RemovePrimitive(UPVGrowerPrimitive* RemoveItem)
{
	RemovePrimitive(RemoveItem->BranchNumber);
}

void UPVGrowerData::RemovePrimitive(int32 PrimitiveNumber, bool bRemoveChildren)
{
	if (UPVGrowerPrimitive* Primitive = GetPrimitive(PrimitiveNumber))
	{
		auto BranchBuds = Primitive->BranchBuds;
		for (auto BudNumber : BranchBuds)
		{
			RemovePoint(Primitive, BudNumber);
		}

		RemovePrimitiveReferences(Primitive, bRemoveChildren);
		RemovePrimitiveInternal(PrimitiveNumber);
	}
}

void UPVGrowerData::RemovePrimitiveReferences(TObjectPtr<UPVGrowerPrimitive> Primitive, bool bRemoveChildren)
{
	for (auto Parent : Primitive->Parents)
	{
		if (UPVGrowerPrimitive* ParentPrimitive = GetPrimitive(Parent))
		{
			ParentPrimitive->Children.Remove(Primitive->BranchNumber);
		}
	}

	if (bRemoveChildren)
	{
		TArray<int32> Children = Primitive->Children;
		for (int32 Child : Children)
		{
			if (UPVGrowerPrimitive* ChildPrimitive = GetPrimitive(Child))
			{
				RemovePrimitive(ChildPrimitive->BranchNumber);
			}
		}
	}
}

void UPVGrowerData::RemovePrimitiveInternal(int32 PrimitiveNumber)
{
	if (!PrimitiveNumberIndexMap.Contains(PrimitiveNumber))
	{
		return;
	}
	
	int32 Index = PrimitiveNumberIndexMap[PrimitiveNumber];
	int LastIndex = Primitives.Num() - 1;

	if (Index != LastIndex)
	{
		TObjectPtr<UPVGrowerPrimitive> LastPrimitive = Primitives[LastIndex];
		Primitives.Swap(Index, LastIndex);

		PrimitiveNumberIndexMap[LastPrimitive->BranchNumber] = Index;
	}
	
	Primitives.RemoveAt(LastIndex);
	PrimitiveNumberIndexMap.Remove(PrimitiveNumber);
}

TArray<UPVGrowerPrimitive*> UPVGrowerData::GetPointPrimitives(UPVGrowerPoint* Point) const
{
	TArray<UPVGrowerPrimitive*> PointPrimitives;
	for (const UPVGrowerPoint* NeighborPoint : Point->Neighbors)
	{
		PointPrimitives.AddUnique(NeighborPoint->Primitive);
	}

	return PointPrimitives; 
}

TArray<UPVGrowerPrimitive*> UPVGrowerData::GetSourceBudPrimitives(int32 SourceBudNumber) const
{
	TArray<UPVGrowerPrimitive*> BudPrimitives;
	for (UPVGrowerPrimitive* Primitive : Primitives)
	{
		if (Primitive->BranchSourceBudNumber == SourceBudNumber)
		{
			BudPrimitives.AddUnique(Primitive);
		}
	}

	return BudPrimitives;
}
