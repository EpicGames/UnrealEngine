// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVGrowerData.generated.h"

struct FPVLeafTransform;

USTRUCT()
struct FPVBudDevelopment
{
	GENERATED_BODY()

	int Generation = 0;

	int BudAge = 0;

	int BranchAge = 0;

	int AgeSenescense = 0;

	int LightSenescense = 0;

	int RelativeBudAge = 0;

	TArray<int> GetDataArray() const
	{
		return {Generation, BudAge, BranchAge, AgeSenescense, LightSenescense, RelativeBudAge};
	}

	void SetData(const TArray<int>& InArray);

	void ResetAge();
};

USTRUCT()
struct FPVBudDirection
{
	GENERATED_BODY()

	FVector Apical = FVector::Zero();

	FVector Axillary = FVector::Zero();

	FVector LightOptimal = FVector::Zero();

	FVector LightSubOptimal = FVector::Zero();

	FVector CurveGuide = FVector::Zero();

	FVector UpVector = FVector::UpVector;

	TArray<FVector3f> GetDataArray() const
	{
		return { FVector3f(Apical), FVector3f(Axillary), FVector3f(LightOptimal), FVector3f(LightSubOptimal), FVector3f(CurveGuide), FVector3f(UpVector)};//-UpVector.X, UpVector.Z, UpVector.Y)};
	}

	void SetData(const TArray<FVector3f>& InArray);
};

USTRUCT()
struct FPVBudHormoneLevels
{
	GENERATED_BODY()

	float Apical = 0.0;

	float Axillary = 0.0;

	float AxillaryInhibition = 0.0;

	float Radical = 0.0;

	float Ethylene = 0.0;

	float Cytokinin = 0.0;

	TArray<float> GetDataArray() const
	{
		return { Apical, Axillary, AxillaryInhibition, Radical, Ethylene, Cytokinin };
	}

	void SetData(const TArray<float>& InArray);
};

USTRUCT()
struct FPVBudLateralMeristem
{
	GENERATED_BODY()

	float LateralMeristem = 0.0;

	float Multiplier = 0.0;

	float Inactive = 0.0;

	float Davinci = 0.0;

	float ParentDot = 0.0;

	float RootDistance = 0.0;

	float Degredation = 0.0;

	TArray<float> GetDataArray() const
	{
		return { LateralMeristem, Multiplier, Inactive, Davinci, ParentDot, RootDistance, Degredation };
	}

	void SetData(const TArray<float>& InArray);
};

USTRUCT()
struct FPVBudLightDetected
{
	GENERATED_BODY()

	float Available = 0.0;

	float Resource = 0.0;

	float Branch = 0.0;

	float Collision = 0.0;

	TArray<float> GetDataArray() const
	{
		return { Available, Resource, Branch, Collision };
	}

	void SetData(const TArray<float>& InArray);
};

USTRUCT()
struct FPVBudStatus
{
	GENERATED_BODY()

	int ApicalMeristem = 0;

	int Codominant = 0;

	int Axillary = 0;

	int Seed = 0;

	int Dormant = 0;

	int Triggered = 0;

	int NumTriggered = 0;

	int Inactive = 0;

	int BrokenTip = 0;

	int Broken = 0;

	TArray<int> GetDataArray() const
	{
		return { ApicalMeristem, Codominant, Axillary, Seed, Dormant, Triggered, NumTriggered, Inactive, BrokenTip, Broken };
	}

	void SetData(const TArray<int>& InArray);
};

USTRUCT()
struct FPVBud
{
	GENERATED_BODY()

	int BudNumber = 0;

	FPVBudDevelopment Development;

	FPVBudDirection Direction;

	FPVBudHormoneLevels HormoneLevels;

	FPVBudLateralMeristem LateralMeristem;

	FPVBudLightDetected LightDetected;

	FPVBudStatus Status;
};

USTRUCT()
struct FPVSeedInfo
{
	GENERATED_BODY()

	float PScale = 0.0;

	float PScaleRatio = 0.0;

	uint32 Type = 0;

	uint32 Generation = 0;

	uint32 PlantNumber = 0;

	uint32 BrancNumber = 0;

	uint32 MaxBranchNumber = 0;

	uint32 BudNumber = 0;

	uint32 MaxBudNumber = 0;
};

UENUM()
enum class EPVAbscissionType : uint8
{
	Age,
	Light
};

USTRUCT()
struct FPVPrimitiveSenescence
{
	GENERATED_BODY()

	UPROPERTY()
	bool bDegradation = false;

	UPROPERTY()
	float DegradationAmount = 0.0;

	UPROPERTY()
	bool bAbscission = false;

	UPROPERTY()
	EPVAbscissionType AbscissionType = EPVAbscissionType::Age;

	UPROPERTY()
	bool bInactive = false;
};

UCLASS()
class UPVGrowerPrimitive : public UObject
{
	GENERATED_BODY()
public:
	int32 BranchNumber = 1;

	int32 BranchParentNumber = 0;

	int32 BranchSourceBudNumber = 1;
	
	int32 PlantNumber = 1;

	//Stores the Bud Numbers of the Points
	TArray<int32> BranchBuds;

	//Stores the BranchNumbers of Children
	TArray<int32> Children;

	//Stores the BranchNumbers of parents
	TArray<int32> Parents;

	bool bNewBranch = false;

	UPROPERTY()
	FPVPrimitiveSenescence Senescence;

	void AddBranchBud(UPVGrowerPoint* Point);
	
	void InsertBranchBud(UPVGrowerPoint* Point, int32 Index);
};

UCLASS()
class UPVGrowerPoint : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FVector Position;

	UPROPERTY()
	float PScale = 0.0;

	UPROPERTY()
	FVector Up = FVector::UpVector;

	UPROPERTY()
	int VisAgeSenescen = 0.0;

	FVector VisAuxin = FVector::ZeroVector;

	FVector VisCytokinin = FVector::ZeroVector;

	FVector VisEthylene = FVector::ZeroVector;

	FVector VisGravity = FVector::ZeroVector;

	int VisLightSenescen = 0;

	UPROPERTY()
	int16 BranchNumber = 0;

	FPVBud Bud;

	FPVSeedInfo SeedInfo;

	UPROPERTY()
	float LengthFromRoot = 0.0;

	UPROPERTY()
	float LengthFromTrunk = 0.0;

	TArray<float> NextAxialElongations;

	FVector NextApicalDirection = FVector::ZeroVector;

	FVector NextAxillaryDirection = FVector::ZeroVector;

	FVector NextUpVector = FVector::UpVector;

	bool bNewBud = false;

	bool bLateralAppend = false;

	float IgnoreGravityBreak = 0.0f;

	UPROPERTY()
	TObjectPtr<UPVGrowerPrimitive> Primitive;

	UPROPERTY()
	TArray<TObjectPtr<UPVGrowerPoint>> Neighbors;

	int32 RefCount = 0;

	bool bInput = false;

	int32 GetBudNumber() const { return Bud.BudNumber; }
};

UCLASS()
class UPVGrowerData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<TObjectPtr<UPVGrowerPoint>> Points;

	UPROPERTY()
	TArray<TObjectPtr<UPVGrowerPrimitive>> Primitives;

	UPROPERTY()
	TMap<int32, int32> BudNumberIndexMap;

	UPROPERTY()
	TMap<int32, int32> PrimitiveNumberIndexMap;

	float MinLengthFromRoot = 0.0;

	int32 MaxBranchNumber = 25;

	int32 MaxBudNumber = 0;

	// Leaf transforms produced by the last growth cycle; written to the output
	// collection by FillCollection for viewport visualization.
	TArray<FPVLeafTransform> LastLeafTransforms;

	// Cycle offset derived from the max bud age in the upstream grower's collection.
	// Added to the current cycle when computing SaplingAge in GenerateLeaves so that a
	// chained grower begins with the foliage maturity the upstream plant already reached.
	// Zero for fresh (seed-initialised) skeletons.
	int32 InputCycleOffset = 0;

public:
	/*
	 * Add the point to Points list and associate to the branch
	 * Increments the BudNumber and add point to BudNumberIndexMap
	 */
	void AddPoint(TObjectPtr<UPVGrowerPoint> Point, TObjectPtr<UPVGrowerPrimitive> Primitive, bool bAddToBranch = true);

	/*
	 * Add the point to points list and also add it to BudNumberIndexMap
	 */
	void AddPoint(TObjectPtr<UPVGrowerPoint> Point, int32 BudNumber);

	void RemovePoint(TObjectPtr<UPVGrowerPrimitive> Primitive, int32 BudNumber);

	void RemovePoint(TObjectPtr<UPVGrowerPoint> Point);
	
	int32 GetPointIndex(int32 BudNumber);
	
	TObjectPtr<UPVGrowerPoint> GetPoint(int32 BudNumber) const;
	
	TObjectPtr<UPVGrowerPoint> GetPointFromIndex(int32 PointIndex) const;

	int32 GetPrimitiveIndex(int32 BranchNumber) const;
	
	void AddPrimitive(TObjectPtr<UPVGrowerPrimitive> Primitive);
	
	void AddPrimitive(TObjectPtr<UPVGrowerPrimitive> Primitive, int32 BranchNumber);

	UPVGrowerPrimitive* GetPrimitive(int32 BranchNumber) const;
	
	void RemovePrimitive(UPVGrowerPrimitive* RemoveItem);
	
	void RemovePrimitive(int32 PrimitiveNumber, bool bRemoveChildren = true);
	
	void RemovePrimitiveReferences(TObjectPtr<UPVGrowerPrimitive> Primitive, bool bRemoveChildren);

	void RemovePrimitiveInternal(int32 PrimitiveNumber);

	TArray<UPVGrowerPrimitive*> GetPointPrimitives(UPVGrowerPoint* Point) const;

	TArray<UPVGrowerPrimitive*> GetSourceBudPrimitives(int32 SourceBudNumber) const;
};
