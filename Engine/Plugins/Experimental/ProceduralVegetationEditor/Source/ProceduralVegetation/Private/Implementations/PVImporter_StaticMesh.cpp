// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVImporter_StaticMesh.h"

#include "Helpers/PVImportHelpers.h"
#include "Helpers/PVDynamicMeshHelpers.h"
#include "Utils/PVDynamicMeshVertexAttribute.h"

#include "Engine/StaticMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "Selections/MeshConnectedComponents.h"

namespace PV::StaticMeshImport
{
using FBooleanVertexAttribute = PV::ImportHelper::FBooleanVertexAttribute;
using FIntVertexAttribute = PV::ImportHelper::FIntVertexAttribute;
using FFloatVertexAttribute = PV::ImportHelper::FFloatVertexAttribute;

using FIndex2i = UE::Geometry::FIndex2i;
using FIndex3i = UE::Geometry::FIndex3i;
using FDynamicMesh3 = UE::Geometry::FDynamicMesh3;
using FAxisAlignedBox3d = UE::Geometry::FAxisAlignedBox3d;
using FDynamicMeshAABBTree3 = UE::Geometry::FDynamicMeshAABBTree3;
using FMeshConnectedComponents = UE::Geometry::FMeshConnectedComponents;
using FDynamicMeshQueries3 = UE::Geometry::TMeshQueries<FDynamicMesh3>;
using FDistPoint3Triangle3d = UE::Geometry::FDistPoint3Triangle3d;

const static PV::TDynamicMeshVertexAttributeDefinition<bool> FillerVertexAttributeDefinition(TEXT("FillerVertex"));
const static PV::TDynamicMeshVertexAttributeDefinition<int32> FuseToAttributeDefinition(TEXT("FuseTo"));

static FVector GenerateFillerVertex(const FVector& Pos1, const FVector& Pos2)
{
	// Since we don't have edges for the DynamicMesh we add a new vertex at the center (offset by 0.1mm) whenever we need to represent a line
	return ((Pos1 + Pos2) * 0.5)  + (Pos1 - Pos2).GetSafeNormal().Cross(FVector::ForwardVector).Cross(FVector::UpVector)/* * 0.01*/;
}

static void RemoveUnusedVertices(FDynamicMesh3& DynamicMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OrphanRemoval);

	DynamicMesh.RemoveUnusedVertices();
	DynamicMesh.CompactInPlace();
}

static bool FilterTrianglesUsingMaterials(FDynamicMesh3& DynamicMesh, const UStaticMesh* StaticMesh, const TArray<FName>& MaterialsToKeep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(KeepMaterial);

	TSet<int32> MaterialIDs;
	MaterialIDs.Reserve(MaterialsToKeep.Num());

	for (FName MaterialName : MaterialsToKeep)
	{
		const int32 MaterialIndexToKeep = StaticMesh->GetMaterialIndex(MaterialName);
		if (MaterialIndexToKeep != INDEX_NONE)
		{
			MaterialIDs.Add(MaterialIndexToKeep);
		}
	}

	if (MaterialIDs.Num() == 0)
	{
		return false;
	}

	UE::Geometry::FDynamicMeshAttributeSet* MeshAttributeSet = DynamicMesh.Attributes();
	UE::Geometry::FDynamicMeshMaterialAttribute* MaterialIDAttribute = MeshAttributeSet ? MeshAttributeSet->GetMaterialID() : nullptr;
		
	if (!MaterialIDAttribute)
	{
		return false;
	}

	for (int32 TriangleID : DynamicMesh.TriangleIndicesItr())
	{
		const int32 MaterialID = MaterialIDAttribute->GetValue(TriangleID);
		if (!MaterialIDs.Contains(MaterialID))
		{
			DynamicMesh.RemoveTriangle(TriangleID);
		}
	}

	RemoveUnusedVertices(DynamicMesh);

	return true;
}

static void ComputeNormals(FDynamicMesh3& DynamicMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeNormals);

	if (!DynamicMesh.HasVertexNormals())
	{
		DynamicMesh.EnableVertexNormals(FVector3f::UnitX());
	}

	if (DynamicMesh.Attributes() && DynamicMesh.Attributes()->NumNormalLayers() > 0)
	{
		UE::Geometry::FDynamicMeshNormalOverlay* Overlay = DynamicMesh.Attributes()->GetNormalLayer(0);
		for (int32 EID : Overlay->ElementIndicesItr())
		{
			int32 ParentVID = Overlay->GetParentVertex(EID);
			if (ParentVID != INDEX_NONE)
			{
				DynamicMesh.SetVertexNormal(ParentVID, Overlay->GetElement(EID));
			}
		}
	}
	else
	{
		UE::Geometry::FMeshNormals::QuickComputeVertexNormals(DynamicMesh);
	}
}

static void ComputeVerticesCenterAndRadius_Internal(
	FDynamicMesh3& DynamicMesh, 
	const TArray<int32>& TriangleIds,
	FFloatVertexAttribute& VertexScaleAttribute
)
{
	// This is a second attempt at creating an algorithm that can walk along each vertex on a tree mesh and find the corresponding center point and
	// radius of the branch at the point of the vertex.
	//
	// This algorithm will iterate the vertices and perform a ray-cast in the opposite direction of the vertex normal.
	// If it hits another triangle then it will check whether the normal of said triangle is opposing to the current vertex (e.g. the dot of the
	// vertex normal and the triangle normal is < -0.5). If the ray-cast fails to find a triangle with an opposing normal, then a second algorithm
	// is employed to try and find an opposing vertex. This second algorithm finds the closest vertex to a position 100cm from the vertex in the
	// opposite direction of the vertex normal, it then checks if the normal of this vertex is opposing. If this check fails, then it moves the
	// query point closer and repeats the check. This is repeated 3 times before the algorithm gives up and the vertex is left with a scale if 0 which
	// indicates that at radius/center point was not found.

	// Issues with this algorithm:
	// 1. This algorithm only works if the branch mesh is fully enclosed. If there are holes then the ray may exit through those holes and hit parts
	// of the branch we do not desire for it to hit.
	// 2. This algorithm does not handle intersection geometry well as it cannot differentiate between hitting an intersection branch and the opposite
	// side of the current branch. This is partially handled by running this algorithm per detached mesh (which for most meshes will mean per branch)
	// 3. It also not work well if the mesh is built up using only faces (e.g flat geometry instead of a tube for the branches)
	// 4. The normal of the vertex may not point directly towards the opposite side (see "Remove extreme radius values" section below for more information)

	const PV::DynamicMeshHelper::FFindMeshElemsResult ComponentVertexIndices = PV::DynamicMeshHelper::GetUniqueVertexIndices(DynamicMesh, TriangleIds);

	FDynamicMeshAABBTree3 DynamicMeshAABBTree(&DynamicMesh, false);
	DynamicMeshAABBTree.Build(TriangleIds);

	TArray<FVector> NewVertexPositions;
	NewVertexPositions.SetNum(ComponentVertexIndices.Num());

	TArray<float> Scales;
	Scales.SetNumZeroed(ComponentVertexIndices.Num());

	TArray<int32> ThisVertexTriangleIDs;
	for (int32 i = 0; i < ComponentVertexIndices.Num(); ++i)
	{
		const int32 VertexIndex = ComponentVertexIndices[i];

		const FVector VertexNormal = FVector(DynamicMesh.GetVertexNormal(VertexIndex));
		const FVector& VertexPosition = DynamicMesh.GetVertex(VertexIndex);

		const FRay3d Ray(VertexPosition, -VertexNormal);

		ThisVertexTriangleIDs.Reset();
		DynamicMesh.GetVtxTriangles(VertexIndex, ThisVertexTriangleIDs);
		
		double HitTriangleDistance = 0;
		int HitTriangleID = INDEX_NONE;
		FVector3d HitBaryCoords = FVector3d::Zero();
		FDynamicMeshAABBTree3::FQueryOptions QueryOptions;
		QueryOptions.TriangleFilterF = [&](int32 TriangleID) { return !ThisVertexTriangleIDs.Contains(TriangleID); }; // Make sure to ignore triangles which includes this vertex, otherise we will instantly get a hit on ourselves
		DynamicMeshAABBTree.FindNearestHitTriangle(Ray, HitTriangleDistance, HitTriangleID, HitBaryCoords, QueryOptions);

		const float CollinearLimit = -0.5;

		if (HitTriangleID != INDEX_NONE)
		{
			const FVector HitNormal = DynamicMesh.GetTriBaryNormal(HitTriangleID, HitBaryCoords[0], HitBaryCoords[1], HitBaryCoords[2]);
			const FVector HitLocation = DynamicMesh.GetTriBaryPoint(HitTriangleID, HitBaryCoords[0], HitBaryCoords[1], HitBaryCoords[2]);

			const float Dot = HitNormal.Dot(VertexNormal);
			const bool bIsOpposingAndCollinear = Dot < CollinearLimit;
			if (bIsOpposingAndCollinear)
			{
				const FVector Center = (HitLocation + VertexPosition) * 0.5;
				Scales[i] = (VertexPosition - Center).Size();
				NewVertexPositions[i] = Center;
			}
		}

		if (Scales[i] == 0)
		{
			// Could not find opposing point using ray intersection, try using FindNearestVertex instead.
			const int32 MaxNumSamples = 3;
			for (int32 j = 1; j <= MaxNumSamples; ++j)
			{
				double NearestDistSqr = 0.f;
				const float MaxSampleDist = 100; // Value is arbitrary, should probably be based on the bounds of the mesh
				const FVector SamplePosition = VertexPosition - VertexNormal * MaxSampleDist / j;
				const int32 NearestVertexID = DynamicMeshAABBTree.FindNearestVertex(SamplePosition, NearestDistSqr, MaxSampleDist);
				if (NearestVertexID == INDEX_NONE)
				{
					continue;
				}

				const FVector OtherVertexPosition = DynamicMesh.GetVertex(NearestVertexID);
				const FVector OtherVertexNormal = FVector(DynamicMesh.GetVertexNormal(NearestVertexID));

				const float Dot = OtherVertexNormal.Dot(VertexNormal);
				const bool bIsOpposingAndCollinear = Dot < CollinearLimit;
				if (!bIsOpposingAndCollinear)
				{
					continue;
				}

				const FVector OpposingPosition = FVector::PointPlaneProject(VertexPosition, OtherVertexPosition, -VertexNormal);
				const FVector Center = (VertexPosition + OpposingPosition) * 0.5;
				Scales[i] = (VertexPosition - Center).Size();
				NewVertexPositions[i] = Center;
				break;
			}
		}
	}

	// Remove extreme radius values
	{
		// In the below example, point "a" may have a normal which causes the ray to hit point "b" which is not the outcome we would expect.
		// We solve this by taking the average radius of all vertices and removing any radius which lies way outside the average.
		// This works well as long as the branches we're operating on have a fairly uniform size.
		//
		//       /    /
		//      /  b*/
		//     /    /
		//    /    /
		//    \*a /
		//   /   /
		//  /    \
		// /    /
		 
		int32 RadiiNum = 0;
		float AvgRadius = 0;
		for (float Scale : Scales)
		{
			if (Scale > 0)
			{
				RadiiNum++;
				AvgRadius += Scale;
			}
		}

		if (RadiiNum > 0)
		{
			AvgRadius /= RadiiNum;
				
			for (int32 i = 0; i < ComponentVertexIndices.Num(); ++i)
			{
				const float Scale = Scales[i];
				if (Scale > 0)
				{
					const float Ratio = Scale / AvgRadius;
					if (Ratio > 3)
					{
						Scales[i] = 0;
					}
				}
			}
		}
	}

	for (int32 i = 0; i < ComponentVertexIndices.Num(); ++i)
	{
		if (Scales[i] > 0)
		{
			const int32 VertexIndex = ComponentVertexIndices[i];
			DynamicMesh.SetVertex(VertexIndex, NewVertexPositions[i]);
			VertexScaleAttribute.SetValue(VertexIndex, Scales[i]);
		}
	}
}

static void ComputeVerticesCenterAndRadius(FDynamicMesh3& DynamicMesh, FFloatVertexAttribute& VertexScaleAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeVerticesCenterAndRadius);

	// We execute this per component (detached mesh) as that often correlates to individual branches which significantly improves
	// the results as we won't accidentally find an opposing vertex in a different branch from the one we're currently operating on. 
	FMeshConnectedComponents ConnectedComponents(&DynamicMesh);
	ConnectedComponents.FindConnectedTriangles(); 

	ParallelFor(ConnectedComponents.Num(), [&](int32 Index)
	{
		ComputeVerticesCenterAndRadius_Internal(DynamicMesh, ConnectedComponents.GetComponent(Index).Indices, VertexScaleAttribute);
	});
}

static void ExtractReductionCloud_Internal(
	const FDynamicMesh3& DynamicMesh, 
	const TArray<int32>& TriangleIds, 
	float SearchRadiusInflate,
	const FFloatVertexAttribute& VertexScaleAttribute,
	FIntVertexAttribute& FuseToAttribute
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ExtractReductionCloud_Internal);
		
	FDynamicMeshAABBTree3 DynamicMeshAABBTree(&DynamicMesh, false);
	DynamicMeshAABBTree.Build(TriangleIds);

	PV::DynamicMeshHelper::FFindMeshElemsResult VertexIndices = PV::DynamicMeshHelper::GetUniqueVertexIndices(DynamicMesh, TriangleIds);

	const int32 MaxVertexID = DynamicMesh.MaxVertexID();
		
	TArray<bool> Removed; 
	Removed.SetNumZeroed(MaxVertexID);
		
	TArray<bool> Checked;
	Checked.SetNumZeroed(MaxVertexID);
		
	TArray<float> Radii;
	Radii.SetNumUninitialized(MaxVertexID);
	
	for (int32 i = 0; i < VertexIndices.Num(); i++)
	{
		const int32 VertexIndex = VertexIndices[i];
		Radii[VertexIndex] = VertexScaleAttribute.GetValue(VertexIndex);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SortByRadius);
		VertexIndices.Sort([&](int32 A, int32 B) { return Radii[A] > Radii[B]; });
	}

	for (int32 i = 0; i < VertexIndices.Num(); i++)
	{
		const int32 VertexIndex = VertexIndices[i];

		if (Removed[VertexIndex])
		{ 
			continue; 
		}

		Checked[VertexIndex] = true;

		const FVector VertexPosition = DynamicMesh.GetVertex(VertexIndex);
		const float VertexRadius = Radii[VertexIndex];
		if (VertexRadius <= 0)
		{
			FuseToAttribute.SetValue(VertexIndex, VertexIndex);
			continue;
		}

		const float SearchRadius = VertexRadius * SearchRadiusInflate;
		const PV::DynamicMeshHelper::FFindMeshElemsResult OverlappingVertices = PV::DynamicMeshHelper::FindVerticesInRadius(DynamicMeshAABBTree, VertexPosition, SearchRadius);
		for (int32 j = 0; j < OverlappingVertices.Num(); j++)
		{
			const int32 OverlappingVertexIndex = OverlappingVertices[j];
				
			if (OverlappingVertexIndex == VertexIndex 
				|| Removed[OverlappingVertexIndex] 
				|| Checked[OverlappingVertexIndex])
			{
				continue;
			}

			const float OverlappingVertexRadius = Radii[OverlappingVertexIndex];

			if (OverlappingVertexRadius > 0)
			{
				const FVector OverlappingVertexPosition = DynamicMesh.GetVertex(OverlappingVertexIndex);
				const float CombinedRadius = OverlappingVertexRadius + VertexRadius;
				const float CombinedRadiusSqr = CombinedRadius * CombinedRadius;
				const float DistSquared = (OverlappingVertexPosition - VertexPosition).SizeSquared();
				if (DistSquared > CombinedRadiusSqr)
				{
					continue;
				}
			}

			Removed[OverlappingVertexIndex] = true;
			FuseToAttribute.SetValue(OverlappingVertexIndex, VertexIndex);
		}

		FuseToAttribute.SetValue(VertexIndex, VertexIndex);
	}
}
	
static void FuseVerticesUsingFuseAttribute(
	FDynamicMesh3& DynamicMesh,
	const FIntVertexAttribute& FuseToAttribute,
	FBooleanVertexAttribute& FillerVertexAttribute
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FuseVerticesUsingFuseAttribute);

	TArray<int32> TriangleIndices;
	TriangleIndices.Reserve(DynamicMesh.TriangleCount());
	for (int32 TID : DynamicMesh.TriangleIndicesItr())
	{
		TriangleIndices.Add(TID);
	}

	for (int32 i = 0; i < TriangleIndices.Num(); i++)
	{
		const int32 TriangleIndex = TriangleIndices[i];
		const FIndex3i VertexIndices = DynamicMesh.GetTriangle(TriangleIndex);

		TArray<int32, TInlineAllocator<3>> NewTriangleIndices;
		for (int32 j = 0; j < 3; j++) 
		{
			const int32 FuseTo = FuseToAttribute.GetValue(VertexIndices[j]);
			NewTriangleIndices.AddUnique(FuseTo);

			FillerVertexAttribute.SetValue(VertexIndices[j], false);
		}

		DynamicMesh.RemoveTriangle(TriangleIndex, false);

		if (NewTriangleIndices.Num() == 2)
		{ 
			const FVector P1 = DynamicMesh.GetVertex(NewTriangleIndices[0]);
			const FVector P2 = DynamicMesh.GetVertex(NewTriangleIndices[1]);

			const FVector P3 = GenerateFillerVertex(P1, P2);
			const int32 P3Index = DynamicMesh.AppendVertex(P3);
			DynamicMesh.AppendTriangle(NewTriangleIndices[0], NewTriangleIndices[1], P3Index);

			FillerVertexAttribute.SetValue(P3Index, true);
		}
		else if (NewTriangleIndices.Num() == 3)
		{
			DynamicMesh.AppendTriangle(NewTriangleIndices[0], NewTriangleIndices[1], NewTriangleIndices[2]);
		}
		else if (NewTriangleIndices.Num() > 3)
		{
			check(false); // This should not be able to happen
		}
	}

	RemoveUnusedVertices(DynamicMesh);
}

static void CollapseBranchesToLine(
	FDynamicMesh3& DynamicMesh,
	const FFloatVertexAttribute& VertexScaleAttribute,
	FIntVertexAttribute& FuseToAttribute,
	FBooleanVertexAttribute& FillerVertexAttribute
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CollapseBranchesToLine);

	FMeshConnectedComponents ConnectedComponents(&DynamicMesh);
	ConnectedComponents.FindConnectedTriangles();

	ParallelFor(ConnectedComponents.Num(), [&](int32 Index)
	{
		ExtractReductionCloud_Internal(DynamicMesh, ConnectedComponents.GetComponent(Index).Indices, 2.f, VertexScaleAttribute, FuseToAttribute);
	});

	FuseVerticesUsingFuseAttribute(DynamicMesh, FuseToAttribute, FillerVertexAttribute);
}

static void FindEndPoints(
	const FDynamicMesh3& DynamicMesh,
	FBooleanVertexAttribute& EndPointAttribute,
	const FBooleanVertexAttribute& FillerVertexAttribute
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindEndPoints);
	
	TArray<double> xComps;
	TArray<double> yComps;
	TArray<double> zComps;
	for (int32 VertexIndex : DynamicMesh.VertexIndicesItr())
	{
		const FVector VertexPosition = DynamicMesh.GetVertex(VertexIndex);

		if (FillerVertexAttribute.GetValue(VertexIndex))
		{
			continue;
		}

		const int32 EdgeCount = DynamicMesh.GetVtxEdgeCount(VertexIndex);
		check(EdgeCount > 0);

		xComps.Reset(EdgeCount);
		yComps.Reset(EdgeCount);
		zComps.Reset(EdgeCount);

		for (int32 EdgeID : DynamicMesh.VtxEdgesItr(VertexIndex))
		{
			const FIndex2i VertexIDs = DynamicMesh.GetEdgeV(EdgeID);
			const int32 OtherVertexID = VertexIDs.A == VertexIndex ? VertexIDs.B : VertexIDs.A;

			if (FillerVertexAttribute.GetValue(OtherVertexID))
			{
				continue;
			}

			const FVector NeighbourVertexPosition = DynamicMesh.GetVertex(OtherVertexID);
			const FVector Dir = (NeighbourVertexPosition - VertexPosition).GetSafeNormal();
			xComps.Add(Dir.X);
			yComps.Add(Dir.Y);
			zComps.Add(Dir.Z);
		}
		
		if (!ensure(xComps.Num() > 0))
		{
			continue;
		}

		const double* MaxXComp = Algo::MaxElement(xComps);
		const double* MinXComp = Algo::MinElement(xComps);
		const double* MaxYComp = Algo::MaxElement(yComps);
		const double* MinYComp = Algo::MinElement(yComps);
		const double* MaxZComp = Algo::MaxElement(zComps);
		const double* MinZComp = Algo::MinElement(zComps);

		const double xDiff = *MaxXComp - *MinXComp;
		const double yDiff = *MaxYComp - *MinYComp;
		const double zDiff = *MaxZComp - *MinZComp;
		const double MaxDiff = FMath::Max3(xDiff, yDiff, zDiff);

		const bool bIsEndPoint = MaxDiff <= 1.0;
		EndPointAttribute.SetValue(VertexIndex, bIsEndPoint);
	}
}

static void ConnectAndTraceBranches(
	FDynamicMesh3& DynamicMesh,
	const FBooleanVertexAttribute& RootPointAttribute,
	FFloatVertexAttribute& VertexScaleAttribute,
	FBooleanVertexAttribute& EndPointAttribute,
	FBooleanVertexAttribute& FillerVertexAttribute,
	FFloatVertexAttribute& LengthFromRootAttribute,
	FIntVertexAttribute& NextBranchPointAttribute
)
{
	// Connects branch mesh components to their parent components (trunk or larger branches), choosing
	// attachment edges that lie on the branch trace path rather than any geometrically-close edge.
	// This avoids attaching at forks or off-path geometry which would confuse the downstream tracer.
	//
	// Algorithm overview:
	//   Phase 0: Build a node graph (one node per connected component) and pre-compute all candidate
	//            attachment edges for each (EndPoint, TargetNode) pair using a sphere-query, 
	//            storing every valid edge instead of only the closest.
	//   Phase 1: Run an initial ComputeLengthFromRoot + TraceBranches on the full mesh. Only the root
	//            component obtains valid LengthFromRoot / NextBranchPoint values; all other components
	//            have LengthFromRoot=-1 / NextBranchPoint=INDEX_NONE.
	//   Phase 2: BFS level-by-level. For each level:
	//            - For each already-connected node, find unvisited source nodes with EndPoints nearby.
	//            - For each such source node, pick the candidate edge on the connected node that lies
	//              on the current trace (NextBranchPoint[A]==B or NextBranchPoint[B]==A). Fall back to
	//              the closest valid edge if no on-trace edge is found.
	//            - Apply all accepted connections for this level (SplitEdge + MergeVertices).
	//            - Re-trace so newly connected components obtain valid trace attributes for the next level.
	// 
	// Note that the ComputeLengthFromRoot+TraceBranches calls will re-trace the entire mesh. This is not 
	// optimal, however, these calls are cheap compared to the rest of the import function (less than 1% 
	// of the overal cost)

	TRACE_CPUPROFILER_EVENT_SCOPE(ConnectAndTraceBranches);

	using namespace PV::ImportHelper;

	FMeshConnectedComponents ConnectedComponents(&DynamicMesh);
	ConnectedComponents.FindConnectedVertices();

	const TFunction<bool(int32)> VertexFilter = [&](int32 VertexID) { return !FillerVertexAttribute.GetValue(VertexID); };

	if (ConnectedComponents.Components.Num() == 1)
	{
		// Single component: skip the connection step and go straight to trace.
		ComputeLengthFromRoot(DynamicMesh, RootPointAttribute, EndPointAttribute, LengthFromRootAttribute, VertexFilter);
		TraceBranches(DynamicMesh, LengthFromRootAttribute, RootPointAttribute, EndPointAttribute, NextBranchPointAttribute, VertexFilter);
		return;
	}

	const TArray<int32> EndPointVertexIndices = EndPointAttribute.FindAllNonZero();
	const int32 NumEndPoints = EndPointVertexIndices.Num();

	TArray<int32> VertexIndexToNodeIndex;
	{
		VertexIndexToNodeIndex.SetNum(DynamicMesh.MaxVertexID());
		for (int32 i = 0; i < ConnectedComponents.Components.Num(); ++i)
		{
			for (int32 VertexID : ConnectedComponents.Components[i].Indices)
			{
				VertexIndexToNodeIndex[VertexID] = i;
			}
		}
	}

	struct FCandidateEdge
	{
		FIndex2i Edge                 = FIndex2i(INDEX_NONE, INDEX_NONE);
		FVector  TargetPoint          = FVector::ZeroVector;
		float    Dist                 = 0.f;
		float    TargetPointEdgeAlpha = 0.f;
		float    TargetPointScale     = 0.f;
	};

	// EndPointsCandidateEdges[EndPointIndex][TargetNodeIndex] = list of valid candidate edges on TargetNode
	TArray<TMap<int32, TArray<FCandidateEdge>>> EndPointsCandidateEdges;
	EndPointsCandidateEdges.SetNum(NumEndPoints);

	// NodesIncomingEndPoints[NodeIndex][IncomingNodeIndex] = list of incoming endpoints from a particular node
	using FEndPointArray = TArray<int32, TInlineAllocator<4>>;
	TArray<TMap<int32, FEndPointArray>> NodesIncomingEndPoints;
	NodesIncomingEndPoints.SetNum(ConnectedComponents.Components.Num());

	// Build candidate edges
	const FDynamicMeshAABBTree3 DynamicMeshAABBTree(&DynamicMesh);
	float GlobalMaxVertexScale = 0.f;
	for (int32 VID : DynamicMesh.VertexIndicesItr())
	{
		GlobalMaxVertexScale = FMath::Max(GlobalMaxVertexScale, VertexScaleAttribute.GetValue(VID));
	}

	for (int32 EndPointIndex = 0; EndPointIndex < NumEndPoints; ++EndPointIndex)
	{
		const int32 EndPointVertexIndex = EndPointVertexIndices[EndPointIndex];
		const FVector VertexPosition    = DynamicMesh.GetVertex(EndPointVertexIndex);
		const int32 EndPointNodeIndex   = VertexIndexToNodeIndex[EndPointVertexIndex];
		const float ThisPointScale      = VertexScaleAttribute.GetValue(EndPointVertexIndex);
		const float ScaleInflation      = 1.25f;
		const PV::DynamicMeshHelper::FFindMeshElemsResult CloseTriangles = PV::DynamicMeshHelper::FindTrianglesInRadius(
			DynamicMeshAABBTree,
			VertexPosition,
			ThisPointScale * ScaleInflation,
			[&VertexScaleAttribute, ScaleInflation](int32 VID) { return VertexScaleAttribute.GetValue(VID) * ScaleInflation; },
			GlobalMaxVertexScale * ScaleInflation
		);

		TMap<int32, TArray<FIndex3i>> NodeToTriangles;
		for (const int32 TriangleID : CloseTriangles)
		{
			const FIndex3i Triangle = DynamicMesh.GetTriangle(TriangleID);
			const int32 OtherNodeIndex = VertexIndexToNodeIndex[Triangle.A];
			if (OtherNodeIndex == EndPointNodeIndex)
			{
				continue;
			}
			NodeToTriangles.FindOrAdd(OtherNodeIndex).Add(Triangle);
		}

		for (const auto& [NodeIndex, Triangles] : NodeToTriangles)
		{
			TArray<FCandidateEdge>& CandidateEdges = EndPointsCandidateEdges[EndPointIndex].FindOrAdd(NodeIndex);

			for (const FIndex3i& Triangle : Triangles)
			{
				for (int32 j = 0; j < 3; ++j)
				{
					FIndex2i Edge = FIndex2i(Triangle[j], Triangle[(j + 1) % 3]);
					Edge.Sort();

					if (FillerVertexAttribute.GetValue(Edge.A) || FillerVertexAttribute.GetValue(Edge.B))
					{
						continue;
					}

					const FVector VertexA      = DynamicMesh.GetVertex(Edge.A);
					const FVector VertexB      = DynamicMesh.GetVertex(Edge.B);
					const FVector ClosestPoint = FMath::ClosestPointOnSegment(VertexPosition, VertexA, VertexB);

					const float DistAlongEdge = (VertexA - ClosestPoint).Size();
					const float EdgeSize      = (VertexB - VertexA).Size();
					const float EdgeAlpha     = EdgeSize > 0.f ? DistAlongEdge / EdgeSize : 0.f;

					const float VertexScaleA    = VertexScaleAttribute.GetValue(Edge.A);
					const float VertexScaleB    = VertexScaleAttribute.GetValue(Edge.B);
					const float TargetScale     = FMath::Lerp(VertexScaleA, VertexScaleB, EdgeAlpha);

					const float DistToEdge = (VertexPosition - ClosestPoint).Size();
					if (DistToEdge > (TargetScale + ThisPointScale) * ScaleInflation)
					{
						continue; // radii do not overlap — not a valid attachment
					}

					const bool bAlreadyAdded = CandidateEdges.FindByPredicate([&](const auto& X) { return X.Edge == Edge; }) != nullptr;
					if (bAlreadyAdded)
					{
						continue;
					}

					FCandidateEdge& Candidate = CandidateEdges.AddDefaulted_GetRef();
					Candidate.Edge                 = Edge;
					Candidate.TargetPoint          = ClosestPoint;
					Candidate.Dist                 = DistToEdge;
					Candidate.TargetPointEdgeAlpha = EdgeAlpha;
					Candidate.TargetPointScale     = TargetScale;
				}
			}

			if (CandidateEdges.Num() > 0)
			{
				NodesIncomingEndPoints[NodeIndex].FindOrAdd(EndPointNodeIndex).Add(EndPointIndex);
			}
		}
	}

	// Phase 1: Initial trace — seeds from real root points.  Disconnected components remain at
	// LengthFromRoot=-1 / NextBranchPoint=INDEX_NONE, which is the signal used in Phase 2 to
	// identify edges that are not yet on the trace.
	ComputeLengthFromRoot(DynamicMesh, RootPointAttribute, EndPointAttribute, LengthFromRootAttribute, VertexFilter);
	TraceBranches(DynamicMesh, LengthFromRootAttribute, RootPointAttribute, EndPointAttribute, NextBranchPointAttribute, VertexFilter);

	// Phase 2: Iterative BFS.  Process one hop per loop iteration so we can re-trace after each
	// level and use the updated NextBranchPoint values when selecting attachment edges for the next.
	TArray<bool> VisitedNodes;
	VisitedNodes.SetNumZeroed(ConnectedComponents.Components.Num());

	int32 BFSQueueIndex = 0;
	TArray<int32> BFSQueues[2];
	{
		const TArray<int32> RootPointVertexIndices = RootPointAttribute.FindAllNonZero();
		for (int32 RootPointVertexIndex : RootPointVertexIndices)
		{
			const int32 RootNodeIndex = VertexIndexToNodeIndex[RootPointVertexIndex];
			if (!VisitedNodes[RootNodeIndex])
			{
				VisitedNodes[RootNodeIndex] = true;
				BFSQueues[BFSQueueIndex].Add(RootNodeIndex);
			}
		}
	}

	struct FLevelConnection
	{
		int32    TargetNode           = INDEX_NONE;
		int32    SourceNode           = INDEX_NONE;
		int32    EndPointIndex        = INDEX_NONE;
		FIndex2i TargetEdge           = FIndex2i(INDEX_NONE, INDEX_NONE);
		FVector  TargetPoint          = FVector::ZeroVector;
		float    TargetPointEdgeAlpha = 0.f;
		float    TargetPointScale     = 0.f;
		float    Dist                 = MAX_flt;
	};
	TArray<FLevelConnection> LevelConnections;
	TArray<int32> EndPointToAttachVertex;

	// Union-find over vertex IDs: when a vertex is merged away its ID maps to the survivor.
	// ResolveVertexID walks the chain (with path compression) to translate a stale ID captured
	// in an earlier level into the current surviving vertex.
	TMap<int32, int32> VertexIDRemap;
	const auto ResolveVertexID = [&VertexIDRemap](int32 ID) -> int32
	{
		if (ID == INDEX_NONE)
		{
			return INDEX_NONE;
		}
		int32 Root = ID;
		while (const int32* Next = VertexIDRemap.Find(Root))
		{
			Root = *Next;
		}
		while (ID != Root)
		{
			int32& Slot = VertexIDRemap.FindChecked(ID);
			const int32 Next = Slot;
			Slot = Root;
			ID = Next;
		}
		return Root;
	};

	while (BFSQueues[BFSQueueIndex].Num() > 0)
	{
		const int32 NextBFSQueueIndex = (BFSQueueIndex + 1) % 2;
		TArray<int32>& NextBFSQueue = BFSQueues[NextBFSQueueIndex];
		BFSQueues[NextBFSQueueIndex].Reset();
		LevelConnections.Reset();

		for (int32 NodeIndex : BFSQueues[BFSQueueIndex])
		{
			for (const auto& [SourceNodeIndex, EndPointIndices] : NodesIncomingEndPoints[NodeIndex])
			{
				if (VisitedNodes[SourceNodeIndex])
				{
					continue;
				}

				// Find the best (EndPoint, Edge) pair for connecting SourceNode to NodeIndex.
				FLevelConnection BestConnection;
				BestConnection.TargetNode = NodeIndex;
				BestConnection.SourceNode = SourceNodeIndex;

				for (int32 EndPointIndex : EndPointIndices)
				{
					const TArray<FCandidateEdge>* CandidatesPtr = EndPointsCandidateEdges[EndPointIndex].Find(NodeIndex);
					if (!CandidatesPtr || CandidatesPtr->Num() == 0)
					{
						continue;
					}

					bool     bFoundOnTrace    = false;
					FIndex2i BestEdge         = FIndex2i(INDEX_NONE, INDEX_NONE);
					FVector  BestTargetPoint  = FVector::ZeroVector;
					float    BestDist         = MAX_flt;
					float    BestEdgeAlpha    = 0.f;
					float    BestTargetScale  = 0.f;

					for (const FCandidateEdge& Candidate : *CandidatesPtr)
					{
						const int32 NextA   = NextBranchPointAttribute.GetValue(Candidate.Edge.A);
						const int32 NextB   = NextBranchPointAttribute.GetValue(Candidate.Edge.B);
						const bool bOnTrace = (NextA == Candidate.Edge.B) || (NextB == Candidate.Edge.A);

						if (bOnTrace && (!bFoundOnTrace || Candidate.Dist < BestDist))
						{
							bFoundOnTrace   = true;
							BestEdge        = Candidate.Edge;
							BestTargetPoint = Candidate.TargetPoint;
							BestDist        = Candidate.Dist;
							BestEdgeAlpha   = Candidate.TargetPointEdgeAlpha;
							BestTargetScale = Candidate.TargetPointScale;
						}
						else if (!bOnTrace && !bFoundOnTrace && Candidate.Dist < BestDist)
						{
							// No on-trace candidate yet — keep the closest overall as fallback.
							BestEdge        = Candidate.Edge;
							BestTargetPoint = Candidate.TargetPoint;
							BestDist        = Candidate.Dist;
							BestEdgeAlpha   = Candidate.TargetPointEdgeAlpha;
							BestTargetScale = Candidate.TargetPointScale;
						}
					}

					if (BestEdge.A != INDEX_NONE && BestDist < BestConnection.Dist)
					{
						BestConnection.EndPointIndex        = EndPointIndex;
						BestConnection.TargetEdge           = BestEdge;
						BestConnection.TargetPoint          = BestTargetPoint;
						BestConnection.Dist                 = BestDist;
						BestConnection.TargetPointEdgeAlpha = BestEdgeAlpha;
						BestConnection.TargetPointScale     = BestTargetScale;
					}
				}

				if (BestConnection.EndPointIndex != INDEX_NONE)
				{
					LevelConnections.Add(BestConnection);
					NextBFSQueue.Add(SourceNodeIndex);
					VisitedNodes[SourceNodeIndex] = true;
				}
			}
		}

		if (LevelConnections.IsEmpty())
		{
			break;
		}

		// Apply connections: group by target edge, split with tail-advancing, then merge vertices.
		EndPointToAttachVertex.Init(INDEX_NONE, NumEndPoints);

		TMap<FIndex2i, TArray<int32>> EdgeToConnectionIndices;
		for (int32 i = 0; i < LevelConnections.Num(); ++i)
		{
			EdgeToConnectionIndices.FindOrAdd(LevelConnections[i].TargetEdge).Add(i);
		}

		for (auto& [Edge, ConnectionIndices] : EdgeToConnectionIndices)
		{
			ConnectionIndices.Sort([&](int32 A, int32 B)
			{
				return LevelConnections[A].TargetPointEdgeAlpha < LevelConnections[B].TargetPointEdgeAlpha;
			});

			// Edge.A/Edge.B come from candidate edges captured against the original mesh,
			// so they may be stale after earlier-level merges — resolve through the remap.
			int32 TailA       = ResolveVertexID(Edge.A);
			const int32 TailB = ResolveVertexID(Edge.B);
			for (const int32 ConnIdx : ConnectionIndices)
			{
				const FLevelConnection& Conn = LevelConnections[ConnIdx];

				const bool bSnapToTailA = DynamicMesh.GetVertex(TailA).Equals(Conn.TargetPoint, 1.f);
				if (bSnapToTailA)
				{
					EndPointToAttachVertex[Conn.EndPointIndex] = TailA;
					continue;
				}

				const bool bSnapToTailB = DynamicMesh.GetVertex(TailB).Equals(Conn.TargetPoint, 1.f);
				if (bSnapToTailB)
				{
					EndPointToAttachVertex[Conn.EndPointIndex] = TailB;
					continue;
				}

				if (DynamicMesh.FindEdge(TailA, TailB) == FDynamicMesh3::InvalidID)
				{
					// Both endpoints of the original candidate edge collapsed into the same survivor so the edge no longer exists.
					continue;
				}

				FDynamicMesh3::FEdgeSplitInfo EdgeSplitInfo;
				const UE::Geometry::EMeshResult SplitResult = DynamicMesh.SplitEdge(TailA, TailB, EdgeSplitInfo);
				if (!ensure(SplitResult == UE::Geometry::EMeshResult::Ok))
				{
					continue;
				}

				DynamicMesh.SetVertex(EdgeSplitInfo.NewVertex, Conn.TargetPoint);
				VertexScaleAttribute.SetValue(EdgeSplitInfo.NewVertex, Conn.TargetPointScale);
				FillerVertexAttribute.SetValue(EdgeSplitInfo.NewVertex, false);

				EndPointToAttachVertex[Conn.EndPointIndex] = EdgeSplitInfo.NewVertex;
				TailA = EdgeSplitInfo.NewVertex;
			}
		}

		for (const FLevelConnection& Conn : LevelConnections)
		{
			const int32 EndPointVertexIndex = EndPointVertexIndices[Conn.EndPointIndex];
			const int32 AttachVertex        = EndPointToAttachVertex[Conn.EndPointIndex];
			if (AttachVertex == INDEX_NONE)
			{
				continue;
			}

			EndPointAttribute.SetValue(EndPointVertexIndex, false);
			EndPointAttribute.SetValue(AttachVertex, false);

			FDynamicMesh3::FMergeVerticesOptions MergeOptions;
			MergeOptions.bAllowNonBoundaryBowtieCreation = true; // AttachVertex is interior (from SplitEdge)

			FDynamicMesh3::FMergeVerticesInfo MergeInfo;
			const UE::Geometry::EMeshResult MergeResult = DynamicMesh.MergeVertices(
				AttachVertex,        // KeepVid: attachment point on the parent skeleton
				EndPointVertexIndex, // DiscardVid: endpoint being welded in
				0.0,                 // InterpolationT: 0 = keep KeepVid position exactly
				MergeOptions,
				MergeInfo
			);
			if (ensure(MergeResult == UE::Geometry::EMeshResult::Ok))
			{
				VertexIDRemap.Add(EndPointVertexIndex, AttachVertex);
			}
		}

		// Re-trace so the newly connected components gain valid LengthFromRoot / NextBranchPoint
		// values, which are used to select on-trace attachment edges in the next BFS level.
		ComputeLengthFromRoot(DynamicMesh, RootPointAttribute, EndPointAttribute, LengthFromRootAttribute, VertexFilter);
		TraceBranches(DynamicMesh, LengthFromRootAttribute, RootPointAttribute, EndPointAttribute, NextBranchPointAttribute, VertexFilter);

		BFSQueueIndex = NextBFSQueueIndex;
	}
}

static void FindRootPoints(
	const FDynamicMesh3& DynamicMesh,
	FBooleanVertexAttribute& RootPointAttribute,
	FBooleanVertexAttribute& EndPointAttribute
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindRootPoints);

	// This function currently just finds all the end points that are close to the bottom of the plant bounding box and converts them to root points.
	// This could result in issues if a plant has multiple points touching the floor. 

	const FAxisAlignedBox3d BoxBounds = DynamicMesh.GetBounds();

	for (int32 VertexIndex : EndPointAttribute.FindAllNonZero())
	{
		const FVector VertexPosition = DynamicMesh.GetVertex(VertexIndex);

		const float Z = FMath::GetMappedRangeValueClamped(FVector2D(BoxBounds.Min.Z, BoxBounds.Max.Z), FVector2D(0, 1), VertexPosition.Z);
		const bool bIsRootPoint = Z <= 0.1;
		if (bIsRootPoint)
		{
			RootPointAttribute.SetValue(VertexIndex, true);
			EndPointAttribute.SetValue(VertexIndex, false);			
		}
	}
}

}

PV::StaticMeshImport::EImportResult PV::StaticMeshImport::ImportGrowthDataFromStaticMesh(const FPVImportStaticMeshParams& InParams, const FPVImportStaticMeshDebugParams& InDebugParams, FPVImportStaticMeshOutput& Output)
{
	using namespace PV::ImportHelper;
	
	TRACE_CPUPROFILER_EVENT_SCOPE(ImportGrowthDataFromStaticMesh);
	
	if (!InParams.StaticMeshAsset)
	{
		return EImportResult::InvalidSourceMesh;
	}

	if (InParams.StaticMeshAsset->IsCompiling())
	{
		return EImportResult::MeshNotReady;
	}

	if (InParams.MaterialsToKeep.Num() == 0)
	{
		return EImportResult::InvalidMaterialFilter;
	}

	UE::Geometry::FDynamicMesh3 DynamicMesh;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(StaticMeshToDynamicMesh);

		FText OutErrorMessage;
		UE::Conversion::FStaticMeshConversionOptions Options;
		if (!UE::Conversion::StaticMeshToDynamicMesh(InParams.StaticMeshAsset, DynamicMesh, OutErrorMessage, Options))
		{
			return EImportResult::InvalidMeshData;
		}
	}

	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::InitialMesh)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
	}
	
	if (!FilterTrianglesUsingMaterials(DynamicMesh, InParams.StaticMeshAsset, InParams.MaterialsToKeep))
	{
		return EImportResult::InvalidMaterialFilter;
	}

	const FVector VisualizationOffset = FVector(0, DynamicMesh.GetBounds().Extents().Y * 2, 0);
	AddDynamicMeshToCollection(DynamicMesh, Output.VisualizationCollection, FTransform(VisualizationOffset));

	ComputeNormals(DynamicMesh);
	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::ComputeNormals)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
	}

	ComputeVerticesCenterAndRadius(
		DynamicMesh,
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);
	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::ComputeVerticesCenterAndRadius)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		VertexScaleAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	CollapseBranchesToLine(
		DynamicMesh,
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		FuseToAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		FillerVertexAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);
	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::CollapseBranchesToLine)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		FuseToAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		VertexScaleAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	FindEndPoints(
		DynamicMesh,
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		FillerVertexAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);
	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::FindEndPoints)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		EndPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	FindRootPoints(
		DynamicMesh,
		RootPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);
	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::FindRootPoints)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		RootPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		EndPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	TArray<FMeshBranchHierarchy> DynamicMeshBranchHierarchies;
	ConnectAndTraceBranches(
		DynamicMesh,
		RootPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		FillerVertexAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		LengthFromRootAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		NextBranchPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);
	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::ConnectBranchesToParent)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		RootPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		VertexScaleAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		EndPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		FillerVertexAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		LengthFromRootAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		NextBranchPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	ExtractBranchHierarchies(
		DynamicMesh, 
		RootPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		NextBranchPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		LengthFromRootAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		DynamicMeshBranchHierarchies
	);


	TArray<FPVBranchHierarchyDescription> BranchHierarchyDescriptions;
	ConvertDynamicMeshBranchHierarchiesToBranchDescription(
		DynamicMesh,
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		DynamicMeshBranchHierarchies,
		BranchHierarchyDescriptions
	);

	if (InDebugParams.DebugState == EPVImportStaticMeshDebugState::ComputeBranchHierarchies)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		AddBranchHierarchiesToCollection(BranchHierarchyDescriptions, DebugCollection);
		RootPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		EndPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		VertexScaleAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		LengthFromRootAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		NextBranchPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	if (!PV::ImportHelper::GenerateGrowthDataFromBranchHierarchies(Output.GrowthDataCollection, BranchHierarchyDescriptions))
	{
		return EImportResult::FailedToGenerateGrowthData;
	}

	// Build visualization output
	{
		AddDynamicMeshVerticesToCollection(
			DynamicMesh,
			EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
			FTransform(VisualizationOffset),
			Output.VisualizationCollection,
			PVImportNames::TipVisualizationGroup,
			PVImportNames::TipVisualizationPositionAttribute
		);

		AddBranchHierarchiesToCollection(BranchHierarchyDescriptions, Output.VisualizationCollection, VisualizationOffset);
	}

	return EImportResult::Success;
}

