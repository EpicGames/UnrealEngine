// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Params/PVImportStaticMeshParams.h"
#include "GeometryCollection/ManagedArrayCollection.h"

class UStaticMesh;

namespace PV::StaticMeshImport
{
	enum class EImportResult
	{
		Success,
		InvalidSourceMesh,
		InvalidOutputPath,
		InvalidMeshData,
		MeshNotReady,
		InvalidMaterialFilter,
		FailedToGenerateGrowthData
	};

	EImportResult ImportGrowthDataFromStaticMesh(const FPVImportStaticMeshParams& InParams, const FPVImportStaticMeshDebugParams& InDebugParams, FPVImportStaticMeshOutput& Output);
};
