// Copyright Epic Games, Inc. All Rights Reserved.

#include "Implementations/PVImporter_Texture2D.h"
#include "PVImportCommon.h"
#include "Helpers/PVImportHelpers.h"
#include "Helpers/PVDynamicMeshHelpers.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "Utils/PVDynamicMeshVertexAttribute.h"

#include "IntVectorTypes.h"
#include "PCGSettings.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

#include "Materials/MaterialInterface.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "SpatialAlgo/PCGMarchingSquares.h"
#include "CompGeom/Delaunay2.h"
#include "Selections/MeshConnectedComponents.h"

namespace PV::Texture2DImport::Internal
{
	using FBooleanVertexAttribute = PV::ImportHelper::FBooleanVertexAttribute;
	using FFloatVertexAttribute = PV::ImportHelper::FFloatVertexAttribute; 
	
	using FIndex2i = UE::Geometry::FIndex2i;
	using FIndex3i = UE::Geometry::FIndex3i;
	using FVector2i = UE::Geometry::FVector2i;
	using FMeshConnectedComponents = UE::Geometry::FMeshConnectedComponents;
	using FDynamicMeshAABBTree3 = UE::Geometry::FDynamicMeshAABBTree3;
	using FDynamicMesh3 = UE::Geometry::FDynamicMesh3;
	using FDynamicMeshQueries3 = UE::Geometry::TMeshQueries<FDynamicMesh3>;
	using FPolygon2f = UE::Geometry::FPolygon2f;
	using FDelaunay2 = UE::Geometry::FDelaunay2;

	const static PV::TDynamicMeshVertexAttributeDefinition<bool> PerimeterAttributeDefinition(TEXT("Perimeter"));
	
	static int32 GetPixelIndex(const FVector2i& PixelCoords, int32 ImageWidth)
	{
		return PixelCoords.X + PixelCoords.Y * ImageWidth;
	}

	static FVector2f PixelCoordToUV(const FVector2i& PixelCoord, const FVector2i& ImageSize)
	{
		const float U = ImageSize.X > 1 ? PixelCoord.X / static_cast<float>(ImageSize.X - 1) : 0;
		const float V = ImageSize.Y > 1 ? PixelCoord.Y / static_cast<float>(ImageSize.Y - 1) : 0;
		return FVector2f(U, V);
	}
	
	static bool SampleImage(
		UTexture2D& InTexture2D, 
		FVector2i& InOutSampleSize, 
		TArray<bool>& OutBlackAndWhiteImageData, 
		bool bInvertImage, 
		float WhiteLevel,
		EPCGTextureColorChannel ColorChannel
	)
	{
#if WITH_EDITOR
		FTextureSource& InTextureSource = InTexture2D.Source;
		if (!InTextureSource.IsValid())
		{
			return false;
		}

		// FTextureSource::FMipLock is only available in editor. For the time being this functionality is not needed outside the editor despite this module being a 
		// runtime module, so we exclude this in non-editor builds.

		const FTextureSource::FMipLock LockedMip0(FTextureSource::ELockState::ReadOnly, &InTextureSource, 0);
		check(LockedMip0.IsValid());

		const int32 ImageWith =  LockedMip0.Image.GetWidth();
		const int32 ImageHeight = LockedMip0.Image.GetHeight();
		InOutSampleSize.X = FMath::Min(InOutSampleSize.X, ImageWith);
		InOutSampleSize.Y  = FMath::Min(InOutSampleSize.Y, ImageHeight);

		OutBlackAndWhiteImageData.SetNum(InOutSampleSize.X * InOutSampleSize.Y);
		for (int32 x = 0; x < InOutSampleSize.X; x++)
		{
			for (int32 y = 0; y < InOutSampleSize.Y; y++)
			{
				const FVector2i PixelCoords = FVector2i(x, y);
				const FVector2f UV = PixelCoordToUV(PixelCoords, InOutSampleSize);

				const int32 SamplePosX = FMath::RoundToInt(UV.X * (ImageWith - 1));
				const int32 SamplePosY = FMath::RoundToInt(UV.Y * (ImageHeight - 1));
				check(SamplePosX >= 0 && SamplePosX < ImageWith);
				check(SamplePosY >= 0 && SamplePosY < ImageHeight);
				
				const FLinearColor PixelColor = LockedMip0.Image.GetOnePixelLinear(SamplePosX, SamplePosY);

				float MaskedColor = 0;
				switch (ColorChannel)
				{
				case EPCGTextureColorChannel::Red:
					MaskedColor = PixelColor.R;
					break;
				case EPCGTextureColorChannel::Green:
					MaskedColor = PixelColor.G;
					break;
				case EPCGTextureColorChannel::Blue:
					MaskedColor = PixelColor.B;
					break;
				case EPCGTextureColorChannel::Alpha:
					MaskedColor = PixelColor.A;
					break;
				default:
					break;
				}

				const float Alpha = bInvertImage ? 1.0 - MaskedColor : MaskedColor;
				const int32 PixelIndex = GetPixelIndex(PixelCoords, InOutSampleSize.X);
				OutBlackAndWhiteImageData[PixelIndex] = Alpha > WhiteLevel;
			}
		}

		return true;
#else
		return false;
#endif
	}
	
	static void TraceImage(const TArray<bool>& ImageData, const FVector2i& ImageSize, TArray<FPolygon2f>& OutCurves)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TraceImage);
		
		const auto ValueQuery = [&](int32 X, int32 Y)->double 
		{ 
			return ImageData[GetPixelIndex(FVector2i(X, Y), ImageSize.X)]; 
		};
		const TArray<PCGSpatialAlgo::FPCGMarchingSquareResult> Results = PCGSpatialAlgo::MarchingSquares(
			ImageSize.X, 
			ImageSize.Y,
			1, 
			ValueQuery, 
			false
		);
		
		TArray<FVector2f> PerimeterCurve;
		OutCurves.Reserve(Results.Num());
		for (const PCGSpatialAlgo::FPCGMarchingSquareResult& Result : Results)
		{
			if (!Result.bClosed)
			{
				continue;
			}
			
			PerimeterCurve.Reset(Result.LinkedGridCoordinates.Num());
			for (const FVector2D& Coordinate : Result.LinkedGridCoordinates)
			{
				PerimeterCurve.Add(FVector2f(Coordinate.X / (ImageSize.X - 1), Coordinate.Y / (ImageSize.Y - 1)));
			}
			OutCurves.Emplace(MoveTemp(PerimeterCurve));
		}
	}
	
	static void RemoveSmallCurves(TArray<FPolygon2f>& InOutPerimeterCurves, float Threshold)
	{
		const float ThresholdSqr = Threshold * Threshold;
		for (int32 i = InOutPerimeterCurves.Num() - 1; i >= 0; i--)
		{
			const TArray<FVector2f>& Curve = InOutPerimeterCurves[i].GetVertices();
			FVector2f Min = FVector2f(MAX_flt);
			FVector2f Max = FVector2f(-MAX_flt);
			
			for (const FVector2f& Point : Curve)
			{
				Min.X = FMath::Min(Min.X, Point.X);
				Min.Y = FMath::Min(Min.Y, Point.Y);
				Max.X = FMath::Max(Max.X, Point.X);
				Max.Y = FMath::Max(Max.Y, Point.Y);
			}
			
			const float Area = (Max.X - Min.X) * (Max.Y - Min.Y); 
			const float AreaSqr = Area * Area;
			if (AreaSqr < ThresholdSqr)
			{
				InOutPerimeterCurves.RemoveAtSwap(i);
			}
		}
	}
	
	static void SmoothCurves(TArray<FPolygon2f>& InOutPerimeterCurves, int32 SmoothingIterations)
	{
		while (SmoothingIterations != 0)
		{
			for (FPolygon2f& PerimeterCurve : InOutPerimeterCurves)
			{
				const TArray<FVector2f>& Vertices = PerimeterCurve.GetVertices();

				TArray<FVector2f> SmoothedCurve;
				SmoothedCurve.SetNum(Vertices.Num());
			
				for (int32 i = 0; i < Vertices.Num(); i++)
				{
					const FVector2f& Pos1 = i == 0 ? Vertices.Last() : Vertices[i - 1];
					const FVector2f& Pos2 = Vertices[i];
					const FVector2f& Pos3 = i == Vertices.Num() - 1 ? Vertices[0] : Vertices[i + 1];
				
					const FVector2f AvgPos = (Pos1 + Pos2 + Pos3) / 3; 
					SmoothedCurve[i] = AvgPos;
				}
			
				PerimeterCurve = FPolygon2f(MoveTemp(SmoothedCurve));
			}
			
			SmoothingIterations -= 1;
		}
	}
	
	static void SimplifyCurves(TArray<FPolygon2f>& InOutPerimeterCurves, float SimplificationAmount)
	{
		for (FPolygon2f& PerimeterCurve : InOutPerimeterCurves)
		{
			const float LineDeviationTolerance = SimplificationAmount * 0.001;
			const float ClusterTolerance = LineDeviationTolerance * 0.001;
			PerimeterCurve.Simplify(ClusterTolerance, LineDeviationTolerance);
		}
	}
	
	static void TriangulateCurvesAndSplitEdges(const TArray<FPolygon2f>& InPerimeterCurves, UE::Geometry::FDynamicMesh3& OutMesh, const TArray<TArray<bool>>& InTips)
	{
		using namespace PV::ImportHelper;

		/*
		* This function convertes the perimeter curves into a DynamicMesh and adds a vertex at the center of each edge spanning two perimeter points.
		* 
		* It does this by first triangulating the curve using FDelaunay2, it then looks at each edge and if said edge has two perimeter points, it inserts a new vertex
		* at the center of the edge. It does this for all edges and then re-triangulates the entire mesh with this new vertex set.
		* 
		* The end result will look something like the following diagram:
		*      *      
		*     / \           /|\
		*    /   \         / | \
		*   /     \       /__|__\
		*   *     *       |\ | /|
		*   |     |  -->  | \|/ |
		*   *     *       |--|--|
		*   |     |       | /|\ |
		*   *     *       |/ | \|
		*/

		PerimeterAttributeDefinition.AttachAttribute(OutMesh);
		PV::ImportHelper::EndPointAttributeDefinition.AttachAttribute(OutMesh);

		for (int32 CurveIndex = 0; CurveIndex < InPerimeterCurves.Num(); CurveIndex++)
		{
			const FPolygon2f& PerimeterCurve = InPerimeterCurves[CurveIndex];
			const TArray<FVector2f>& CurveVertices = PerimeterCurve.GetVertices();

			const TArray<FIndex2i> OriginalCurveEdges = Invoke([&]() 
			{
				TArray<FIndex2i> Edges;
				Edges.Reserve(CurveVertices.Num());

				for (int32 Last = CurveVertices.Num() - 1, Idx = 0; Idx < CurveVertices.Num(); Last = Idx++)
				{
					Edges.Emplace(Last, Idx);
				}

				return Edges;
			});
			
			FDelaunay2 Delaunay;
			Delaunay.Triangulate(CurveVertices, OriginalCurveEdges);
			
			bool bTriangulateSucess = Delaunay.GetResult() == UE::Geometry::FDelaunay2::EResult::MissingEdges
				|| Delaunay.GetResult() == UE::Geometry::FDelaunay2::EResult::Success;

			if (!bTriangulateSucess)
			{
				continue;
			}

			const FDelaunay2::EFillMode FillMode = PerimeterCurve.IsClockwise() 
				? FDelaunay2::EFillMode::NegativeWinding 
				: FDelaunay2::EFillMode::PositiveWinding;
			
			TArray<FIndex3i> OriginalCurveTriangles;
			Delaunay.GetFilledTrianglesGeneralizedWinding(OriginalCurveTriangles, CurveVertices, OriginalCurveEdges, FillMode);

			TArray<FVector2f> SubdividedMeshPoints;
			SubdividedMeshPoints.Reserve(CurveVertices.Num() + OriginalCurveTriangles.Num());
			SubdividedMeshPoints.Append(CurveVertices);
			
			TArray<FIndex2i> SubdividedMeshEdges;
			SubdividedMeshEdges.Reserve(OriginalCurveTriangles.Num() * 4);
			SubdividedMeshEdges.Append(OriginalCurveEdges);
			
			TSet<FIndex2i> CheckedEdges;
			CheckedEdges.Reserve(OriginalCurveTriangles.Num() * 3);
			CheckedEdges.Append(OriginalCurveEdges);
			
			for (const FIndex3i& Triangle : OriginalCurveTriangles)
			{
				for (int32 i = 0; i < 3; i++)
				{
					FIndex2i EdgeIndex(Triangle[i], Triangle[(i + 1) % 3]);
					EdgeIndex.Sort();
					
					bool bAlreadyAdded = false;
					CheckedEdges.Add(EdgeIndex, &bAlreadyAdded);
					if (!bAlreadyAdded)
					{
						const int32 PointIndex1 = EdgeIndex[0];
						const int32 PointIndex2 = EdgeIndex[1];
						const FVector2f& Pos1 = CurveVertices[PointIndex1];
						const FVector2f& Pos2 = CurveVertices[PointIndex2];
						const int32 NewPointIndex = SubdividedMeshPoints.Add((Pos1 + Pos2) / 2);
						
						const FIndex2i NewEdge1(PointIndex1, NewPointIndex);
						const FIndex2i NewEdge2(PointIndex2, NewPointIndex);
						
						SubdividedMeshEdges.Add(NewEdge1);
						SubdividedMeshEdges.Add(NewEdge2);
					}
				}
			}
			
			Delaunay.Triangulate(SubdividedMeshPoints, SubdividedMeshEdges);

			bTriangulateSucess = Delaunay.GetResult() == UE::Geometry::FDelaunay2::EResult::MissingEdges
				|| Delaunay.GetResult() == UE::Geometry::FDelaunay2::EResult::Success;

			if (!bTriangulateSucess)
			{
				continue;
			}
			
			UE::Geometry::FDynamicMesh3 NewDynamicMesh;
			PerimeterAttributeDefinition.AttachAttribute(NewDynamicMesh);
			PV::ImportHelper::EndPointAttributeDefinition.AttachAttribute(NewDynamicMesh);

			auto& PerimeterAttribute = PerimeterAttributeDefinition.GetAttributeChecked(NewDynamicMesh);
			auto* EndPointAttribute = &PV::ImportHelper::EndPointAttributeDefinition.GetAttributeChecked(NewDynamicMesh);

			const TArray<FVector2f>& Vertices = SubdividedMeshPoints;
			TArray<FIndex3i> Triangles;
			Delaunay.GetFilledTrianglesGeneralizedWinding(Triangles, Vertices, OriginalCurveEdges, FillMode);
			
			for (int32 i = 0; i < Vertices.Num(); ++i)
			{
				const FVector2f& Vertex = Vertices[i];
				const UE::Geometry::FVertexInfo VertexInfo(FVector(Vertex.X, Vertex.Y, 0), FVector3f(-1, 0, 0));
				const int32 VertexID = NewDynamicMesh.AppendVertex(VertexInfo);
				PerimeterAttribute.SetValue(VertexID, i < CurveVertices.Num());
				if (EndPointAttribute && i < CurveVertices.Num())
				{
					EndPointAttribute->SetValue(VertexID, InTips[CurveIndex][i]);
				}
			}
			
			for (const FIndex3i& Triangle : Triangles)
			{
				NewDynamicMesh.AppendTriangle(Triangle);
			}
			
			OutMesh.AppendWithOffsets(NewDynamicMesh);
		}
	}

	static void ComputeVertexScale2D(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const FBooleanVertexAttribute& PerimeterAttribute,
		FFloatVertexAttribute& VertexScaleAttribute
	)
	{
		/*
		* This function computes the vertex scale for each vertex in the mesh by finding the nearest point on the perimeter curve.
		*/

		const auto FindDistToPerimeter = [&](const FVector3d& Point, const TArray<int32>& PerimeterVertices)
		{
			// TODO: This algorithm is N^2, we can probably reduce that by using FSparseDynamicOctree3

			double ClosestDistSqr = MAX_flt;

			for (int32 i = 0; i < PerimeterVertices.Num(); ++i)
			{
				const int32 PointIndex0 = PerimeterVertices[i];
				const int32 PointIndex1 = PerimeterVertices[FMath::WrapExclusive(i + 1, 0, PerimeterVertices.Num())];

				const FVector3d& VertexLocation1 = DynamicMesh.GetVertex(PointIndex0);
				const FVector3d& VertexLocation2 = DynamicMesh.GetVertex(PointIndex1);

				const FVector3d ClosestPointOnLine = FMath::ClosestPointOnSegment(Point, VertexLocation1, VertexLocation2);
				const double DistToLineSqr = (Point - ClosestPointOnLine).SizeSquared();
				if (DistToLineSqr < ClosestDistSqr)
				{
					ClosestDistSqr = DistToLineSqr;
				}
			}

			return ClosestDistSqr > 0 ? FMath::Sqrt(ClosestDistSqr) : 0;
		};

		UE::Geometry::FMeshConnectedComponents ConnectedComponents(&DynamicMesh);
		ConnectedComponents.FindConnectedVertices();

		for (const auto& Component : ConnectedComponents)
		{
			TArray<int32> PerimeterVertices = PerimeterAttribute.FindAllNonZero(Component.Indices);
			PerimeterVertices.Sort(); // This assumes perimeter vertices are ordered sequentially, which is currently the case.

			for (int32 VertexID : Component.Indices)
			{
				if (!PerimeterAttribute.GetValue(VertexID))
				{
					const float VertexScale = FindDistToPerimeter(DynamicMesh.GetVertex(VertexID), PerimeterVertices);
					VertexScaleAttribute.SetValue(VertexID, VertexScale);
				}
			}
		}
	}

	struct FPlantMesh
	{
		FMeshConnectedComponents::FComponent MeshComponent;
		int32 RootVertexIndex = INDEX_NONE;
	};

	static void MapPlantSettingsToMeshComponents(
		const UE::Geometry::FDynamicMesh3& DynamicMesh,
		const TArray<FPVImportTexture2DPlantSettings>& InputPlantSettings,
		TArray<FPlantMesh>& OutPlantMapping
	)
	{
		using namespace PV::DynamicMeshHelper;
		using namespace PV::ImportHelper;

		/*
		* 1. Compute the distance to the closest mesh component for each InputPlantSettings
		* 2. For each mesh component, select the closest InputPlantSettings
		* 3. Determine which vertex to use as root point
		*/

		FMeshConnectedComponents Components(&DynamicMesh);
		Components.FindConnectedTriangles();

		TArray<FDynamicMeshAABBTree3> AABBTrees;
		AABBTrees.Reserve(Components.Components.Num());

		for (const auto& Component : Components)
		{
			FDynamicMeshAABBTree3 DynamicMeshAABBTree(&DynamicMesh, false);
			DynamicMeshAABBTree.Build(Component.Indices);

			AABBTrees.Add(MoveTemp(DynamicMeshAABBTree));
		}

		struct FClosestTriangle
		{
			int32 ComponentIndex = INDEX_NONE;
			double DistSqr = TNumericLimits<double>::Max();
			int32 ClosestTriangle = INDEX_NONE;
		};
		TArray<FClosestTriangle> PlantSettingsClosestTriangle;
		PlantSettingsClosestTriangle.SetNum(InputPlantSettings.Num());

		for (int32 PlantIndex = 0; PlantIndex < InputPlantSettings.Num(); ++PlantIndex)
		{
			const FPVImportTexture2DPlantSettings& PlantSettings = InputPlantSettings[PlantIndex];
			FClosestTriangle& ClosestTriangle = PlantSettingsClosestTriangle[PlantIndex];

			for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
			{
				const FDynamicMeshAABBTree3& AABBTree = AABBTrees[ComponentIndex];

				double DistSqr = 0;
				const FVector3d RootPoint = FVector(PlantSettings.RootPosition.X, PlantSettings.RootPosition.Y, 0);
				const int32 NearTriID = AABBTree.FindNearestTriangle(RootPoint, DistSqr);
				if (NearTriID != INDEX_NONE)
				{
					UE::Geometry::FDistPoint3Triangle3d Query = FDynamicMeshQueries3::TriangleDistance(DynamicMesh, NearTriID, RootPoint);
					DistSqr = Query.GetSquared();
					if (DistSqr < ClosestTriangle.DistSqr)
					{
						ClosestTriangle.ComponentIndex = ComponentIndex;
						ClosestTriangle.DistSqr = DistSqr;
						ClosestTriangle.ClosestTriangle = NearTriID;
					}
				}
			}
		}

		struct FPlantIndexDistPair
		{
			int32 PlantIndex = INDEX_NONE;
			double DistSqr = TNumericLimits<double>::Max();
		};
		TArray<FPlantIndexDistPair> ComponentToPlantDist;
		ComponentToPlantDist.SetNum(Components.Num());

		for (int32 PlantIndex = 0; PlantIndex < InputPlantSettings.Num(); ++PlantIndex)
		{
			const int32 ComponentIndex = PlantSettingsClosestTriangle[PlantIndex].ComponentIndex;
			if (ComponentIndex == INDEX_NONE)
			{
				continue;
			}

			if (PlantSettingsClosestTriangle[PlantIndex].DistSqr < ComponentToPlantDist[ComponentIndex].DistSqr)
			{
				ComponentToPlantDist[ComponentIndex].PlantIndex = PlantIndex;
				ComponentToPlantDist[ComponentIndex].DistSqr = PlantSettingsClosestTriangle[PlantIndex].DistSqr;
			}
		}

		OutPlantMapping.SetNum(InputPlantSettings.Num());
		for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
		{
			if (ComponentToPlantDist[ComponentIndex].PlantIndex == INDEX_NONE)
			{
				// Unable to map this plant setting to a mesh component
				continue;
			}

			const int32 PlantIndex = ComponentToPlantDist[ComponentIndex].PlantIndex;
			const FPVImportTexture2DPlantSettings& PlantSettings = InputPlantSettings[PlantIndex];

			const FVector3d& RootPosition = FVector(PlantSettings.RootPosition.X, PlantSettings.RootPosition.Y, 0);
			const int32 TriangleID = PlantSettingsClosestTriangle[PlantIndex].ClosestTriangle;
			const FIndex3i& Triangle = DynamicMesh.GetTriangle(TriangleID);
			const int32 ClosestTriangleVertex = FMath::Min3Index(
				(RootPosition - DynamicMesh.GetVertex(Triangle[0])).SizeSquared(),
				(RootPosition - DynamicMesh.GetVertex(Triangle[1])).SizeSquared(),
				(RootPosition - DynamicMesh.GetVertex(Triangle[2])).SizeSquared()
			);

			const int32 RootVertexID = Triangle[ClosestTriangleVertex];

			OutPlantMapping[PlantIndex] = { MoveTemp(Components[ComponentIndex]), RootVertexID };
		}
	}

	static void MapPlantIndicesToBranchHierarchies(
		const TArray<FPlantMesh>& PlantMapping,
		const TArray<PV::ImportHelper::FMeshBranchHierarchy>& DynamicMeshBranchHierarchies,
		TArray<int32>& OutBranchHierarchiesToPlantIndices
	)
	{
		TMap<int32, int32> RootVertexPlantIndexMap;
		RootVertexPlantIndexMap.Reserve(PlantMapping.Num());
		for (int32 PlantIndex = 0; PlantIndex < PlantMapping.Num(); ++PlantIndex)
		{
			const FPlantMesh& PlantMesh = PlantMapping[PlantIndex];
			RootVertexPlantIndexMap.Emplace(PlantMesh.RootVertexIndex, PlantIndex);
		}

		OutBranchHierarchiesToPlantIndices.SetNum(DynamicMeshBranchHierarchies.Num());
		for (int32& PlantIndex : OutBranchHierarchiesToPlantIndices)
		{
			PlantIndex = INDEX_NONE;
		}

		for (int32 i = 0; i < DynamicMeshBranchHierarchies.Num(); ++i)
		{
			check(DynamicMeshBranchHierarchies[i].Branches.Num() > 0);
			check(DynamicMeshBranchHierarchies[i].Branches[0].BranchPoints.Num() > 0);

			const int32 RootVertexID = DynamicMeshBranchHierarchies[i].Branches[0].BranchPoints[0];
			if (int32* PlantIndex = RootVertexPlantIndexMap.Find(RootVertexID))
			{
				OutBranchHierarchiesToPlantIndices[i] = *PlantIndex;
			}
		}
	}

	static void ApplyPlantMappingToDynamicMesh(
		UE::Geometry::FDynamicMesh3& DynamicMesh,
		const TArray<FPVImportTexture2DPlantSettings>& InputPlantSettings,
		const TArray<FPlantMesh>& InPlantMapping,
		FBooleanVertexAttribute& RootPointAttribute,
		FBooleanVertexAttribute& EndPointAttribute,
		FFloatVertexAttribute& VertexScaleAttribute
	)
	{
		check(InputPlantSettings.Num() == InPlantMapping.Num());

		for (int32 PlantIndex = 0; PlantIndex < InputPlantSettings.Num(); ++PlantIndex)
		{
			const FPlantMesh& PlantMesh = InPlantMapping[PlantIndex];
			if (PlantMesh.RootVertexIndex == INDEX_NONE)
			{
				continue;
			}

			const int32 RootVertexID = PlantMesh.RootVertexIndex;
			const FPVImportTexture2DPlantSettings& PlantSettings = InputPlantSettings[PlantIndex];

			RootPointAttribute.SetValue(RootVertexID, true);
			EndPointAttribute.SetValue(RootVertexID, false);

			const FVector VertexTranslation = DynamicMesh.GetVertex(RootVertexID);
			const FQuat VertexRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(PlantSettings.Rotation));

			for (int32 VertexIndex : PV::DynamicMeshHelper::GetUniqueVertexIndices(DynamicMesh, PlantMesh.MeshComponent.Indices))
			{
				FVector Vertex = DynamicMesh.GetVertex(VertexIndex);
				Vertex -= VertexTranslation;
				Vertex = VertexRotation.RotateVector(Vertex);
				Vertex *= PlantSettings.Scale;
				Vertex = FVector(0, Vertex.X, 1.0 - Vertex.Y); // Convert from UV space to world space
				DynamicMesh.SetVertex(VertexIndex, Vertex);

				VertexScaleAttribute.SetValue(VertexIndex, VertexScaleAttribute.GetValue(VertexIndex) * PlantSettings.Scale);
			}
		}
	}
};

PV::Texture2DImport::ETracePerimeterCurvesResult PV::Texture2DImport::TracePerimeterCurves(
	TObjectPtr<UTexture2D> Texture2DAsset,
	int32 SampleResolution,
	bool bInvertImage,
	float WhiteLevel,
	EPCGTextureColorChannel ColorChannel,
	float MinBoundsArea,
	int32 SmoothingIterations,
	float SimplificationAmount,
	TArray<UE::Geometry::FPolygon2f>& OutPerimeterCurves
)
{
	using namespace Internal;

	if (!Texture2DAsset)
	{
		return ETracePerimeterCurvesResult::InvalidSourceTexture;
	}

	FVector2i SampleSize = FVector2i(SampleResolution, SampleResolution); // Non-uniform textures are treated as uniform so we do not scale the SampleResolution with the aspect ratio.
	if (SampleSize.X <= 1 || SampleSize.Y <= 1)
	{
		return ETracePerimeterCurvesResult::InvalidSampleSize;
	}

	TArray<bool> BlackAndWhiteImageData;
	if (!SampleImage(*Texture2DAsset, SampleSize, BlackAndWhiteImageData, bInvertImage, WhiteLevel, ColorChannel))
	{
		return ETracePerimeterCurvesResult::InvalidSourceTexture;
	}

	if (SampleSize.X <= 1 || SampleSize.Y <= 1)
	{
		return ETracePerimeterCurvesResult::InvalidTextureSize;
	}

	TraceImage(BlackAndWhiteImageData, SampleSize, OutPerimeterCurves);
	RemoveSmallCurves(OutPerimeterCurves, MinBoundsArea);
	SmoothCurves(OutPerimeterCurves, SmoothingIterations);
	SimplifyCurves(OutPerimeterCurves, SimplificationAmount);

	return ETracePerimeterCurvesResult::Success;
}

PV::Texture2DImport::EFindTipsResult PV::Texture2DImport::FindTips(
	const TArray<UE::Geometry::FPolygon2f>& PerimeterCurves,
	float TipAngleThresholdInDegrees,
	float MaxTipAngleSearchDist,
	TArray<TArray<bool>>& OutTips
)
{
	using namespace Internal;

	/*
	* This function finds the end points (or "tips") on a 2D poly-curve by walking the perimeter vertices and checking the angle between itself and its nearest
	* two neighbours on the perimeter. To allow for curved (or flat-tipped) end points to be detected, the function takes a second pass where it walks outwards along the
	* perimited (limited by MaxTipAngleSearchDist), and checks the angle of the current vertex towards the subsequent neighbours. (see diagram a for example of flat-tipped end point)
	*
	*     a: ___     b:
	*       /   \      /\
	*       |   |     |  |
	*       |   |     |  |
	*/

	OutTips.SetNum(PerimeterCurves.Num());

	bool bAnyTipsFound = false;
	const float MaxTipAngleSearchDistSqr = MaxTipAngleSearchDist * MaxTipAngleSearchDist;

	for (int32 CurveIndex = 0; CurveIndex < PerimeterCurves.Num(); ++CurveIndex)
	{
		const FPolygon2f PolyCurve = PerimeterCurves[CurveIndex];
		const TArray<FVector2f>& Vertices = PolyCurve.GetVertices();
		const int32 NumVerts = Vertices.Num();

		TArray<bool>& TipFlags = OutTips[CurveIndex];
		TipFlags.SetNumZeroed(NumVerts);

		if (NumVerts < 3)
		{
			continue;
		}

		const bool bIsClockwise = PolyCurve.IsClockwise();

		// Returns the angle in degrees, positive for convex (tip) vertices and negative for concave (crevice) vertices.
		const auto ComputeSignedAngleDeg = [&](int32 Center, int32 Left, int32 Right) -> float
		{
			Left  = FMath::WrapExclusive(Left,  0, NumVerts);
			Right = FMath::WrapExclusive(Right, 0, NumVerts);

			const FVector2f N1 = (Vertices[Center] - Vertices[Left]).GetSafeNormal();
			const FVector2f N2 = (Vertices[Center] - Vertices[Right]).GetSafeNormal();

			const float Cross = FVector2f::CrossProduct(N1, N2);
			const float Sign = FMath::Sign(bIsClockwise ? Cross : -Cross);

			const float ClampedDot = FMath::Clamp(FVector2f::DotProduct(N1, N2), -1.f, 1.f);
			return Sign * (180.f - FMath::RadiansToDegrees(FMath::Acos(ClampedDot)));
		};

		TArray<float> PerimeterAngles;
		PerimeterAngles.SetNumZeroed(NumVerts);

		TArray<int32> ExcludedIndices;

		for (int32 i = 0; i < NumVerts; ++i)
		{
			const float Angle = ComputeSignedAngleDeg(i, i - 1, i + 1);

			if (Angle >= TipAngleThresholdInDegrees)
			{
				TipFlags[i] = true;
				PerimeterAngles[i] = Angle;
				bAnyTipsFound = true;
			}
			else
			{
				ExcludedIndices.Add(i);
			}
		}

		for (int32 i : ExcludedIndices)
		{
			float LeftDistSqr  = 0.f;
			float RightDistSqr = 0.f;

			int32 LeftIdx  = i;
			int32 RightIdx = i;

			while (LeftDistSqr < MaxTipAngleSearchDistSqr || RightDistSqr < MaxTipAngleSearchDistSqr)
			{
				if (LeftDistSqr < MaxTipAngleSearchDistSqr)
				{
					LeftIdx = FMath::WrapExclusive(LeftIdx - 1, 0, NumVerts);
					if (LeftIdx == i)
					{
						break;
					}

					if (TipFlags[LeftIdx])
					{
						LeftDistSqr = MaxTipAngleSearchDistSqr;
					}
					else
					{
						LeftDistSqr = (Vertices[i] - Vertices[LeftIdx]).SizeSquared();
					}
				}

				if (RightDistSqr < MaxTipAngleSearchDistSqr)
				{
					RightIdx = FMath::WrapExclusive(RightIdx + 1, 0, NumVerts);
					if (RightIdx == i)
					{
						break;
					}

					if (TipFlags[RightIdx])
					{
						RightDistSqr = MaxTipAngleSearchDistSqr;
					}
					else
					{
						RightDistSqr = (Vertices[i] - Vertices[RightIdx]).SizeSquared();
					}
				}

				const float Angle = ComputeSignedAngleDeg(i, LeftIdx, RightIdx);
				if (Angle >= TipAngleThresholdInDegrees)
				{
					TipFlags[i] = true;
					PerimeterAngles[i] = Angle;
					bAnyTipsFound = true;
					break;
				}
			}
		}

		// When two adjacent vertices are both tips, keep the one with the steeper angle
		for (int32 i = 0; i < NumVerts; ++i)
		{
			if (!TipFlags[i])
			{
				continue;
			}

			const int32 Next = FMath::WrapExclusive(i + 1, 0, NumVerts);
			if (TipFlags[Next])
			{
				const int32 ToRemove = PerimeterAngles[Next] >= PerimeterAngles[i] ? i : Next;
				TipFlags[ToRemove] = false;
			}
		}
	}

	return bAnyTipsFound ? EFindTipsResult::Success : EFindTipsResult::NoTipsFound;
}

PV::Texture2DImport::EImportResult PV::Texture2DImport::ImportGrowthDataFromPerimeterCurves(
	const TArray<UE::Geometry::FPolygon2f>& PerimeterCurves,
	const TArray<TArray<bool>>& Tips,
	const TArray<FPVImportTexture2DPlantSettings>& PlantSettings,
	EPVImportTexture2DDebugState DebugState,
	FPVImportTexture2DOutput& Output
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ImportGrowthDataFromPerimeterCurves);

	using namespace PV::ImportHelper;
	using namespace Internal;

	if (PerimeterCurves.Num() == 0)
	{
		return EImportResult::InvalidPerimeterCurves;
	}

	for (int32 i = 0; i < PerimeterCurves.Num(); ++i)
	{
		if (!Tips.IsValidIndex(i) || Tips[i].Num() != PerimeterCurves[i].VertexCount())
		{
			return EImportResult::InvalidTipsAttribute;
		}
	}

	UE::Geometry::FDynamicMesh3 DynamicMesh;
	TriangulateCurvesAndSplitEdges(PerimeterCurves, DynamicMesh, Tips);

	if (DebugState == EPVImportTexture2DDebugState::TriangulateCurves)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();

		const FMatrix44f UVToWorldMatrix = PVTexture2DImporterVisualization::GetUVToWorldMatrix();
		const FTransform3f DynamicMeshTransform = FTransform3f(UVToWorldMatrix);
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection, FTransform(DynamicMeshTransform));
		PerimeterAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	ComputeVertexScale2D(
		DynamicMesh, 
		PerimeterAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);

	if (DebugState == EPVImportTexture2DDebugState::ComputeVertexScale)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		VertexScaleAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	TArray<FPlantMesh> PlantMapping;
	MapPlantSettingsToMeshComponents(
		DynamicMesh,
		PlantSettings,
		PlantMapping
	);

	ApplyPlantMappingToDynamicMesh(
		DynamicMesh,
		PlantSettings,
		PlantMapping,
		RootPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh)
	);

	auto& PerimeterAttribute = PerimeterAttributeDefinition.GetOrAttachAttribute(DynamicMesh);
	
	TArray<FMeshBranchHierarchy> DynamicMeshBranchHierarchies;
	ComputeDynamicMeshBranchHierarchies(
		DynamicMesh,
		RootPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		LengthFromRootAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		NextBranchPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		[&](int32 VertexID) { return !PerimeterAttribute.GetValue(VertexID); },
		DynamicMeshBranchHierarchies
	);

	TArray<FPVBranchHierarchyDescription> BranchHierarchyDescriptions;
	ConvertDynamicMeshBranchHierarchiesToBranchDescription(
		DynamicMesh,
		VertexScaleAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		EndPointAttributeDefinition.GetOrAttachAttribute(DynamicMesh),
		DynamicMeshBranchHierarchies,
		BranchHierarchyDescriptions
	);

	check(DynamicMeshBranchHierarchies.Num() == BranchHierarchyDescriptions.Num());

	if (DebugState == EPVImportTexture2DDebugState::ComputeBranchHierarchies)
	{
		FManagedArrayCollection& DebugCollection = Output.DebugCollection.Emplace();
		AddDynamicMeshToCollection(DynamicMesh, DebugCollection);
		AddBranchHierarchiesToCollection(BranchHierarchyDescriptions, DebugCollection);
		RootPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		EndPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		VertexScaleAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		LengthFromRootAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
		NextBranchPointAttributeDefinition.CopyAttributeToCollection(DynamicMesh, DebugCollection);
	}

	TArray<int32> BranchHierarchiesToPlantIndices;
	MapPlantIndicesToBranchHierarchies(PlantMapping, DynamicMeshBranchHierarchies, BranchHierarchiesToPlantIndices);

	Output.PlantCollections.SetNum(PlantSettings.Num());
	for (int32 i = 0; i < BranchHierarchyDescriptions.Num(); ++i)
	{
		const int32 PlantIndex = BranchHierarchiesToPlantIndices[i];
		if (PlantIndex == INDEX_NONE)
		{
			continue;
		}

		if (!PV::ImportHelper::GenerateGrowthDataFromBranchHierarchy(Output.PlantCollections[PlantIndex], BranchHierarchyDescriptions[i]))
		{
			return EImportResult::FailedToGenerateGrowthData;
		}
	}

	return EImportResult::Success;
}
