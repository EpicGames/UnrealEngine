// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Params/PVImportTexture2DParams.h"
#include "Polygon2.h"

class UMaterialInterface;
class UTexture2D;

namespace PV::Texture2DImport
{
	enum class ETracePerimeterCurvesResult
	{
		Success,
		InvalidSourceTexture,
		InvalidSampleSize,
		InvalidTextureSize
	};

	ETracePerimeterCurvesResult TracePerimeterCurves(
		TObjectPtr<UTexture2D> Texture2DAsset,
		int32 SampleResolution,
		bool bInvertImage,
		float WhiteLevel,
		EPCGTextureColorChannel ColorChannel,
		float MinBoundsArea,
		int32 SmoothingIterations,
		float SimplificationAmount,
		TArray<UE::Geometry::FPolygon2f>& OutPerimeterCurves
	);

	enum class EFindTipsResult
	{
		Success,
		NoTipsFound
	};

	EFindTipsResult FindTips(
		const TArray<UE::Geometry::FPolygon2f>& PerimeterCurves,
		float TipAngleThresholdInDegrees,
		float MaxTipAngleSearchDist,
		TArray<TArray<bool>>& OutTips
	);

	enum class EImportResult
	{
		Success,
		InvalidPerimeterCurves,
		InvalidTipsAttribute,
		FailedToGenerateGrowthData
	};

	EImportResult ImportGrowthDataFromPerimeterCurves(
		const TArray<UE::Geometry::FPolygon2f>& PerimeterCurves,
		const TArray<TArray<bool>>& Tips,
		const TArray<FPVImportTexture2DPlantSettings>& PlantSettings,
		EPVImportTexture2DDebugState DebugState,
		FPVImportTexture2DOutput& Output
	);
};
