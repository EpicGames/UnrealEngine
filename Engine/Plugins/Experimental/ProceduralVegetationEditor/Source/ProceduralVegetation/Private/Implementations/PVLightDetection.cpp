// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVLightDetection.h"

#include "MeshDescriptionAdapter.h"
#include "RenderingThread.h"
#include "Runtime/RHICore/Internal/RHICoreResourceCollection.h"
#include "PVLightDetectionCS.h"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"
#include "Nodes/PVObjectInteractionSettings.h"

IMPLEMENT_GLOBAL_SHADER(FPVLightDetectionCS, "/Plugins/Experimental/ProceduralVegetationEditor/PVLightDetectionCS.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FPVLightVectorCalculationCS, "/Plugins/Experimental/ProceduralVegetationEditor/PVLightVectorCalculationCS.usf", "MainCS", SF_Compute);

TArray<FVector3f> FPVLightDetection::GetApicalRayArray()
{
	static TArray<FVector3f> ApicalRays =
	{
		{-2.84143e-8f, -1.40031e-9f, 1.0f}, {0.234507f, 0.114193f, 0.982423f}, {-0.305524f, 0.0235983f, 0.981009f}, {0.00739088f, 0.424973f, 0.9323f}, {-0.317116f, -0.389863f, 0.902587f},
	   {0.149854f, -0.528508f, 0.90468f}, {0.606845f, -0.207917f, 0.826982f}, {-0.404097f, 0.623729f, 0.755625f}, {0.492474f, 0.354341f, 0.808317f}, {-0.639645f, 0.0519751f, 0.78914f},
	   {-0.247094f, -0.752495f, 0.684462f}, {0.652055f, 0.594714f, 0.566125f}, {0.351819f, 0.797601f, 0.573025f}, {0.00147185f, 0.785973f, 0.65508f}, {-0.74921f, 0.497099f, 0.557625f},
	   {-0.751355f, -0.365035f, 0.603212f}, {0.418768f, -0.713289f, 0.617841f}, {0.872889f, 0.252431f, 0.502171f}, {-0.966985f, 0.0907287f, 0.355526f}, {0.785918f, -0.456201f, 0.45183f},
	   {0.929777f, -0.0727426f, 0.430619f}, {-0.0424619f, 1.04075f, 0.173671f}, {-0.476119f, 0.894721f, 0.255286f}, {-0.879871f, -0.41078f, 0.308392f}, {-0.588844f, -0.814759f, 0.268149f},
	   {-0.287579f, -0.953623f, 0.272879f}, {0.122043f, -0.998882f, 0.265045f}, {0.605764f, -0.800053f, 0.239487f}, {0.746444f, 0.739372f, 0.107898f}, {0.959235f, -0.418556f, 0.048922f},
	   {1.00368f, 0.129782f, 0.0646386f}, {0.499379f, 0.930634f, -0.123444f}, {-0.861201f, 0.57548f, 0.0203667f}, {-1.02923f, -0.156867f, -0.00512988f}, {-0.821102f, -0.645414f, -0.0101089f},
	   {-1.00156f, 0.242109f, -0.203205f}, {0.898769f, 0.422095f, -0.253771f}, {0.125841f, 0.985105f, -0.286825f}, {-0.486041f, 0.889624f, -0.130751f}, {-0.397868f, -0.926287f, -0.270558f},
	   {-0.0784483f, -0.996366f, -0.318154f}, {0.306366f, -0.951752f, -0.254201f}, {0.67136f, -0.753999f, -0.231252f}, {1.00567f, 0.0428265f, -0.250912f}, {-0.932872f, -0.156542f, -0.435971f},
	   {-0.38234f, 0.865618f, -0.443661f}, {0.600538f, 0.66739f, -0.545646f}, {-0.720414f, 0.550412f, -0.481888f}, {-0.728051f, -0.599793f, -0.384338f}, {0.806914f, -0.378309f, -0.554144f},
	   {0.767476f, 0.163515f, -0.659601f}, {0.0763147f, 0.694541f, -0.801055f}, {-0.612633f, 0.339384f, -0.774136f}, {-0.589258f, -0.46293f, -0.71389f}, {-0.315536f, -0.675672f, -0.733484f},
	   {0.29348f, -0.697025f, -0.714709f}, {0.483263f, 0.290497f, -0.903892f}, {-0.228362f, 0.626725f, -0.766045f}, {-0.582414f, -0.0466f, -0.885318f}, {0.204627f, -0.49566f, -0.873935f},
	   {0.337001f, -0.219798f, -0.948781f}, {-0.19544f, -0.260404f, -0.979539f}, {-0.10889f, 0.227743f, -0.987461f}, {1.52086e-7f, 1.37666e-8f, -1.0f}
	};

	return ApicalRays;
}

TArray<FPVCollisionData> FPVLightDetection::BuildPVCollisionDataSkeleton(const UPVGrowerData* Skeleton)
{
	TArray<FPVCollisionData> CollisionData;

	for (auto Primitive : Skeleton->Primitives)
	{
		for (int i = 0 ; i < Primitive->BranchBuds.Num() -1; i++)
		{
			auto Point = Skeleton->GetPoint(Primitive->BranchBuds[i]);
			auto Point2 = Skeleton->GetPoint(Primitive->BranchBuds[i + 1]);

			FPVCollisionData Data;
			Data.Type = 1;
			Data.ApicalRay = 1;
			Data.PointNumber = Point->Bud.BudNumber;
			Data.BranchNumber = Primitive->BranchNumber;
			Data.Position = FVector3f(Point->Position) * 100;
			FVector3f Position2 = FVector3f(Point2->Position) * 100;
			Data.Direction = (Position2 - Data.Position).GetSafeNormal();
			Data.Extents = FVector3f(Point->Bud.LateralMeristem.LateralMeristem * 100, (Position2 - Data.Position).Length(), 0);
			CollisionData.Add(Data);
		}
	}

	return CollisionData;
}

TArray<FPVRaycastOrigin> FPVLightDetection::BuildPVRayOriginData(const UPVGrowerData* Skeleton)
{
	TArray<FPVRaycastOrigin> RayOriginData;
	//for (auto Point : Skeleton->Points)
	for (auto Primitive : Skeleton->Primitives)
	{
		for (int i = 0 ; i < Primitive->BranchBuds.Num(); i++)
		{
			auto Point = Skeleton->GetPoint(Primitive->BranchBuds[i]);

			FPVRaycastOrigin RaycastOrigin;
			RaycastOrigin.PointNumber = Point->Bud.BudNumber;
			RaycastOrigin.BranchNumber = Primitive->BranchNumber;
			RaycastOrigin.Position = FVector3f(Point->Position) * 100;
			RayOriginData.Add(RaycastOrigin);
		}
	}

	return RayOriginData;
}

void FPVLightDetection::BuildLeafMeshGeometry(const UStaticMesh* Mesh, FPVLeafMeshGeometry& OutGeometry)
{
	OutGeometry.Reset();
	if (!Mesh) return;
	if (!Mesh->GetRenderData() || !Mesh->GetRenderData()->LODResources.IsValidIndex(0)) return;

	const FStaticMeshLODResources& Lod = Mesh->GetRenderData()->LODResources[0];
	const int32 NumVerts   = Lod.GetNumVertices();
	const int32 NumIndices = Lod.IndexBuffer.GetNumIndices();

	OutGeometry.Vertices.Reserve(NumVerts);
	OutGeometry.Indices.Reserve(NumIndices);

	FVector3f ObjMin( TNumericLimits<float>::Max(),  TNumericLimits<float>::Max(),  TNumericLimits<float>::Max());
	FVector3f ObjMax(-TNumericLimits<float>::Max(), -TNumericLimits<float>::Max(), -TNumericLimits<float>::Max());

	for (int32 i = 0; i < NumVerts; ++i)
	{
		const FVector3f V = Lod.VertexBuffers.PositionVertexBuffer.VertexPosition(i);
		OutGeometry.Vertices.Add(V);
		ObjMin.X = FMath::Min(ObjMin.X, V.X); ObjMin.Y = FMath::Min(ObjMin.Y, V.Y); ObjMin.Z = FMath::Min(ObjMin.Z, V.Z);
		ObjMax.X = FMath::Max(ObjMax.X, V.X); ObjMax.Y = FMath::Max(ObjMax.Y, V.Y); ObjMax.Z = FMath::Max(ObjMax.Z, V.Z);
	}

	for (int32 i = 0; i < NumIndices; ++i)
	{
		OutGeometry.Indices.Add(Lod.IndexBuffer.GetIndex(i));
	}

	OutGeometry.ObjMin = ObjMin;
	OutGeometry.ObjMax = ObjMax;
}

void FPVLightDetection::BuildPVLeafInstanceData(const FPVLeafMeshGeometry& LeafGeometry, const TArray<FPVLeafTransform>& LeafTransforms, FPVColliderMeshData& OutMeshColliderData)
{
	if (!LeafGeometry.IsValid() || LeafTransforms.Num() == 0)
		return;

	FPVMeshGeometryRange LeafGeomRange;
	LeafGeomRange.StartVertex = OutMeshColliderData.Vertices.Num();
	LeafGeomRange.StartIndex  = OutMeshColliderData.Indices.Num();
	OutMeshColliderData.Vertices.Append(LeafGeometry.Vertices);
	OutMeshColliderData.Indices.Append(LeafGeometry.Indices);
	LeafGeomRange.EndVertex = OutMeshColliderData.Vertices.Num();
	LeafGeomRange.EndIndex  = OutMeshColliderData.Indices.Num();
	const int32 LeafGeomIndex = OutMeshColliderData.Geometries.Add(LeafGeomRange);

	const FVector3f ObjMin = LeafGeometry.ObjMin;
	const FVector3f ObjMax = LeafGeometry.ObjMax;

	const FVector3f ObjCorners[8] = {
		{ObjMin.X, ObjMin.Y, ObjMin.Z}, {ObjMax.X, ObjMin.Y, ObjMin.Z},
		{ObjMin.X, ObjMax.Y, ObjMin.Z}, {ObjMax.X, ObjMax.Y, ObjMin.Z},
		{ObjMin.X, ObjMin.Y, ObjMax.Z}, {ObjMax.X, ObjMin.Y, ObjMax.Z},
		{ObjMin.X, ObjMax.Y, ObjMax.Z}, {ObjMax.X, ObjMax.Y, ObjMax.Z},
	};

	for (const FPVLeafTransform& Leaf : LeafTransforms)
	{
		const FMatrix44f M(Leaf.Transform.ToMatrixWithScale());

		FVector3f WMin( TNumericLimits<float>::Max(),  TNumericLimits<float>::Max(),  TNumericLimits<float>::Max());
		FVector3f WMax(-TNumericLimits<float>::Max(), -TNumericLimits<float>::Max(), -TNumericLimits<float>::Max());
		for (const FVector3f& C : ObjCorners)
		{
			const FVector3f W = M.TransformPosition(C);
			WMin.X = FMath::Min(WMin.X, W.X); WMin.Y = FMath::Min(WMin.Y, W.Y); WMin.Z = FMath::Min(WMin.Z, W.Z);
			WMax.X = FMath::Max(WMax.X, W.X); WMax.Y = FMath::Max(WMax.Y, W.Y); WMax.Z = FMath::Max(WMax.Z, W.Z);
		}

		FPVLeafMeshInstanceData Instance;
		Instance.Transform     = M;
		Instance.AABBMin       = FVector4f(WMin.X, WMin.Y, WMin.Z, 0.f);
		Instance.AABBMax       = FVector4f(WMax.X, WMax.Y, WMax.Z, 0.f);
		Instance.GeometryIndex = LeafGeomIndex;
		Instance.BranchNumber  = Leaf.BranchNumber;
		Instance._Pad[0] = Instance._Pad[1] = 0;
		OutMeshColliderData.LeafInstances.Add(Instance);
	}
}

void FPVLightDetection::BuildPVCollisionData(const TArray<FPVColliderParams>& Colliders, FPVColliderMeshData& MeshColliderData)
{
	// Cache entry used to deduplicate geometry across instances of the same UStaticMesh
	struct FGeometryCacheEntry
	{
		int32     GeometryIndex;
		FVector3f ObjMin;
		FVector3f ObjMax;
	};
	TMap<const UStaticMesh*, FGeometryCacheEntry> GeometryCache;

	for (const FPVColliderParams& Collider : Colliders)
	{
		const UStaticMesh* ColliderMesh = Collider.Mesh.LoadSynchronous();
		if (!ColliderMesh) continue;
		if (!ColliderMesh->GetRenderData() || !ColliderMesh->GetRenderData()->LODResources.IsValidIndex(0)) continue;

		const FStaticMeshLODResources& Lod = ColliderMesh->GetRenderData()->LODResources[0];

		// Find or register shared geometry for this unique mesh
		FGeometryCacheEntry* CacheEntry = GeometryCache.Find(ColliderMesh);
		if (!CacheEntry)
		{
			const int32 NumVerts   = Lod.GetNumVertices();
			const int32 NumIndices = Lod.IndexBuffer.GetNumIndices();

			FPVMeshGeometryRange GeomRange;
			GeomRange.StartVertex = MeshColliderData.Vertices.Num();
			GeomRange.StartIndex  = MeshColliderData.Indices.Num();

			MeshColliderData.Vertices.Reserve(MeshColliderData.Vertices.Num() + NumVerts);
			MeshColliderData.Indices.Reserve(MeshColliderData.Indices.Num()   + NumIndices);

			FVector3f ObjMin( TNumericLimits<float>::Max(),  TNumericLimits<float>::Max(),  TNumericLimits<float>::Max());
			FVector3f ObjMax(-TNumericLimits<float>::Max(), -TNumericLimits<float>::Max(), -TNumericLimits<float>::Max());

			for (int32 i = 0; i < NumVerts; ++i)
			{
				const FVector3f V = Lod.VertexBuffers.PositionVertexBuffer.VertexPosition(i);
				MeshColliderData.Vertices.Add(V);
				ObjMin.X = FMath::Min(ObjMin.X, V.X);
				ObjMin.Y = FMath::Min(ObjMin.Y, V.Y);
				ObjMin.Z = FMath::Min(ObjMin.Z, V.Z);
				ObjMax.X = FMath::Max(ObjMax.X, V.X);
				ObjMax.Y = FMath::Max(ObjMax.Y, V.Y);
				ObjMax.Z = FMath::Max(ObjMax.Z, V.Z);
			}

			for (int32 i = 0; i < NumIndices; ++i)
			{
				MeshColliderData.Indices.Add(Lod.IndexBuffer.GetIndex(i));
			}

			GeomRange.EndVertex = MeshColliderData.Vertices.Num();
			GeomRange.EndIndex  = MeshColliderData.Indices.Num();

			FGeometryCacheEntry NewEntry;
			NewEntry.GeometryIndex = MeshColliderData.Geometries.Add(GeomRange);
			NewEntry.ObjMin        = ObjMin;
			NewEntry.ObjMax        = ObjMax;
			CacheEntry = &GeometryCache.Add(ColliderMesh, NewEntry);
		}

		// Build world-space AABB by transforming all 8 corners of the object-space AABB
		const FMatrix44f  M(Collider.Transform.ToMatrixWithScale());
		const FVector3f   ObjMin = CacheEntry->ObjMin;
		const FVector3f   ObjMax = CacheEntry->ObjMax;

		const FVector3f ObjCorners[8] = {
			{ObjMin.X, ObjMin.Y, ObjMin.Z}, {ObjMax.X, ObjMin.Y, ObjMin.Z},
			{ObjMin.X, ObjMax.Y, ObjMin.Z}, {ObjMax.X, ObjMax.Y, ObjMin.Z},
			{ObjMin.X, ObjMin.Y, ObjMax.Z}, {ObjMax.X, ObjMin.Y, ObjMax.Z},
			{ObjMin.X, ObjMax.Y, ObjMax.Z}, {ObjMax.X, ObjMax.Y, ObjMax.Z},
		};

		FVector3f WMin( TNumericLimits<float>::Max(),  TNumericLimits<float>::Max(),  TNumericLimits<float>::Max());
		FVector3f WMax(-TNumericLimits<float>::Max(), -TNumericLimits<float>::Max(), -TNumericLimits<float>::Max());

		for (const FVector3f& Corner : ObjCorners)
		{
			const FVector3f W = M.TransformPosition(Corner);
			WMin.X = FMath::Min(WMin.X, W.X);
			WMin.Y = FMath::Min(WMin.Y, W.Y);
			WMin.Z = FMath::Min(WMin.Z, W.Z);
			WMax.X = FMath::Max(WMax.X, W.X);
			WMax.Y = FMath::Max(WMax.Y, W.Y);
			WMax.Z = FMath::Max(WMax.Z, W.Z);
		}

		FPVMeshInstanceData Instance;
		Instance.Transform     = M;
		Instance.AABBMin       = FVector4f(WMin.X, WMin.Y, WMin.Z, 0.0f);
		Instance.AABBMax       = FVector4f(WMax.X, WMax.Y, WMax.Z, 0.0f);
		Instance.GeometryIndex = CacheEntry->GeometryIndex;
		Instance._Pad[0]       = 0;
		Instance._Pad[1]       = 0;
		Instance._Pad[2]       = 0;
		MeshColliderData.Instances.Add(Instance);
	}
}

TArray<FPVPointLightVectorData> FPVLightDetection::ExecuteLightDetection(TArray<FPVCollisionData> CollisionData, TArray<FPVRaycastOrigin> RaycastOrigins, const FPVColliderMeshData& MeshColliderData)
{
	if (RaycastOrigins.Num() <= 0)
	{
		return TArray<FPVPointLightVectorData>();
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(PV::LightDetection::ExecuteLightDetection);

	TArray<FVector3f> Rays = GetApicalRayArray();

	TArray<FPVLightDetectionData> LightDetectionData;
	LightDetectionData.Init({0,0, FVector3f(), 0, 0, 0},RaycastOrigins.Num() * 64);

	TArray<FPVPointLightVectorData> PointLightVectorData;
	PointLightVectorData.Init({0,0, FVector3f(), FVector3f()},RaycastOrigins.Num());

	FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

	ENQUEUE_RENDER_COMMAND(RunNeuralNetwork)
	([ &CollisionData, Signal, &LightDetectionData, &Rays , &RaycastOrigins, &PointLightVectorData, &MeshColliderData](FRHICommandListImmediate& RHICmdList){

		FRDGBuilder GraphBuilder(RHICmdList);

		FRDGBufferRef RDGCollisionDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("CollisionData"), sizeof(FPVCollisionData),
			CollisionData.Num(), CollisionData.GetData(), sizeof(FPVCollisionData)* CollisionData.Num());

		FRDGBufferRef RDGRaycastOriginsBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("PointData"), sizeof(FPVRaycastOrigin),
			RaycastOrigins.Num(), RaycastOrigins.GetData(), sizeof(FPVRaycastOrigin)* RaycastOrigins.Num());

		FRDGBufferRef RDGRaysBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("Rays"), sizeof(FVector3f),
			Rays.Num(), Rays.GetData(), sizeof(FVector3f)* Rays.Num());

		// Vertex and index buffers — provide a 1-element dummy when no external meshes are present
		static const FVector3f             DummyVertex(0.0f, 0.0f, 0.0f);
		static const uint32                DummyIndex        = 0u;
		static const FPVMeshGeometryRange  DummyGeomRange    = {};
		static const FPVMeshInstanceData   DummyInstance     = { FMatrix44f(ForceInit), FVector4f(ForceInit), FVector4f(ForceInit), 0, {0, 0, 0} };
		static const FPVLeafMeshInstanceData DummyLeafInstance = { FMatrix44f(ForceInit), FVector4f(ForceInit), FVector4f(ForceInit), 0, 0, {0, 0} };

		FRDGBufferRef RDGVerticesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("Vertices"), sizeof(FVector3f),
			FMath::Max(MeshColliderData.Vertices.Num(), 1),
			MeshColliderData.Vertices.Num() > 0 ? MeshColliderData.Vertices.GetData() : &DummyVertex,
			sizeof(FVector3f) * FMath::Max(MeshColliderData.Vertices.Num(), 1));

		FRDGBufferRef RDGIndicesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("Indices"), sizeof(uint32),
			FMath::Max(MeshColliderData.Indices.Num(), 1),
			MeshColliderData.Indices.Num() > 0 ? MeshColliderData.Indices.GetData() : &DummyIndex,
			sizeof(uint32) * FMath::Max(MeshColliderData.Indices.Num(), 1));

		FRDGBufferRef RDGGeometriesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("MeshGeometries"), sizeof(FPVMeshGeometryRange),
			FMath::Max(MeshColliderData.Geometries.Num(), 1),
			MeshColliderData.Geometries.Num() > 0 ? MeshColliderData.Geometries.GetData() : &DummyGeomRange,
			sizeof(FPVMeshGeometryRange) * FMath::Max(MeshColliderData.Geometries.Num(), 1));

		FRDGBufferRef RDGInstancesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("MeshInstances"), sizeof(FPVMeshInstanceData),
			FMath::Max(MeshColliderData.Instances.Num(), 1),
			MeshColliderData.Instances.Num() > 0 ? MeshColliderData.Instances.GetData() : &DummyInstance,
			sizeof(FPVMeshInstanceData) * FMath::Max(MeshColliderData.Instances.Num(), 1));

		FRDGBufferRef RDGLeafInstancesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("LeafInstances"), sizeof(FPVLeafMeshInstanceData),
			FMath::Max(MeshColliderData.LeafInstances.Num(), 1),
			MeshColliderData.LeafInstances.Num() > 0 ? MeshColliderData.LeafInstances.GetData() : &DummyLeafInstance,
			sizeof(FPVLeafMeshInstanceData) * FMath::Max(MeshColliderData.LeafInstances.Num(), 1));

		FRDGBufferRef RDGLightDetectionData = CreateStructuredBuffer(GraphBuilder, TEXT("LightDetectionData"), sizeof(FPVLightDetectionData),
			LightDetectionData.Num(), LightDetectionData.GetData(), sizeof(FPVLightDetectionData)* LightDetectionData.Num());

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(RDGLightDetectionData), 0u);

		constexpr int MaxBounce = 3;

		// Pass 1: fire 64 rays per skeleton point
		FPVLightDetectionCS::FParameters* Params = GraphBuilder.AllocParameters<FPVLightDetectionCS::FParameters>();
		Params->NumCollisionData     = CollisionData.Num();
		Params->MaxBounce            = MaxBounce;
		Params->NumRaycastOrigins    = RaycastOrigins.Num();
		Params->NumInstances         = MeshColliderData.Instances.Num();
		Params->NumGeometries        = MeshColliderData.Geometries.Num();
		Params->NumLeafInstances     = MeshColliderData.LeafInstances.Num();
		Params->CollisionData        = GraphBuilder.CreateSRV(RDGCollisionDataBuffer);
		Params->RaycastOrigins       = GraphBuilder.CreateSRV(RDGRaycastOriginsBuffer);
		Params->Rays                 = GraphBuilder.CreateSRV(RDGRaysBuffer);
		Params->ColliderMeshVertices = GraphBuilder.CreateSRV(RDGVerticesBuffer);
		Params->ColliderMeshIndices  = GraphBuilder.CreateSRV(RDGIndicesBuffer);
		Params->MeshGeometries       = GraphBuilder.CreateSRV(RDGGeometriesBuffer);
		Params->MeshInstances        = GraphBuilder.CreateSRV(RDGInstancesBuffer);
		Params->LeafInstances        = GraphBuilder.CreateSRV(RDGLeafInstancesBuffer);
		Params->LightDetectionData   = GraphBuilder.CreateUAV(RDGLightDetectionData);

		TShaderMapRef<FPVLightDetectionCS> LightDetectionCS(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		// 2D dispatch: one group of 64 threads per skeleton point, capped to avoid D3D12 X-dim limit
		const int32 NumPoints  = RaycastOrigins.Num();
		const int32 DispatchX  = FMath::Min(NumPoints, 65535);
		const int32 DispatchY  = FMath::DivideAndRoundUp(NumPoints, 65535);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("PVLightDetection"), LightDetectionCS, Params, FIntVector(DispatchX, DispatchY, 1));

		// Pass 2: collapse 64 ray results per point into a single light vector
		FRDGBufferRef RDGPointLightVectorData = CreateStructuredBuffer(GraphBuilder, TEXT("PointLightVectorData"), sizeof(FPVPointLightVectorData),
			PointLightVectorData.Num(), PointLightVectorData.GetData(), sizeof(FPVPointLightVectorData)* PointLightVectorData.Num());

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(RDGPointLightVectorData), 0u);

		FPVLightVectorCalculationCS::FParameters* Params2 = GraphBuilder.AllocParameters<FPVLightVectorCalculationCS::FParameters>();
		Params2->MaxBounce            = MaxBounce;
		Params2->NumRays              = Rays.Num();
		Params2->NumPoints            = NumPoints;
		Params2->LightDetectionData   = GraphBuilder.CreateSRV(RDGLightDetectionData);
		Params2->Rays                 = GraphBuilder.CreateSRV(RDGRaysBuffer);
		Params2->PointLightVectorData = GraphBuilder.CreateUAV(RDGPointLightVectorData);

		TShaderMapRef<FPVLightVectorCalculationCS> LightVectorCalculationCS(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		// Same 2D dispatch — one thread per skeleton point
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("PVLightVectorCalculation"), LightVectorCalculationCS, Params2, FIntVector(DispatchX, DispatchY, 1));

		FRHIGPUBufferReadback* Readback = new FRHIGPUBufferReadback(TEXT("ReadBackPointLightVectorData"));
		AddEnqueueCopyPass(GraphBuilder, Readback, RDGPointLightVectorData, sizeof(FPVPointLightVectorData)* PointLightVectorData.Num());

		GraphBuilder.Execute();
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		Readback->Wait(RHICmdList, RHICmdList.GetGPUMask());

		const FPVPointLightVectorData* Src = static_cast<FPVPointLightVectorData*>(Readback->Lock(sizeof(FPVPointLightVectorData) * PointLightVectorData.Num()));
		FMemory::Memcpy(PointLightVectorData.GetData(), Src, sizeof(FPVPointLightVectorData) * PointLightVectorData.Num());

		Readback->Unlock();

		delete Readback;

		Signal->Trigger();
	});

	Signal->Wait();
	FPlatformProcess::ReturnSynchEventToPool(Signal);

	//PrintLightDetectionData(LightDetectionData);
	//PrintPointLightVectorData(PointLightVectorData);

	return PointLightVectorData;
}

void FPVLightDetection::PrintLightDetectionData(TArray<FPVLightDetectionData> LightDetectionData)
{
	for (auto Data : LightDetectionData)
	{
		UE_LOGF(LogTemp, Log, "LightDetectionData PointNumber %i RayNumber %i Hits %i Direction %ls FaliureType %i Light Value %f", Data.PointNumber, Data.RayNumber, Data.Hits, *Data.Direction.ToString(), Data.FaliureType, Data.LightValue);
	}
}

void FPVLightDetection::PrintPointLightVectorData(TArray<FPVPointLightVectorData> PointLightVectorData)
{
	for (auto Data : PointLightVectorData)
	{
		UE_LOGF(LogTemp, Log, "PointLightVectorData PointNumber %i LightAvailable %f OptimalDirection %ls SubOptimalDirection %ls", Data.PointNumber, Data.LightAvailable, *Data.LightOptimalDirection.ToString(), *Data.LightSubOptimalDirection.ToString());
	}
}
