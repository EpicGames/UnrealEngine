// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "PVGrowerData.h"
#include "Nodes/PVObjectInteractionSettings.h"
#include "DataTypes/PVGrowerParams.h"
#include "PVLightDetection.generated.h"

UENUM(BlueprintType)
enum class EPVLightDetectionCollisionType : uint8
{
	Cylinder,
	TaperedCylinder,
	Card,
	Mesh
};

struct FPVRaycastOrigin
{
	int32 PointNumber;
	int32 BranchNumber;
	FVector3f Position;
};

struct FPVCollisionData
{
	int32 PointNumber;
	int32 BranchNumber;
	int32 ApicalRay;
	int32 Type;
	FVector3f Position;
	FVector3f Direction;
	FVector3f Extents;
};

struct FPVLightDetectionData
{
	int32 PointNumber;
	int32 RayNumber;
	FVector3f Direction;
	float LightValue;
	int32 Hits;
	int32 FaliureType;
};

struct FPVPointLightVectorData
{
	int32 PointNumber;
	float LightAvailable;
	FVector3f LightOptimalDirection;
	FVector3f LightSubOptimalDirection;
};

// GPU struct: geometry for one unique static mesh (object-space vertices/indices).
// Multiple instances can reference the same geometry via GeometryIndex.
// Layout must match PVMeshGeometryRange in PVLightDetectionCS.usf (16 bytes).
struct FPVMeshGeometryRange
{
	int32 StartVertex;
	int32 EndVertex;
	int32 StartIndex;
	int32 EndIndex;
};

// GPU struct: one placed instance of a geometry (external colliders — no branch association).
// Layout must match PVMeshInstanceData in PVLightDetectionCS.usf (112 bytes).
struct FPVMeshInstanceData
{
	FMatrix44f Transform;      // 64 bytes — object-to-world
	FVector4f  AABBMin;        // 16 bytes — world-space AABB (w unused)
	FVector4f  AABBMax;        // 16 bytes — world-space AABB (w unused)
	int32      GeometryIndex;  //  4 bytes — index into Geometries array
	int32      _Pad[3];        // 12 bytes — explicit padding to reach 112 bytes
};
static_assert(sizeof(FPVMeshInstanceData) == 112, "FPVMeshInstanceData size mismatch with HLSL struct");

// GPU struct: one placed leaf instance — like FPVMeshInstanceData but carries the branch it
// belongs to so the shader can skip leaves on the same branch as the ray-cast origin.
// Layout must match PVLeafMeshInstanceData in PVLightDetectionCS.usf (112 bytes).
struct FPVLeafMeshInstanceData
{
	FMatrix44f Transform;      // 64 bytes — object-to-world
	FVector4f  AABBMin;        // 16 bytes — world-space AABB (w unused)
	FVector4f  AABBMax;        // 16 bytes — world-space AABB (w unused)
	int32      GeometryIndex;  //  4 bytes — index into Geometries array
	int32      BranchNumber;   //  4 bytes — owning branch (for self-exclusion)
	int32      _Pad[2];        //  8 bytes — explicit padding to reach 112 bytes
};
static_assert(sizeof(FPVLeafMeshInstanceData) == 112, "FPVLeafMeshInstanceData size mismatch with HLSL struct");

struct FPVColliderMeshData
{
	TArray<FVector3f>              Vertices;      // object-space, shared across all instances
	TArray<uint32>                 Indices;       // shared across all instances
	TArray<FPVMeshGeometryRange>   Geometries;    // one entry per unique UStaticMesh
	TArray<FPVMeshInstanceData>    Instances;     // external collider instances (no branch exclusion)
	TArray<FPVLeafMeshInstanceData> LeafInstances; // leaf instances (self-branch exclusion applied in shader)
};

struct FPVLightDetection
{
	static TArray<FVector3f> GetApicalRayArray();

	static TArray<FPVCollisionData> BuildPVCollisionDataSkeleton(const UPVGrowerData* Skeleton);

	static TArray<FPVRaycastOrigin> BuildPVRayOriginData(const UPVGrowerData* Skeleton);

	static void BuildPVCollisionData(const TArray<FPVColliderParams>& Colliders, FPVColliderMeshData& MeshColliderData);

	/** Extract vertices, indices, and object-space AABB from a static mesh.
	 *  Call once when the mesh is selected; reuse OutGeometry every growth cycle. */
	static void BuildLeafMeshGeometry(const UStaticMesh* Mesh, FPVLeafMeshGeometry& OutGeometry);

	/** Append leaf geometry and per-instance data into OutMeshColliderData.
	 *  Each leaf instance carries its BranchNumber so the shader can skip leaves
	 *  on the same branch as the ray origin. No-op if LeafGeometry is invalid or
	 *  LeafTransforms is empty. */
	static void BuildPVLeafInstanceData(const FPVLeafMeshGeometry& LeafGeometry, const TArray<FPVLeafTransform>& LeafTransforms, FPVColliderMeshData& OutMeshColliderData);

	static TArray<FPVPointLightVectorData> ExecuteLightDetection(TArray<FPVCollisionData> CollisionData, TArray<FPVRaycastOrigin> RaycastOrigins, const FPVColliderMeshData& MeshColliderData);

	static void PrintLightDetectionData(TArray<FPVLightDetectionData> LightDetectionData);

	static void PrintPointLightVectorData(TArray<FPVPointLightVectorData> PointLightVectorData);
};
