// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

class FPVLightDetectionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPVLightDetectionCS);
	SHADER_USE_PARAMETER_STRUCT(FPVLightDetectionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER(uint32, NumCollisionData)
	SHADER_PARAMETER(uint32, MaxBounce)
	SHADER_PARAMETER(uint32, NumRaycastOrigins)
	SHADER_PARAMETER(uint32, NumInstances)
	SHADER_PARAMETER(uint32, NumGeometries)
	SHADER_PARAMETER(uint32, NumLeafInstances)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<PVCollisionCylinder>, CollisionData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector3f>, Rays)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<PVRaycastOrigin>, RaycastOrigins)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector3f>, ColliderMeshVertices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, ColliderMeshIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<PVMeshGeometryRange>, MeshGeometries)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<PVMeshInstanceData>, MeshInstances)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<PVLeafMeshInstanceData>, LeafInstances)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<PVLightDetectionData>, LightDetectionData)
	END_SHADER_PARAMETER_STRUCT()
};

class FPVLightVectorCalculationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPVLightVectorCalculationCS);
	SHADER_USE_PARAMETER_STRUCT(FPVLightVectorCalculationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER(uint32, MaxBounce)
	SHADER_PARAMETER(uint32, NumRays)
	SHADER_PARAMETER(uint32, NumPoints)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<PVLightDetectionData>, LightDetectionData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector3f>, Rays)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<PVPointLightVectorData>, PointLightVectorData)
	END_SHADER_PARAMETER_STRUCT()
};
