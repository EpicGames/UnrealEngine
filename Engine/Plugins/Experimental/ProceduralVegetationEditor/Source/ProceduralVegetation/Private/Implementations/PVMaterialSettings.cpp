// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMaterialSettings.h"

#include "ProceduralVegetationModule.h"
#include "Algo/MaxElement.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"
#include "Helpers/PVAttributesHelper.h"

struct FBranchUVData
{
	float XOffset;
	float YOffset;
	float YScale;
	int	MaterialId;
};

struct FPointUVData
{
	FVector2f BaseUVs;
};

void FPVMaterialSettings::ApplyMaterialSettings(FManagedArrayCollection& Collection) const
{
	TArray<int32> UsedMaterials;
	TArray<FBranchUVData> BranchUVData;
	SetBranchUVMaterial(Collection, UsedMaterials, BranchUVData);
	SetUVOffsetAndScale(Collection, BranchUVData);

	TArray<FPointUVData> PointUVData;
	SetBaseUVs(Collection, PointUVData);

	if (!PointUVData.IsEmpty())
	{
		FitUVs01(Collection, PointUVData, UsedMaterials, BranchUVData);
		SetOutputUVs(Collection, PointUVData, BranchUVData);
	}
}

void FPVMaterialSettings::SetMaterial(UMaterialInterface* Material, FVector2f UMinMax, int32 Index)
{
	FTrunkGenerationMaterialSetup MaterialSetup;
	MaterialSetup.Material = Material;
	MaterialSetup.URange = FFloatRange(UMinMax.X, UMinMax.Y);

	if (!MaterialSetups.IsValidIndex(Index))
	{
		MaterialSetups.Add(MaterialSetup);
	}
	else
	{
		MaterialSetups[Index].Material = MaterialSetup.Material;
		MaterialSetups[Index].URange = MaterialSetup.URange;
	}
}

void FPVMaterialSettings::SetMaterial(FString TrunkMaterial, FVector2f UMinMax, int32 Index)
{
	UMaterialInterface* Material = LoadObject<UMaterialInterface>(nullptr, TrunkMaterial, {}, LOAD_NoWarn | LOAD_Quiet);
	SetMaterial(Material, UMinMax, Index);
}

void FPVMaterialSettings::SetBranchUVMaterial(FManagedArrayCollection& Collection, TArray<int32>& UsedMaterials, TArray<FBranchUVData>& BranchUVData) const
{
	PV::Facades::FPointFacade PointFacade = PV::Facades::FPointFacade(Collection);
	PV::Facades::FBranchFacade BranchFacade = PV::Facades::FBranchFacade(Collection);

	const auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(Collection);
	const auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(Collection);
	const auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(Collection);
	const auto BudDevelopmentAttribute = PV::FBudDevelopmentAttribute::GetAttribute(Collection);

	BranchUVData.Init(FBranchUVData(), BranchFacade.GetElementCount());
	
	float MaxValue = 10000;
	float MinGeneration = MaxValue;
	float MinAge = MaxValue;
	float MinPScale = MaxValue;
	
	float MaxGeneration = 0;
	float MaxAge = 0;
	float MaxPScale = 0;

	for (int32 PointIndex = 0; PointIndex < PointFacade.GetElementCount(); PointIndex++)
	{
		TArray<int> BudDevelopment = PointFacade.GetBudDevelopment(PointIndex);
		const PV::FBudDevelopmentView BudDevelopmentView(BudDevelopment);
		
		float PScale = PointFacade.GetPointScale(PointIndex);

		MinAge = FMath::Min(MinAge, BudDevelopmentView.BranchAge);
		MinPScale = FMath::Min(MinPScale, PScale);

		MaxAge = FMath::Max(MaxAge, BudDevelopmentView.BranchAge);
		MaxPScale = FMath::Max(MaxPScale, PScale);
	}
	
	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++)
	{
		const int32 GenerationNumber = PV::Facades::FTreeFacade::GetBranchGenerationNumber(Collection, BranchIndex);
		MinGeneration = FMath::Min(MinGeneration, GenerationNumber);
		MaxGeneration = FMath::Max(MaxGeneration, GenerationNumber);
	}

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++)
	{
		float BranchMaterialVal = 0;

		const int32 GenerationNumber = PV::Facades::FTreeFacade::GetBranchGenerationNumber(Collection, BranchIndex);

		TArray<int32> BudDevelopment = PV::AttributesHelper::GetBudDevelopment(
			BudDevelopmentAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, 0);
		
		const PV::FBudDevelopmentView BudDevelopmentView(BudDevelopment);

		float PScale = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, 0);

		if (MaterialMode == EUVMaterialMode::Generation)
		{
			BranchMaterialVal = GetMaterialValue(GenerationNumber, MinGeneration, MaxGeneration, MinGeneration, MaxGeneration);
		}
		else if (MaterialMode == EUVMaterialMode::Age)
		{
			BranchMaterialVal = GetMaterialValue(BudDevelopmentView.BranchAge, MinAge, MaxAge, MaxGeneration, MinGeneration);
		}
		else if (MaterialMode == EUVMaterialMode::Radius)
		{
			BranchMaterialVal = GetMaterialValue(PScale, MinPScale, MaxPScale, MaxGeneration, MinGeneration);
		}

		const int MinValue = FMath::Min( MinGeneration + GenerationOffset , MaterialSetups.Num());
		
		if (DistributionMethod == EMaterialDistributionMethod::Repeat)
		{
			BranchMaterialVal = FMath::Clamp(BranchMaterialVal + GenerationOffset,MinValue,MaterialSetups.Num());
		}
		else if (DistributionMethod == EMaterialDistributionMethod::Fit)
		{
			FVector2f OldRange = FVector2f(MinGeneration, MaxGeneration);
			FVector2f NewRange = FVector2f(MinValue, MaterialSetups.Num());
			BranchMaterialVal = FMath::GetMappedRangeValueClamped(OldRange, NewRange, BranchMaterialVal);
		}

		int BranchUVMaterial = FMath::RoundToInt(BranchMaterialVal);
		//Set this as branch atrribute
		BranchFacade.SetBranchUVMaterial(BranchIndex, BranchUVMaterial - 1);
		UsedMaterials.AddUnique(BranchUVMaterial);
		BranchUVData[BranchIndex].MaterialId = BranchUVMaterial - 1;
	}
}

float FPVMaterialSettings::GetMaterialValue(float Value, float MinValue, float MaxValue, float MinGeneration, float MaxGeneration) const
{
	//float RampValue = 0;
	FVector2f OldRange = FVector2f(MinValue, MaxValue);
	FVector2f NewRange = FVector2f(0.0f, 1.0f);
	FVector2f GenerationRange = FVector2f(MinGeneration, MaxGeneration);

	float MaterialValue = FMath::GetMappedRangeValueClamped(OldRange, NewRange, Value);
    MaterialValue = FMath::GetMappedRangeValueClamped(NewRange, GenerationRange, MaterialValue);

	return MaterialValue;
}

void FPVMaterialSettings::SetUVOffsetAndScale(FManagedArrayCollection& Collection, TArray<FBranchUVData>& BranchUVData) const
{
	const PV::Facades::FBranchFacade& BranchFacade = PV::Facades::FBranchFacade(Collection);
	
	FRandomStream RandomStream(103);
				
	for (int BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++ )
	{
		int BranchUVMaterial = BranchUVData[BranchIndex].MaterialId;
			
		for (int i=0; i< MaterialSetups.Num(); i++)
		{
			float OffsetX = MaterialSetups[i].XOffset;
			float OffsetXRand = MaterialSetups[i].XOffsetRandom;
			float OffsetY = MaterialSetups[i].YOffset;
			float ScaleY = MaterialSetups[i].YScale;
			float OffsetYRand = MaterialSetups[i].YOffsetRandom;
	
			if (BranchUVMaterial == i)
			{
				float TextureOffsetX = OffsetX + (RandomStream.FRand() * OffsetXRand);
				float TextureOffsetY = OffsetY + (RandomStream.FRand() * OffsetYRand);
				FBranchUVData UVData{ TextureOffsetX, TextureOffsetY, ScaleY, BranchUVMaterial};
				check(BranchUVData.IsValidIndex(BranchIndex));
				BranchUVData[BranchIndex] = UVData;
			}
		}
		
		if (BranchUVMaterial > MaterialSetups.Num())
		{
			FBranchUVData UVData{ 0, 0, 1, BranchUVMaterial};
			check(BranchUVData.IsValidIndex(BranchIndex));
			BranchUVData[BranchIndex] = UVData;
		}
	}
}

void FPVMaterialSettings::SetBaseUVs(FManagedArrayCollection& Collection, TArray<FPointUVData>& PointUVData) const
{
	PV::Facades::FPointFacade PointFacade = PV::Facades::FPointFacade(Collection);
	PV::Facades::FBranchFacade BranchFacade = PV::Facades::FBranchFacade(Collection);

	const TManagedArray<float>& PointScales = PointFacade.GetPointScales();

	if(PointScales.Num() == 0)
	{
		UE_LOGF(LogProceduralVegetation, Warning, "PointScales Array is empty when setting Base UVs.");
		return;
	}
	
	const float MaxPointScale = *Algo::MaxElement(PointScales);

	if(MaxPointScale <= 0.0)
	{
		UE_LOGF(LogProceduralVegetation, Warning, "MaxPointScale is zero when setting Base UVs.");
		return;
	}

	PointUVData.Init(FPointUVData(), PointScales.Num());
	
	for (int BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++ )
	{
		auto& BranchPoints = BranchFacade.GetPoints(BranchIndex);
		TArray<float> SegmentArrayMetric = MakeSegmentArray(PointFacade,BranchPoints, MaxPointScale, 0);
		TArray<float> SegmentArrayBase = MakeSegmentArray(PointFacade,BranchPoints, MaxPointScale, 1);
    
		float UVYPreviousMetric = 0;
		float UVYPreviousBase = 0;

		for (int j=1; j < BranchPoints.Num(); j++)
		{
			int PTCurrent = BranchPoints[j];
			float UVYMetric = UVYPreviousMetric + SegmentArrayMetric[j-1];
        
			// Set the V of the UV using segment length
			FVector2f UVMetric = FVector2f(0.0f, UVYMetric);
			float UVYBase = UVYPreviousBase + SegmentArrayBase[j-1];
			FVector2f UVBase = FVector2f(0.0f, UVYBase);
			FPointUVData UVData = { UVBase };
			check(PointUVData.IsValidIndex(PTCurrent));
			PointUVData[PTCurrent] = UVData;
			UVYPreviousMetric = UVYMetric;
			UVYPreviousBase = UVYBase;
		}
	}
}

TArray<float> FPVMaterialSettings::MakeSegmentArray(const PV::Facades::FPointFacade& PointFacade,const TArray<int32>& Points,const float MaxPointScale, const int Mode) const
{
	TArray<float> SegmentArray;

	check(MaxPointScale > 0.0);
	float MaxPointScaleRatio = 1.0f /(MaxPointScale * UE_TWO_PI);

	for (int PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
	{
		float PointScale = PointFacade.GetPointScale(Points[PointIndex]);
		FVector3f Position = PointFacade.GetPosition(Points[PointIndex]);
		FVector3f NextPosition = PointFacade.GetPosition(Points[PointIndex + 1]);

		float Dist = FVector3f::Distance(Position,NextPosition);
		float PScaleRatio = 1.0;
		if (Mode == 1)
		{
			PScaleRatio = MaxPointScale / (PointScale == 0 ? UE_KINDA_SMALL_NUMBER : PointScale);
		}
		float SegmentLength = Dist * (MaxPointScaleRatio * PScaleRatio);
        
		SegmentArray.Add(SegmentLength);
	}
	
	return SegmentArray;
}

void FPVMaterialSettings::FitUVs01(FManagedArrayCollection& Collection, TArray<FPointUVData>& PointUVData, const TArray<int32>& UsedMaterials, TArray<FBranchUVData>& BranchUVData) const
{
	PV::Facades::FPointFacade PointFacade = PV::Facades::FPointFacade(Collection);
	PV::Facades::FBranchFacade BranchFacade = PV::Facades::FBranchFacade(Collection);
	
	//For every material
	for(int i = 0; i < UsedMaterials.Num(); i++)
	{
		//Get the UI y Texture Mode for Material
		// yTextureMode = 0 Default, 1 O-1 Fit, 2 0-1 Fit Adjusted
		for (int BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++ )
		{
			int BranchUVMaterial = BranchUVData[BranchIndex].MaterialId;
			bool bFit01 = MaterialSetups.IsValidIndex(i) && MaterialSetups[i].YTextureMode > EYTextureMode::Default;
			//Get all the branches with the material
			if (BranchUVMaterial == i && bFit01)
			{
				//zeroOneFitUVs(matPRs,minMax_array);
				auto& BranchPoints = BranchFacade.GetPoints(BranchIndex);
	
				//Get the V min Max for the Branch
				check(BranchPoints.IsValidIndex(0));
				check(PointUVData.IsValidIndex(BranchPoints[0]));
				FVector2f RootPTBaseUV = PointUVData[BranchPoints[0]].BaseUVs;//point(0,"uv_base",rootPT);
				check(PointUVData.IsValidIndex(BranchPoints[BranchPoints.Num() - 1]));
				FVector2f LastPTBaseUV = PointUVData[BranchPoints[BranchPoints.Num() - 1]].BaseUVs;//point(0,"uv_base",PRPTs[len(PRPTs)-1]);
				FVector2f BranchUVMinMax(RootPTBaseUV.Y, LastPTBaseUV.Y);
				for (int j=0; j < BranchPoints.Num(); j++)
				{
					FVector2f& UVBase = PointUVData[BranchPoints[j]].BaseUVs;//point(0,"uv_base",PRPT);
    
					//if point V >= Branch min V -> remap the V to 0 and 1
					if (UVBase.Y >= BranchUVMinMax.X)
					{
						UVBase.Y = FMath::GetMappedRangeValueClamped(BranchUVMinMax, FVector2f(0,1), UVBase.Y);
					}
					// if point V < Branch min V -> remap the V between 0 and  -1
					else
					{
						FVector2f OldRange(BranchUVMinMax.X, -BranchUVMinMax.Y);
						UVBase.Y = FMath::GetMappedRangeValueClamped(OldRange, FVector2f(0,-1), UVBase.Y);
					}
				}
			}
		}
	}
}

void FPVMaterialSettings::SetOutputUVs(FManagedArrayCollection& Collection, TArray<FPointUVData>& PointUVData, TArray<FBranchUVData>& BranchUVData) const
{
	PV::Facades::FBranchFacade BranchFacade = PV::Facades::FBranchFacade(Collection);
	PV::Facades::FPointFacade PointFacade = PV::Facades::FPointFacade(Collection);
	PV::Facades::FPlantFacade PlantFacade = PV::Facades::FPlantFacade(Collection);
	
	for (int BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++ )
	{
		int BranchUVMaterial = BranchUVData[BranchIndex].MaterialId;
		
		float UVRatio = 1.0;
		float UVYTextureOffset = 0;
		float UVXTextureOffset = 0;
		float UVYTextureScale = 1;
		FFloatRange URange(0, 1);
		//Get all the branches with the material
		if (MaterialSetups.IsValidIndex(BranchUVMaterial))
		{
			URange = MaterialSetups[BranchUVMaterial].URange;
			if (MaterialSetups[BranchUVMaterial].YTextureMode == EYTextureMode::Default)
			{
				UVYTextureOffset = BranchUVData[BranchIndex].YOffset;
				UVYTextureScale = BranchUVData[BranchIndex].YScale;
				UVRatio *= 1;
			}
		}

		UVXTextureOffset = BranchUVData[BranchIndex].XOffset;
		
		auto& BranchPoints = BranchFacade.GetPoints(BranchIndex);

		const bool IsTrunk = PlantFacade.IsTrunkIndex(BranchIndex);
		const int StartingIndex = IsTrunk ? 0 : 1;
		
		for (int j= StartingIndex; j < BranchPoints.Num(); j++)
		{
			FVector2f& UV = PointUVData[BranchPoints[j]].BaseUVs;
			UV.Y *= -1;
			UV.Y += UVYTextureOffset;
			UV.X = -UVXTextureOffset;
			UV.Y *= UVRatio;
			UV.Y *= UVYTextureScale;

			PointFacade.SetTextureCoordV(BranchPoints[j], UV.Y);
			PointFacade.SetTextureCoordUOffset(BranchPoints[j], UV.X);
			PointFacade.SetURange( BranchPoints[j], FVector2f(URange.GetLowerBound().GetValue(), URange.GetUpperBound().GetValue()));
		}
	}
}
