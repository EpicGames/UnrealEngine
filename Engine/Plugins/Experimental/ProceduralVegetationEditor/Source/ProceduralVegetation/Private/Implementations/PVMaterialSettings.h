// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Facades/PVPointFacade.h"
#include "Materials/MaterialInterface.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "PVMaterialSettings.generated.h"

UENUM()
enum class EGenerationOffsetMethod : uint8
{
	Clamped,
	Refit,
};

UENUM()
enum class EMaterialDistributionMethod : uint8
{
	Repeat,
	Fit,
};

UENUM()
enum class EYTextureMode : uint8
{
	Default,
	Fit0_1,
	//Fit0_1_Adjusted,
};

UENUM()
enum class EUVMaterialMode : uint8
{
	Generation,
	Age,
	Radius,
};

USTRUCT()
struct FTrunkGenerationMaterialSetup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Materials", meta=(DisplayName="Use Trunk Texture Setup", Tooltip="When enabled, the Material and X Range for this generation are sourced automatically from the connected Trunk Texture Setup pin, matched by generation index."))
	bool bDeriveFromTrunkTextureSetup = false;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(EditCondition="!bDeriveFromTrunkTextureSetup", EditConditionHides, Tooltip="Material to apply to this generation."))
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY(EditAnywhere, Category="Materials" , meta=(DisplayName = "Y Texture Mode", Tooltip="How textures align vertically (Default uses native tiling; Fit0_1 stretches once)."))
	EYTextureMode YTextureMode = EYTextureMode::Default;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(DisplayName = "Y Scale", EditCondition = "YTextureMode == EYTextureMode::Default", Tooltip="Texture tiling scale along Y.\n\nAdjusts vertical texture scale. Higher values increase tiling frequency (finer detail); lower values broaden patterns."))
	float YScale = 1.0f;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(DisplayName = "Y Offset", EditCondition = "YTextureMode == EYTextureMode::Default", Tooltip="Shifts texture position along Y.\n\nMoves the material vertically without changing scale. Useful for aligning repeats or hiding seams."))
	float YOffset = 0.0f;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(DisplayName = "Y Offset Random", EditCondition = "YTextureMode == EYTextureMode::Default", Tooltip="Randomizes Y offset per instance.\n\nAdds per-instance variation to the vertical texture offset to avoid visible repetitions across branches."))
	float YOffsetRandom = 0.0f;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(Tooltip="Shifts texture position along X.\n\nMoves the material horizontally across the mesh to fine-tune alignment or hide seams."))
	float XOffset = 0.0f;

	UPROPERTY(EditAnywhere, Category="Materials" , meta=(DisplayName = "X Offset Random", Tooltip="Randomizes X offset per instance.\n\nAdds per-instance variation to the horizontal texture offset to break pattern repetition."))
	float XOffsetRandom = 0.0f;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(DisplayName = "X Range",
		EditCondition="!bDeriveFromTrunkTextureSetup", EditConditionHides,
		Tooltip="Limits horizontal coverage range.\n\nSets the portion of the mesh affected along the X axis, useful for restricting materials (e.g., moss/lichen) to a subsection."))
	FFloatRange URange = FFloatRange(0,1);
	
};

struct FBranchUVData;
struct FPointUVData;

USTRUCT()
struct FPVMaterialSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Materials", meta=(ClampMin = "0", Tooltip="Shifts which branch generation the material targets.\n\nAdjusts the generation index used for material assignment. For example, an offset of 1 targets branches one level above the default (trunk → gen0, first branches → gen1, etc.)."))
	int32 GenerationOffset = 0;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(Tooltip="Repeat or Fit materials across generations.\n\nRepeat: cycle materials when generations exceed material count. Fit: stretch N materials across all generations."))
	EMaterialDistributionMethod DistributionMethod = EMaterialDistributionMethod::Repeat;

	UPROPERTY(EditAnywhere, Category="Materials" , DisplayName="Generation Materials", meta=(Tooltip="List of materials applied to the mesh.\n\nDefines the collection of materials that can be distributed by generation, age, or radius. Enables complex setups (e.g., bark, moss, twig materials by region)."))
	TArray<FTrunkGenerationMaterialSetup> MaterialSetups;

	UPROPERTY(EditAnywhere, Category="Materials", meta=(Tooltip="Driver for material selection: Generation, Age, or Radius.\n\nGeneration: by branch hierarchy. Age: by segment age. Radius: by branch thickness."))
	EUVMaterialMode MaterialMode = EUVMaterialMode::Generation;

	/*
	 Function takes the Collection as input and sets the branchUVMaterial for Branch,
	 calculates the V value of the UV according to the settings and write into Point Group
	 calculates the U offset and write to the Point Group in collection
	 */
	void ApplyMaterialSettings(FManagedArrayCollection& Collection) const;
	
	void SetMaterial(UMaterialInterface* Material, FVector2f UMinMax, int32 Index);

	void SetMaterial(FString TrunkMaterial, FVector2f UMinMax, int32 Index);

private:

	void SetBranchUVMaterial(FManagedArrayCollection& Collection, TArray<int32>& UsedMaterials, TArray<FBranchUVData>& BranchUVData) const;

	float GetMaterialValue(float Value, float MinValue, float MaxValue, float MinGeneration, float MaxGeneration) const;

	void SetUVOffsetAndScale(FManagedArrayCollection& Collection, TArray<FBranchUVData>& BranchUVData) const;
	
	void SetBaseUVs(FManagedArrayCollection& Collection, TArray<FPointUVData>& PointUVData) const;

	TArray<float> MakeSegmentArray(const PV::Facades::FPointFacade& PointFacade,const TArray<int32>& Points, const float MaxPointScale, const int Mode) const;

	void FitUVs01(FManagedArrayCollection& Collection, TArray<FPointUVData>& PointUVData, const TArray<int32>& UsedMaterials, TArray<FBranchUVData>& BranchUVData) const;

	void SetOutputUVs(FManagedArrayCollection& Collection, TArray<FPointUVData>& PointUVData, TArray<FBranchUVData>& BranchUVData) const;
};
