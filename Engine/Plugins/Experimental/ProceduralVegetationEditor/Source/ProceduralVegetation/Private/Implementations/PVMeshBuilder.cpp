// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilder.h"

#include <Facades/PVProfileFacade.h>

#include "ProceduralVegetationModule.h"
#include "Algo/MaxElement.h"
#include "Facades/PVBoneFacade.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"
#include "Helpers/PVAttributesHelper.h"
#include "Helpers/PVPlantTraversalHelper.h"

#include "Helpers/PVUtilities.h"

void MergeMaterials(const TArray<FTrunkGenerationMaterialSetup>& InMaterialSetups, TArray<FLocalDynamicMeshData>& MeshDatas)
{
	TMap<FString, int32> MaterialPathIDMap;
	for (auto& MeshData : MeshDatas)
	{
		TObjectPtr<UMaterialInterface> Material;
		if (InMaterialSetups.IsValidIndex(MeshData.MaterialID))
		{
			Material = InMaterialSetups[MeshData.MaterialID].Material;
		}

		FString MaterialPathName = "";
		if (Material)
		{
			MaterialPathName = Material->GetPathName();
		}

		if (!MaterialPathIDMap.Contains(MaterialPathName))
		{
			MaterialPathIDMap.Add(MaterialPathName , MeshData.MaterialID);
		}
		else
		{
			MeshData.MaterialID = MaterialPathIDMap[MaterialPathName];
		}
	}
}

void FPVMeshBuilder::GenerateGeometryCollection(const FManagedArrayCollection& InSkeletonCollection, const FManagedArrayCollection& InPlantProfileCollection, const FPVMeshBuilderParams& MeshBuilderParams,
                                                FGeometryCollection& OutGeometryCollection)
{
	InSkeletonCollection.CopyTo(&OutGeometryCollection);

	TArray<FLocalDynamicMeshData> MeshesDataCollection;

	PV::Facades::FBoneFacade::DefineSchema(OutGeometryCollection);
	PV::Facades::FBoneFacade BoneFacade = PV::Facades::FBoneFacade(OutGeometryCollection);

	PV::Facades::FBranchFacade BranchFacade(OutGeometryCollection);
	PV::Facades::FPointFacade PointFacade(OutGeometryCollection);

	if (BranchFacade.GetElementCount() == 0)
	{
		UE_LOGF(LogProceduralVegetation, Log, "No branch data available for generating mesh.");
		return;
	}

	BranchFacade.RecomputeBranchChildren();

	ApplyDaVinciRule(OutGeometryCollection, MeshBuilderParams);
	ApplyBranchGenerationRamps(OutGeometryCollection, MeshBuilderParams);
	ApplyMinRadius(OutGeometryCollection, MeshBuilderParams);
	ApplyBranchGenerationScales(OutGeometryCollection, MeshBuilderParams);

	AddPointsToSkeleton(OutGeometryCollection, MeshBuilderParams.MeshDetails.SkeletonResolution);
	ApplyNoiseToSkeleton(OutGeometryCollection, MeshBuilderParams);

	if (!PV::FPointNjordPixelIndexAttribute::HasAttribute(OutGeometryCollection))
	{
		PV::FPointNjordPixelIndexAttribute::AddAttribute(OutGeometryCollection);
	}
	
	PV::AttributesHelper::ComputeNjordPixelIndex(OutGeometryCollection);
	PV::AttributesHelper::ComputeLengthFromRoot(OutGeometryCollection);
	PV::AttributesHelper::ComputePointPlantGradient(OutGeometryCollection);
	PV::AttributesHelper::ComputePointGroundGradient(OutGeometryCollection);
	PV::AttributesHelper::ComputePointScaleGradient(OutGeometryCollection);
	PV::AttributesHelper::ComputePointHullGradient(OutGeometryCollection);
	PV::AttributesHelper::ComputePointMainTrunkGradient(OutGeometryCollection);

	MeshBuilderParams.MaterialDetails.ApplyMaterialSettings(OutGeometryCollection);
	GetLocalDynamicMeshData(OutGeometryCollection, InPlantProfileCollection, MeshBuilderParams, MeshesDataCollection);

	uint32 VertexCount = 0;
	uint32 TriangleCount = 0;
	for (auto [Vertices, Triangles, MaterialID] : MeshesDataCollection)
	{
		VertexCount += Vertices.Num();
		TriangleCount += Triangles.Num();
	}

	TManagedArray<int32>& PointIDs = BoneFacade.ModifyVertexPointIds();

	OutGeometryCollection.AddElements(1, OutGeometryCollection.TransformGroup);
	OutGeometryCollection.AddElements(1, OutGeometryCollection.GeometryGroup);
	OutGeometryCollection.AddElements(TriangleCount, OutGeometryCollection.FacesGroup);
	OutGeometryCollection.AddElements(VertexCount, OutGeometryCollection.VerticesGroup);
	OutGeometryCollection.SetNumUVLayers(3);

	TManagedArray<FVector3f>& Vertices = OutGeometryCollection.Vertex;
	TManagedArray<FVector3f>& Normals = OutGeometryCollection.Normal;
	TManagedArray<FVector2f>& UVs = *OutGeometryCollection.FindUVLayer(0);
	TManagedArray<FVector2f>& UVs1 = *OutGeometryCollection.FindUVLayer(1);
	TManagedArray<FVector2f>& UVs2 = *OutGeometryCollection.FindUVLayer(2);
	TManagedArray<FIntVector>& Indices = OutGeometryCollection.Indices;
	TManagedArray<int32>& MaterialIDAttribute = OutGeometryCollection.MaterialID;

	const FName MaterialPathAttributeName("MaterialPath");
	auto& MaterialArray = OutGeometryCollection.AddAttribute<FString>(MaterialPathAttributeName, FGeometryCollection::MaterialGroup);
	auto& MaterialSetups = MeshBuilderParams.MaterialDetails.MaterialSetups;

	struct FSection
	{
		int32 MaterialID;
		int32 FirstIndex;
		int32 NumTriangles;
		int32 MinVertexIndex;
		int32 MaxVertexIndex;
	};

	int32 VertexIndex = 0;
	int32 FaceIndex = 0;
	int32 PrevMaterialID = -1;
	TArray<FSection> Sections;

	//Merge MaterialIds based on material Paths
	MergeMaterials(MaterialSetups, MeshesDataCollection);

	//Sorting by material id so sections are created according to material ids
	MeshesDataCollection.Sort([](const FLocalDynamicMeshData& A, const FLocalDynamicMeshData& B)
		{
			return A.MaterialID > B.MaterialID;
		});

	int32 SectionId = -1;
	for (auto MeshIndex = MeshesDataCollection.Num() - 1; MeshIndex >= 0; --MeshIndex)
	{
		int MaterialID = MeshesDataCollection[MeshIndex].MaterialID;
		if (PrevMaterialID != MaterialID)
		{
			PrevMaterialID = MaterialID;

			TObjectPtr<UMaterialInterface> SectionMaterial;

			if (MaterialSetups.IsValidIndex(MaterialID))
			{
				SectionMaterial = MaterialSetups[MaterialID].Material;
			}
			else
			{
				UE_LOGF(LogProceduralVegetation, Warning, "Unassigned Material for section %i, MaterialID %i not found in MaterialSetups.",
					SectionId, MaterialID);
			}

			FString MaterialPathName = "";
			if (SectionMaterial)
			{
				MaterialPathName = SectionMaterial->GetPathName();
			}
			else
			{
				UE_LOGF(LogProceduralVegetation, Warning, "Null Material for section %i, Assign Material slot no %i in MaterialSetups.",
					SectionId, MaterialID);
			}

			int Element = OutGeometryCollection.AddElements(1, FGeometryCollection::MaterialGroup);
			MaterialArray[Element] = SectionMaterial.GetPathName();
			Sections.Add({MaterialID, FaceIndex, 0, VertexIndex, 0});
			SectionId++;
		}

		const FLocalDynamicMeshData& MeshesData = MeshesDataCollection[MeshIndex];
		for (const FIntVector4& Triangle : MeshesData.Triangles)
		{
			const FIntVector4 OffsetTriangle = Triangle + FIntVector4(VertexIndex);
			Indices[FaceIndex] = FIntVector(OffsetTriangle);
			MaterialIDAttribute[FaceIndex] = SectionId;
			++FaceIndex;
		}
		for (const FLocalDynamicMeshData::FVertex& Vertex : MeshesData.Vertices)
		{
			Vertices[VertexIndex] = static_cast<FVector3f>(Vertex.Position);
			Normals[VertexIndex] = static_cast<FVector3f>(Vertex.Normal);
			UVs[VertexIndex] = Vertex.UV;
			UVs1[VertexIndex] = Vertex.UV1;
			UVs2[VertexIndex] = Vertex.UV2;
			PointIDs[VertexIndex] = Vertex.PointIndex;
			++VertexIndex;
		}

		int32 CurrentSectionIndex = Sections.Num() - 1;
		if (Sections.IsValidIndex(CurrentSectionIndex))
		{
			Sections[CurrentSectionIndex].MaxVertexIndex = VertexIndex;
			Sections[CurrentSectionIndex].NumTriangles = FaceIndex - Sections[CurrentSectionIndex].FirstIndex;
		}
	}

	OutGeometryCollection.SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Rigid;
	OutGeometryCollection.TransformToGeometryIndex[0] = 0;
	OutGeometryCollection.TransformIndex[0] = 0;
	OutGeometryCollection.BoneName[0] = "TreeTest";
	OutGeometryCollection.BoneColor[0] = FLinearColor::White;
	OutGeometryCollection.FaceStart[0] = 0;
	OutGeometryCollection.FaceCount[0] = TriangleCount;
	OutGeometryCollection.VertexStart[0] = 0;
	OutGeometryCollection.VertexCount[0] = VertexCount;

	OutGeometryCollection.Visible.Fill(true);
	OutGeometryCollection.BoneMap.Fill(0);
	OutGeometryCollection.TangentU.Fill(FVector3f::ForwardVector);
	OutGeometryCollection.TangentV.Fill(FVector3f::RightVector);
	OutGeometryCollection.Color.Fill(FLinearColor::White);

	int32 SectionIndex = 0;
	for (auto Section : Sections)
	{
		TManagedArray<FGeometryCollectionSection>& GeometrySections = OutGeometryCollection.Sections;

		GeometrySections[SectionIndex].MaterialID = Section.MaterialID;
		GeometrySections[SectionIndex].FirstIndex = Section.FirstIndex;
		GeometrySections[SectionIndex].NumTriangles = Section.NumTriangles;
		GeometrySections[SectionIndex].MinVertexIndex = Section.MinVertexIndex;
		GeometrySections[SectionIndex].MaxVertexIndex = Section.MaxVertexIndex;

		SectionIndex++;
	}

	OutGeometryCollection.UpdateBoundingBox();
}

void FPVMeshBuilder::AddPointsToSkeleton(FManagedArrayCollection& OutCollection, int32 SkeletonResolution)
{
	PV::Facades::FBranchFacade BranchFacade(OutCollection);
	PV::Facades::FPointFacade PointFacade(OutCollection);
	
	const int32 NumOfBranches = BranchFacade.GetElementCount();
	const int32 NumOfPoints = PointFacade.GetElementCount();

	int32 LongestBranchIndex = -1;
	float LongestBranchLength = TNumericLimits<float>::Lowest();
	TMap<int32, TArray<int32>> BranchIndexToBranchPoints;
	BranchIndexToBranchPoints.Reserve(NumOfPoints);
	
	for (int32 BranchIndex = 0; BranchIndex < NumOfBranches; BranchIndex++)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
		TArray<int32> SortedPointIndices = BranchPoints;
		SortedPointIndices.Sort([&PointFacade](const int32& A, const int32& B)
			{
				return PointFacade.GetLengthFromRoot(A) < PointFacade.GetLengthFromRoot(B);
			});
		BranchIndexToBranchPoints.Add(BranchIndex, SortedPointIndices);
		
		if (BranchPoints.Num() < 2)
		[[unlikely]]
		{
			continue;
		}

		const float BranchLength = PointFacade.GetLengthFromRoot(SortedPointIndices.Last()) - PointFacade.GetLengthFromRoot(SortedPointIndices[0]);
		if (BranchLength > LongestBranchLength)
		{
			LongestBranchLength = BranchLength;
			LongestBranchIndex = BranchIndex;
		}
	}
	
	if (LongestBranchIndex == -1)
	{
		return;
	}
	
	const float DivisionUnitLength = LongestBranchLength / (10.0f * static_cast<float>(SkeletonResolution));
	
	for (int32 BranchIndex = 0; BranchIndex < NumOfBranches; BranchIndex++)
	{
		const TArray<int32>& BranchPoints = BranchIndexToBranchPoints[BranchIndex];
		TArray<int32> BranchPointsUpdated(BranchPoints);
		if (BranchPoints.Num() < 2)
		[[unlikely]]
		{
			continue;
		}
		
		for (int32 i = 0; i < BranchPoints.Num() - 1; i++)
		{
			const int32 CurrentPointIndex = BranchPoints[i];
			const int32 NextPointIndex = BranchPoints[i + 1];
			const int32 CurrentPointBudNumber = PointFacade.GetBudNumber(CurrentPointIndex);

			const float DistanceBetweenPoints = PointFacade.GetLengthFromRoot(NextPointIndex) - PointFacade.GetLengthFromRoot(CurrentPointIndex);
			const int32 NumOfSections = FMath::RoundToInt32(DistanceBetweenPoints / DivisionUnitLength);
		
			if (NumOfSections <= 1)
			[[unlikely]]
			{
				continue;
			}
		
			const float SectionLength = DistanceBetweenPoints / NumOfSections;
			const int32 NumberOfPointsToAdd = NumOfSections - 1;
		
			for (int32 j = 0; j < NumberOfPointsToAdd; j++)
			{
				const float Alpha = (SectionLength * (j + 1)) / DistanceBetweenPoints;
				const int32 NewPointIndex = PV::Utilities::AddInterpolatedPointToCollection(
					OutCollection,
					BranchIndex,
					CurrentPointIndex,
					NextPointIndex,
					Alpha,
					CurrentPointBudNumber);
				
				BranchPointsUpdated.Add(NewPointIndex);
			}
		}
		
		BranchPointsUpdated.Sort([&PointFacade](const int32& A, const int32& B)
			{
				return PointFacade.GetLengthFromRoot(A) < PointFacade.GetLengthFromRoot(B);
			});
		BranchFacade.SetPoints(BranchIndex, MoveTemp(BranchPointsUpdated));
	}
}

void FPVMeshBuilder::ApplyNoiseToSkeleton(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams)
{
	if (MeshBuilderParams.SkeletonShaping.Entries.IsEmpty())
	{
		return;
	}

	const PV::FBranchPointsAttributeConstView BranchPointsAttr = PV::FBranchPointsAttribute::FindAttribute(OutCollection);
	const PV::FBranchParentsAttributeConstView BranchParentsAttr = PV::FBranchParentsAttribute::FindAttribute(OutCollection);
	const PV::FBranchParentNumberAttributeView BranchParentNumberAttr = PV::FBranchParentNumberAttribute::FindAttribute(OutCollection);
	const PV::FBranchNumberAttributeConstView BranchNumbersAttr = PV::FBranchNumberAttribute::FindAttribute(OutCollection);
	const PV::FBranchHierarchyNumberAttributeConstView BranchHierarchyNumbersAttr = PV::FBranchHierarchyNumberAttribute::FindAttribute(OutCollection);
	PV::FPointPositionAttributeView PointPositionsAttr = PV::FPointPositionAttribute::FindAttribute(OutCollection);

	if (!BranchPointsAttr.IsValid() || !BranchParentsAttr.IsValid() || !BranchNumbersAttr.IsValid()
		|| !BranchHierarchyNumbersAttr.IsValid() || !PointPositionsAttr.IsValid() || !BranchParentNumberAttr.IsValid())
	{
		return;
	}

	const int32 NumBranches = BranchPointsAttr.Num();
	const int32 TotalPoints = PointPositionsAttr.Num();
	if (NumBranches == 0 || TotalPoints == 0)
	{
		return;
	}

	// Sort entries once by generation ascending so the per-branch lookup can use a
	// simple linear scan with early termination and always sees fallback candidates
	// in order from lowest to highest generation.
	TArray<FPVSkeletonShapingEntry> SortedEntries = MeshBuilderParams.SkeletonShaping.Entries;
	SortedEntries.Sort([](const FPVSkeletonShapingEntry& A, const FPVSkeletonShapingEntry& B)
	{
		return A.Generation < B.Generation;
	});

	// Returns the entry that governs a branch of the given hierarchy generation, or
	// nullptr when no entry covers it (no noise should be applied to that branch).
	//
	// Priority rules:
	//   1. An entry whose Generation == BranchGen is an exact match and wins outright.
	//   2. Otherwise, the highest-generation entry whose Generation < BranchGen AND
	//      bImpactRemainingGenerations == true acts as the fallback ("cascade down").
	//   A higher-generation exact match always beats a lower-generation cascade entry.
	auto FindEntryForGeneration = [&SortedEntries](const int32 BranchGen) -> const FPVSkeletonShapingEntry*
	{
		const FPVSkeletonShapingEntry* BestFallback = nullptr;
		for (const FPVSkeletonShapingEntry& Entry : SortedEntries)
		{
			if (Entry.Generation > BranchGen)
			{
				break; // Entries are sorted; nothing further can match.
			}
			if (Entry.Generation == BranchGen)
			{
				return &Entry; // Exact match — highest possible priority.
			}
			// Entry.Generation < BranchGen
			if (Entry.bImpactRemainingGenerations)
			{
				BestFallback = &Entry; // Overwrite each time; last one is highest gen.
			}
		}
		return BestFallback;
	};

	// ---- Pass 1: Compute the lateral noise displacement for every point ----
	//
	// Every point — including branch-first attachment points — gets a displacement
	// derived from its own world position so the field is spatially coherent.
	// Points whose branch has no applicable noise entry keep a zero displacement.
	// Storing everything in a flat array (indexed by point index) lets pass 2
	// replace attachment displacements in O(1).
	TArray<FVector3f> Displacements;
	Displacements.SetNumZeroed(TotalPoints);

	// Pre-compute the set of point indices that are fused attachment points: a point is
	// fused when it is BranchPoints[0] of a non-trunk branch AND the same index also
	// appears in the parent branch's BranchPoints array (i.e. both branches share one
	// physical point, not two coincident-but-distinct points as in the split topology).
	//
	// A fused point must be owned exclusively by the parent branch during noise
	// processing:
	//   Pass 1 — only the parent writes the displacement; the child skips index 0 to
	//             avoid overwriting the parent's value with a different-generation noise.
	//   Pass 3 — only the parent applies the displacement; the child skips index 0 to
	//             avoid displacing the shared point a second time.
	// In the split topology BranchPoints[0] of a child is a distinct index, so Pass 2's
	// nearest-parent sync is sufficient and no special skipping is needed.
	TSet<int32> FusedAttachmentPoints;
	FusedAttachmentPoints.Reserve(NumBranches);
	
	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		const int32 ParentBranchIndex = PV::PlantTraversalHelper::GetBranchParentIndex(BranchParentNumberAttr, BranchNumbersAttr, BranchIndex);
		if (ParentBranchIndex == INDEX_NONE)
		{
			continue;
		}
		const TArray<int32>& ChildBranchPoints = BranchPointsAttr[BranchIndex];
		if (ChildBranchPoints.IsEmpty())
		{
			continue;
		}
		if (BranchPointsAttr[ParentBranchIndex].Contains(ChildBranchPoints[0]))
		{
			FusedAttachmentPoints.Add(ChildBranchPoints[0]);
		}
	}

	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		// Resolve which noise entry governs this branch's hierarchy generation.
		// Branches with no applicable entry contribute zero displacement.
		const int32 BranchGen = BranchHierarchyNumbersAttr[BranchIndex];
		const FPVSkeletonShapingEntry* Entry = FindEntryForGeneration(BranchGen);
		if (!Entry)
		{
			continue;
		}

		const TArray<int32>& BranchPoints = BranchPointsAttr[BranchIndex];
		const int32 NumPoints = BranchPoints.Num();
		if (NumPoints < 2)
		{
			continue;
		}

		const float Strength  = Entry->NoiseStrength;
		const float Frequency = Entry->NoiseFrequency;
		const float SeedScale = static_cast<float>(Entry->NoiseSeed);
		const FVector SeedOffset(SeedScale * 127.1f, SeedScale * 311.7f, SeedScale * 74.7f);

		// Snapshot original positions so the tangent used for lateral projection
		// always comes from the unperturbed skeleton — this prevents compounding
		// errors on densely interpolated branches.
		TArray<FVector3f> OrigPos;
		OrigPos.Reserve(NumPoints);
		for (int32 i = 0; i < NumPoints; ++i)
		{
			OrigPos.Add(PointPositionsAttr[BranchPoints[i]]);
		}

		// For non-trunk branches, BranchPoints[0] is the attachment point.
		// • Fused topology: that index is shared with the parent branch; writing
		//   child-generation noise here would overwrite the correct parent value.
		// • Split topology: the displacement will be replaced by the parent's in Pass 2.
		// Starting from index 1 is safe and correct for both topologies; OrigPos[0] is
		// still captured above so that the tangent at i=1 can use central-differencing.
		const bool bHasParentBranch = (PV::PlantTraversalHelper::GetBranchParentIndex(BranchParentNumberAttr, BranchNumbersAttr, BranchIndex) != INDEX_NONE);
		const int32 NoiseStartIndex = bHasParentBranch ? 1 : 0;

		for (int32 i = NoiseStartIndex; i < NumPoints; ++i)
		{
			// Sample three independent Perlin channels at domain-offset coordinates
			// for a coherent, non-axis-aligned 3D displacement field.
			const FVector SamplePos = FVector(OrigPos[i]) * Frequency + SeedOffset;
			FVector3f NoiseVec(
				static_cast<float>(FMath::PerlinNoise3D(SamplePos)),
				static_cast<float>(FMath::PerlinNoise3D(FVector(SamplePos.Y + 31.41, SamplePos.Z + 159.27, SamplePos.X + 26.53))),
				static_cast<float>(FMath::PerlinNoise3D(FVector(SamplePos.Z + 271.83, SamplePos.X + 182.84, SamplePos.Y + 14.14)))
			);

			// Central-difference tangent (forward/backward at the ends).
			FVector3f Tangent;
			if (i > 0 && i < NumPoints - 1)
			{
				Tangent = (OrigPos[i + 1] - OrigPos[i - 1]).GetSafeNormal();
			}
			else if (i == NumPoints - 1)
			{
				Tangent = (OrigPos[i] - OrigPos[i - 1]).GetSafeNormal();
			}
			else
			{
				Tangent = (OrigPos[i + 1] - OrigPos[i]).GetSafeNormal();
			}

			// Project out the along-tangent component so displacement is purely
			// lateral — this prevents the tangent direction from flipping between
			// consecutive points, which is the primary cause of vertex-ring twisting.
			if (!Tangent.IsNearlyZero())
			{
				NoiseVec -= Tangent * FVector3f::DotProduct(NoiseVec, Tangent);
			}

			Displacements[BranchPoints[i]] = NoiseVec * Strength;
		}
	}

	// ---- Pass 2: Synchronise child attachment displacements with the parent ----
	//
	// The first point of a child branch is spatially coincident with a point on the
	// parent branch.  If their displacements differ, the child tube visually detaches
	// from the parent surface.  The fix: replace the child's first-point displacement
	// with the displacement of the nearest parent-branch point.
	//
	// Branches are visited in hierarchy order (root → tips) so that when a chain of
	// branches is present (grandparent → parent → child) the parent's first-point
	// displacement is already the corrected value by the time the child is visited.
	TArray<int32> HierarchySortedBranches;
	HierarchySortedBranches.Reserve(NumBranches);
	for (int32 Index = 0; Index < NumBranches; ++Index)
	{
		HierarchySortedBranches.Add(Index);
	}
	HierarchySortedBranches.Sort([&BranchParentsAttr](int32 Index1, int32 Index2)
	{
		const int32 Index1Hierarchy = BranchParentsAttr[Index1].Num();
		const int32 Index2Hierarchy = BranchParentsAttr[Index2].Num();
		if (Index1Hierarchy == Index2Hierarchy)
		{
			return Index1 < Index2;
		}
		return Index1Hierarchy < Index2Hierarchy;
	});

	for (const int32 BranchIndex : HierarchySortedBranches)
	{
		const int32 ParentBranchIndex = PV::PlantTraversalHelper::GetBranchParentIndex(BranchParentNumberAttr, BranchNumbersAttr, BranchIndex);
		if (ParentBranchIndex == INDEX_NONE)
		{
			continue; // Trunk / root — no parent to sync with.
		}

		const TArray<int32>& ChildPoints = BranchPointsAttr[BranchIndex];
		if (ChildPoints.Num() == 0)
		{
			continue;
		}

		const TArray<int32>& ParentPoints = BranchPointsAttr[ParentBranchIndex];
		if (ParentPoints.Num() == 0)
		{
			continue;
		}

		const int32 ChildFirstPoint = ChildPoints[0];
		const FVector3f ChildFirstPos = PointPositionsAttr[ChildFirstPoint];

		// Find the parent point whose original position is closest to the child
		// attachment so we match the displacement of the correct parent point.
		const int32* ClosestParentPoint = Algo::MinElement(ParentPoints,
			[&ChildFirstPos, &PointPositionsAttr](const int32 A, const int32 B)
			{
				return FVector3f::DistSquared(PointPositionsAttr[A], ChildFirstPos) <
				       FVector3f::DistSquared(PointPositionsAttr[B], ChildFirstPos);
			});

		if (ClosestParentPoint)
		{
			// Overwrite the child first-point displacement with the parent's so
			// both the parent tube and the child's root ring move identically.
			Displacements[ChildFirstPoint] = Displacements[*ClosestParentPoint];
		}
	}

	// ---- Pass 3: Apply displacements ----
	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttr[BranchIndex];
		const int32 NumPoints = BranchPoints.Num();
		if (NumPoints < 2)
		{
			continue;
		}

		// For fused-topology branches BranchPoints[0] is the same point index as the
		// parent's attachment point.  The parent branch's loop above has already applied
		// the displacement to it; applying it here a second time would double-displace
		// that shared point while all neighbouring points move only once.
		// For split-topology branches BranchPoints[0] is a unique index that is not
		// present in FusedAttachmentPoints, so it must be displaced here as normal.
		const bool bFirstPointFused = (PV::PlantTraversalHelper::GetBranchParentIndex(BranchParentNumberAttr, BranchNumbersAttr, BranchIndex) != INDEX_NONE)
		                           && FusedAttachmentPoints.Contains(BranchPoints[0]);
		const int32 ApplyStartIndex = bFirstPointFused ? 1 : 0;

		for (int32 i = ApplyStartIndex; i < NumPoints; ++i)
		{
			const int32 PointIndex = BranchPoints[i];
			PointPositionsAttr[PointIndex] = PointPositionsAttr[PointIndex] + Displacements[PointIndex];
		}
	}

	// ---- Pass 4: Smooth point positions along each branch ----
	//
	// Iterative Laplacian smoothing is applied to the final point positions so that
	// Smoothness produces a visible smooth curve regardless of whether NoiseStrength
	// is zero.  Operating on positions (rather than displacements) means the pass
	// succeeds even when the entire noise amplitude is 0.
	// The attachment point and tip are pinned so branches remain geometrically connected.
	PV::Facades::FBranchFacade BranchFacade(OutCollection);

	TArray<int32> BranchIndices;
	BranchFacade.GetSortedBranchIndicesByHierarchy(BranchIndices);
	for (int32 BranchIndex : BranchIndices)
	{
		const int32 BranchGen = BranchHierarchyNumbersAttr[BranchIndex];
		const FPVSkeletonShapingEntry* Entry = FindEntryForGeneration(BranchGen);
		if (!Entry || Entry->Smoothness <= 0.0f)
		{
			continue;
		}

		const TArray<int32>& BranchPoints = BranchPointsAttr[BranchIndex];
		const int32 NumPoints = BranchPoints.Num();

		// Need at least one interior point between the two pinned endpoints.
		if (NumPoints < 3)
		{
			continue;
		}

		// Map Smoothness [0,1] to iteration count [1,10].
		const int32 SmoothIterations = FMath::Max(1, FMath::RoundToInt(Entry->Smoothness * 10.0f));

		// Extract positions into a local double-buffer for in-place smoothing.
		TArray<FVector3f> BranchPositions;
		BranchPositions.SetNum(NumPoints);
		for (int32 i = 0; i < NumPoints; ++i)
		{
			BranchPositions[i] = PointPositionsAttr[BranchPoints[i]];
		}
		TArray<FVector3f> TempPositions = BranchPositions;

		for (int32 Iter = 0; Iter < SmoothIterations; ++Iter)
		{
			// Smooth interior points only; the attachment (1) and
			// tip (NumPoints-1) are never written so their values carry forward
			// unchanged through every Swap, keeping the branch connected.
			for (int32 i = 1; i < NumPoints - 1; ++i)
			{
				TempPositions[i] = (BranchPositions[i - 1] + BranchPositions[i] + BranchPositions[i + 1]) / 3.0f;
			}
			Swap(BranchPositions, TempPositions);
		}

		// Write smoothed positions back.
		for (int32 i = 1; i < NumPoints - 1; ++i)
		{
			PointPositionsAttr[BranchPoints[i]] = BranchPositions[i];
		}
	}
}

void FPVMeshBuilder::GenerateDynamicMesh(FManagedArrayCollection& Collection, const FManagedArrayCollection& InPlantProfileCollection, const FPVMeshBuilderParams& MeshBuilderParams,
                                         TObjectPtr<UDynamicMesh>& OutMesh)
{
	TArray<FLocalDynamicMeshData> MeshesDataCollection;
	GetLocalDynamicMeshData(Collection, InPlantProfileCollection, MeshBuilderParams, MeshesDataCollection);

	uint32 VertexCount = 0;
	uint32 TriangleCount = 0;
	for (auto [Vertices, Triangles , MaterialID] : MeshesDataCollection)
	{
		VertexCount += Vertices.Num();
		TriangleCount += Triangles.Num();
	}

	OutMesh->InitializeMesh();
	OutMesh->EditMesh([&](UE::Geometry::FDynamicMesh3& InternalMesh)
			{
				FPVMeshGenerator Generator(VertexCount, TriangleCount, &MeshesDataCollection);
				Generator.Generate();
				InternalMesh.Copy(&Generator);
			},
		EDynamicMeshChangeType::MeshChange);
}

TSet<int32> FPVMeshBuilder::CollectHardPoints(const PV::Facades::FBranchFacade& BranchFacade, const PV::Facades::FPointFacade& PointFacade,
                                              const PV::Facades::FPlantFacade& PlantFacade, PV::FPointNjordPixelIndexAttributeConstView NjordPixelIndexAttribute)
{
	TSet<int32> HardPoints;
	TSet<int32> BranchSourceBudNumbers;
	BranchSourceBudNumbers.Reserve(BranchFacade.GetElementCount());
	TSet<int32> BranchSourcePointIndices;
	BranchSourcePointIndices.Reserve(BranchFacade.GetElementCount());

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const int32 SourceBudNumber = BranchFacade.GetBranchSourceBudNumber(BranchIndex);
		BranchSourceBudNumbers.Add(SourceBudNumber);

		const int32 ParentBranchIndex = BranchFacade.GetParentBranchIndex(BranchIndex);

		if (const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
			ParentBranchIndex != INDEX_NONE
			&& BranchPoints.Num() > 0)
		{
			const FVector3f FirstPointPosition = PointFacade.GetPosition(BranchPoints[0]);
			const TArray<int32>& ParentBranchPoints = BranchFacade.GetPoints(ParentBranchIndex);
			const int32* Closest = Algo::MinElement(ParentBranchPoints,
				[FirstPointPosition, PointFacade](const int32& PointIndexA, const int32& PointIndexB)
					{
						return FVector3f::Dist(PointFacade.GetPosition(PointIndexA), FirstPointPosition) <
							FVector3f::Dist(PointFacade.GetPosition(PointIndexB), FirstPointPosition);
					});
			const int32 ClosestPointIndex = Closest
				? *Closest
				: INDEX_NONE;

			if (ClosestPointIndex != INDEX_NONE)
			{
				BranchSourcePointIndices.Add(ClosestPointIndex);
			}
		}
	}

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
		if (BranchPoints.Num() == 0)
		{
			continue;
		}

		const int32 LastPointIndex = BranchPoints.Last();
		const int32 FirstPointIndex = BranchPoints[0];

		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			const int32 PointIndex = BranchPoints[i];
			const int32 PointBudNumber = PointFacade.GetBudNumber(PointIndex);
			const float NjordPixelIndex = NjordPixelIndexAttribute[PointIndex];

			bool bIsSeedPoint = PlantFacade.IsTrunkIndex(BranchIndex) && PointIndex == FirstPointIndex;
			bool bIsBranchJoint = ((NjordPixelIndex == FMath::FloorToFloat(NjordPixelIndex))
					&& BranchSourceBudNumbers.Contains(PointBudNumber))
				|| BranchSourcePointIndices.Contains(PointIndex);

			if (PointIndex == FirstPointIndex ||
				PointIndex == LastPointIndex ||
				bIsSeedPoint ||
				bIsBranchJoint
			)
			{
				HardPoints.Add(PointIndex);
			}
		}
	}

	return HardPoints;
}

TSet<int32> FPVMeshBuilder::ComputePointGradients(PV::FPointHullGradientAttributeConstView HullGradientAttribute,
                                                  PV::FPointMainTrunkGradientAttributeConstView MainTrunkGradientAttribute,
                                                  PV::FPointGroundGradientAttributeConstView GroundGradientAttribute,
                                                  PV::FPointScaleGradientAttributeConstView ScaleGradientAttribute,
                                                  const FPVMeshBuilderParams& MeshBuilderParams, const TSet<int32>& HardPoints,
                                                  const float MaxPointScale, TArray<float>& OutMeshDivisionsGradients,
                                                  TArray<float>& OutDeltaModifiers)
{
	TSet<int32> PointsToRemove;
	OutMeshDivisionsGradients.Reserve(HullGradientAttribute.Num());
	OutDeltaModifiers.Reserve(HullGradientAttribute.Num());

	for (int32 PointIndex = 0; PointIndex < HullGradientAttribute.Num(); ++PointIndex)
	{
		const float HullGradient = HullGradientAttribute[PointIndex];
		const float MainTrunkGradient = MainTrunkGradientAttribute[PointIndex];
		const float GroundGradient = GroundGradientAttribute[PointIndex];
		const float ScaleGradient = ScaleGradientAttribute[PointIndex];

		// Compute scale removal gradient
		const float PointHullGradient =
			MeshBuilderParams.MeshDetails.HullRetentionGradient.GetRichCurveConst()->Eval(1.0f - HullGradient) * MeshBuilderParams.MeshDetails.HullRetention;

		const float PointMainTrunkGradient =
			MeshBuilderParams.MeshDetails.MainTrunkRetentionGradient.GetRichCurveConst()->Eval(1.0f - MainTrunkGradient) * MeshBuilderParams.MeshDetails.MainTrunkRetention;

		const float PointGroundGradient =
			MeshBuilderParams.MeshDetails.GroundRetentionGradient.GetRichCurveConst()->Eval(1.0f - GroundGradient) * MeshBuilderParams.MeshDetails.GroundRetention;

		const float ScaleRemovalGradient =
			(ScaleGradient + PointHullGradient + PointMainTrunkGradient + PointGroundGradient) * MaxPointScale / 100.0f;

		// Compute mesh divisions gradient
		const float DivisionsScaleGradient = MeshBuilderParams.MeshDetails.ScaleRetentionGradient.GetRichCurveConst()->Eval(1.0f - ScaleGradient);
		const float DivisionsHullGradient = MeshBuilderParams.MeshDetails.HullRetentionGradient.GetRichCurveConst()->Eval(1.0f - HullGradient);
		const float DivisionsMainTrunkGradient = MeshBuilderParams.MeshDetails.MainTrunkRetentionGradient.GetRichCurveConst()->Eval(1.0f - MainTrunkGradient);
		const float DivisionsGroundGradient = MeshBuilderParams.MeshDetails.GroundRetentionGradient.GetRichCurveConst()->Eval(1.0f - GroundGradient);

		const float InterpolatedHullGradient = FMath::GetMappedRangeValueClamped(
			FVector2f(0.0f, 1.0f),
			FVector2f(DivisionsHullGradient, 1.0f),
			1.0f - MeshBuilderParams.MeshDetails.HullRetention);
		const float InterpolatedMainTrunkGradient = FMath::GetMappedRangeValueClamped(
			FVector2f(0.0f, 1.0f),
			FVector2f(DivisionsMainTrunkGradient, 1.0f),
			1.0f - MeshBuilderParams.MeshDetails.MainTrunkRetention);
		const float InterpolatedGroundGradient = FMath::GetMappedRangeValueClamped(
			FVector2f(0.0f, 1.0f),
			FVector2f(DivisionsGroundGradient, 1.0f),
			1.0f - MeshBuilderParams.MeshDetails.GroundRetention);

		const float MeshDivisionsGradient = DivisionsScaleGradient * InterpolatedHullGradient * InterpolatedMainTrunkGradient *
			InterpolatedGroundGradient;
		OutMeshDivisionsGradients.Add(MeshDivisionsGradient);

		// Compute delta modifier gradient
		const float SegmentScaleGradient = MeshBuilderParams.MeshDetails.ScaleRetentionGradient.GetRichCurveConst()->Eval(1.0f - ScaleGradient);
		const float SegmentHullGradient = MeshBuilderParams.MeshDetails.HullRetentionGradient.GetRichCurveConst()->Eval(1.0f - HullGradient);
		const float SegmentMainTrunkGradient = MeshBuilderParams.MeshDetails.MainTrunkRetentionGradient.GetRichCurveConst()->Eval(1.0f - MainTrunkGradient);
		const float SegmentGroundGradient = MeshBuilderParams.MeshDetails.GroundRetentionGradient.GetRichCurveConst()->Eval(1.0f - GroundGradient);

		const float DeltaModifierGradient =
			(SegmentScaleGradient * MeshBuilderParams.MeshDetails.ScaleRetention) +
			(SegmentHullGradient * MeshBuilderParams.MeshDetails.HullRetention) +
			(SegmentMainTrunkGradient * MeshBuilderParams.MeshDetails.MainTrunkRetention) +
			(SegmentGroundGradient * MeshBuilderParams.MeshDetails.GroundRetention);

		// Should the point be removed per scale gradient
		if (!HardPoints.Contains(PointIndex) && MeshBuilderParams.MeshDetails.PointRemoval > 0.0f && ScaleRemovalGradient < MeshBuilderParams.MeshDetails.PointRemoval)
		{
			PointsToRemove.Add(PointIndex);
		}

		// Compute delta modifier
		float DeltaModifier = 1.0f;
		if (!HardPoints.Contains(PointIndex))
		{
			DeltaModifier = MeshBuilderParams.MeshDetails.SegmentRetentionImpact * DeltaModifierGradient;
		}
		OutDeltaModifiers.Add(DeltaModifier);
	}

	return PointsToRemove;
}

float FPVMeshBuilder::GetMaxDeltaBetweenHardPoints(const PV::Facades::FBranchFacade& BranchFacade, const PV::Facades::FPointFacade& PointFacade,
                                                   const TSet<int32>& HardPoints)
{
	float MaxDeltaBetweenHardPoints = 0.0f;

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);

		if (BranchPoints.Num() < 2)
		{
			continue;
		}

		TArray<int32> IndicesOfHardPointsForBranch;
		IndicesOfHardPointsForBranch.Add(0);

		for (int32 Idx = 1; Idx < BranchPoints.Num() - 1; ++Idx)
		{
			if (HardPoints.Contains(BranchPoints[Idx]))
			{
				IndicesOfHardPointsForBranch.Add(Idx);
			}
		}

		IndicesOfHardPointsForBranch.Add(BranchPoints.Num() - 1);

		for (int32 Idx = 0; Idx < IndicesOfHardPointsForBranch.Num() - 1; ++Idx)
		{
			const int32 Pt0Index = BranchPoints[IndicesOfHardPointsForBranch[Idx]];
			const int32 Pt2Index = BranchPoints[IndicesOfHardPointsForBranch[Idx + 1]];
			const FVector Pt0Position = FVector(PointFacade.GetPosition(Pt0Index));
			const FVector Pt2Position = FVector(PointFacade.GetPosition(Pt2Index));

			for (int32 k = IndicesOfHardPointsForBranch[Idx]; k < IndicesOfHardPointsForBranch[Idx + 1]; ++k)
			{
				const int32 Pt1Index = BranchPoints[k];
				const FVector Pt1Position = FVector(PointFacade.GetPosition(Pt1Index));

				if (const float Delta = FMath::PointDistToLine(Pt1Position, Pt2Position - Pt0Position, Pt0Position);
					Delta > MaxDeltaBetweenHardPoints)
				{
					MaxDeltaBetweenHardPoints = Delta;
				}
			}
		}
	}

	return MaxDeltaBetweenHardPoints;
}

void FPVMeshBuilder::PerformPathSimplification(const PV::Facades::FBranchFacade& BranchFacade,
                                               const PV::Facades::FPointFacade& PointFacade, const FPVMeshBuilderParams& MeshBuilderParams,
                                               const float MaxPointScale, const TSet<int32>& HardPoints, const TArray<float>& DeltaModifiers,
                                               TSet<int32>& InOutPointsToRemove)
{
	const int32 Iterations = MeshBuilderParams.MeshDetails.Accuracy;
	// Epsilon is the maximum allowed curvature error
	const float Epsilon = FMath::Pow(MeshBuilderParams.MeshDetails.SegmentReduction, 2) / static_cast<float>(Iterations);
	const float MaxDeltaBetweenHardPoints = GetMaxDeltaBetweenHardPoints(BranchFacade, PointFacade, HardPoints);

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		TArray<int32> BranchPoints = BranchFacade.GetPoints(BranchIndex).FilterByPredicate(
			[&InOutPointsToRemove](const int32 PointIndex)
				{
					return !InOutPointsToRemove.Contains(PointIndex);
				});

		for (int32 j = 0; j < Iterations; ++j)
		{
			const float IterationEpsilon = Epsilon * (j + 1);
			for (int32 i = 1; i < BranchPoints.Num() - 1; ++i)
			{
				const int32 Pt0Index = BranchPoints[i - 1],
				            Pt1Index = BranchPoints[i],
				            Pt2Index = BranchPoints[i + 1];
				const FVector Pt0Position = FVector(PointFacade.GetPosition(Pt0Index)),
				              Pt1Position = FVector(PointFacade.GetPosition(Pt1Index)),
				              Pt2Position = FVector(PointFacade.GetPosition(Pt2Index));
				const float DeltaModifier = DeltaModifiers[Pt1Index];
				const float Delta = FMath::PointDistToLine(Pt1Position, Pt2Position - Pt0Position, Pt0Position);

				const float NormalizedDelta = FMath::GetMappedRangeValueClamped(
					FVector2f(0.0f, MaxDeltaBetweenHardPoints),
					FVector2f(0.0f, 1.0f),
					Delta);

				if (NormalizedDelta + DeltaModifier < IterationEpsilon && !HardPoints.Contains(Pt1Index))
				{
					InOutPointsToRemove.Add(Pt1Index);
					BranchPoints.Remove(Pt1Index);
					i--;
				}
			}
		}

		// Ensure long chains of removed points do not exceed a maximum threshold
		BranchPoints = BranchFacade.GetPoints(BranchIndex);
		if (BranchPoints.Num() == 0)
		{
			return;
		}

		FVector3f PreviousPointPosition = PointFacade.GetPosition(BranchPoints[0]);
		float TravelDistance = 0.0f;

		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			const int32 CurrentPointIndex = BranchPoints[i];
			FVector3f CurrentPointPosition = PointFacade.GetPosition(CurrentPointIndex);
			const float Distance = FVector3f::Distance(PreviousPointPosition, CurrentPointPosition) / 100.0f;

			if (InOutPointsToRemove.Contains(CurrentPointIndex))
			{
				TravelDistance += Distance;
			}
			else
			{
				TravelDistance = 0.0f;
			}

			if (TravelDistance > MeshBuilderParams.MeshDetails.LongestSegmentLength)
			{
				InOutPointsToRemove.Remove(CurrentPointIndex);
				TravelDistance = 0.0f;
			}

			PreviousPointPosition = CurrentPointPosition;
		}

		// Ensure two kept points are never closer than a minimum threshold
		BranchPoints = BranchFacade.GetPoints(BranchIndex).FilterByPredicate(
			[&InOutPointsToRemove](const int32 PointIndex)
				{
					return !InOutPointsToRemove.Contains(PointIndex);
				});
		PreviousPointPosition = PointFacade.GetPosition(BranchPoints[0]);

		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			const int32 CurrentPointIndex = BranchPoints[i];
			FVector3f CurrentPointPosition = PointFacade.GetPosition(CurrentPointIndex);
			const float Distance = FVector3f::Distance(PreviousPointPosition, CurrentPointPosition) / 100.0f;

			if (Distance < MeshBuilderParams.MeshDetails.ShortestSegmentLength && !HardPoints.Contains(CurrentPointIndex))
			{
				InOutPointsToRemove.Add(CurrentPointIndex);
			}
			else
			{
				PreviousPointPosition = CurrentPointPosition;
			}
		}
	}
}

TArray<int32> FPVMeshBuilder::ComputeMeshDivisions(const PV::Facades::FBranchFacade& BranchFacade,
                                                   const FPVMeshBuilderParams& MeshBuilderParams, const TArray<float>& MeshDivisionsGradients,
                                                   const int32 PointCount)
{
	TArray<int32> TargetMeshDivisions;
	static constexpr int32 MinimumDivisions = 3;
	TargetMeshDivisions.Init(MinimumDivisions, PointCount);

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
		const float MinDivisions = static_cast<float>(MeshBuilderParams.MeshDetails.MinDivisions);
		const float MaxDivisions = static_cast<float>(MeshBuilderParams.MeshDetails.MaxDivisions);
		const FVector2f NormalRange(0.0f, 1.0f);
		const FVector2f DivisionsRange(MinDivisions, MaxDivisions);

		for (int32 i = 0; i < BranchPoints.Num(); ++i)
		{
			const int32 CurrentPointIndex = BranchPoints[i];
			const float MeshDivisionsGradient = MeshDivisionsGradients[CurrentPointIndex];

			const int32 TargetMeshDivisionsForCurrentPoint = FMath::RoundToInt32(
				FMath::GetMappedRangeValueClamped(
					NormalRange,
					DivisionsRange,
					MeshDivisionsGradient));

			TargetMeshDivisions[CurrentPointIndex] = TargetMeshDivisionsForCurrentPoint;
		}
	}

	return TargetMeshDivisions;
}

void FPVMeshBuilder::TriangulateRings(const TArray<int32>& PreviousIndices, const TArray<int32>& CurrentIndices, int32& InOutPolyGroupIndex,
                                      FLocalDynamicMeshData& OutMeshData)
{
	if (!PreviousIndices.IsEmpty())
	[[likely]]
	{
		const float IndexDelta1 = PreviousIndices.Num() < CurrentIndices.Num()
			? static_cast<float>(PreviousIndices.Num()) / CurrentIndices.Num()
			: 1.0f;
		const float IndexDelta2 = CurrentIndices.Num() < PreviousIndices.Num()
			? static_cast<float>(CurrentIndices.Num()) / PreviousIndices.Num()
			: 1.0f;

		for (
			float Index1 = 0.0f, Index2 = 0.0f;
			FMath::RoundToInt32(Index1 + IndexDelta1) <= PreviousIndices.Num() && FMath::RoundToInt32(Index2 + IndexDelta2) <=
			CurrentIndices.Num();
			Index1 += IndexDelta1, Index2 += IndexDelta2
		)
		{
			int32 VertexIndices[4] = {
				PreviousIndices[FMath::RoundToInt32(Index1) % PreviousIndices.Num()],
				CurrentIndices[FMath::RoundToInt32(Index2) % CurrentIndices.Num()],
				PreviousIndices[FMath::RoundToInt32(Index1 + IndexDelta1) % PreviousIndices.Num()],
				CurrentIndices[FMath::RoundToInt32(Index2 + IndexDelta2) % CurrentIndices.Num()],
			};

			if (VertexIndices[0] != VertexIndices[2])
			{
				OutMeshData.Triangles.Emplace(VertexIndices[0], VertexIndices[1], VertexIndices[2], InOutPolyGroupIndex);
			}
			if (VertexIndices[1] != VertexIndices[3])
			{
				OutMeshData.Triangles.Emplace(VertexIndices[1], VertexIndices[3], VertexIndices[2], InOutPolyGroupIndex);
			}

			++InOutPolyGroupIndex;
		}
	}
}

float FPVMeshBuilder::GetProfileMultiplier(const TArray<float>& InProfilePoints, const float ProfileUV_U)
{
	if (InProfilePoints.IsEmpty())
	{
		return 1.0f;
	}

	const int32 MaxPointsIndex = InProfilePoints.Num() - 1;
	const int32 ProfileIndex0 = FMath::Min(FMath::FloorToInt(ProfileUV_U * 100), MaxPointsIndex);
	const int32 ProfileIndex1 = FMath::Min(FMath::CeilToInt(ProfileUV_U * 100), MaxPointsIndex);

	const float Scale0 = InProfilePoints[ProfileIndex0];
	const float Scale1 = InProfilePoints[ProfileIndex1];

	const float BlendValue = FMath::GetMappedRangeValueClamped(
		FVector2f(static_cast<float>(ProfileIndex0), static_cast<float>(ProfileIndex1)),
		FVector2f(0.0f, 1.0f),
		ProfileUV_U * 100);

	const float ProfileMultiplier = FMath::Lerp(Scale0, Scale1, BlendValue);

	return ProfileMultiplier;
}

TMap<int32, TArray<int32>> FPVMeshBuilder::GetPointsIndicesToFoliageIndicesMap(const FManagedArrayCollection& Collection)
{
	TMap<int32, TArray<int32>> Map;
	const PV::Facades::FPointFacade PointFacade(Collection);
	const PV::Facades::FBranchFacade BranchFacade(Collection);
	const PV::Facades::FFoliageFacade FoliageFacade(Collection);

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const TArray<int32>& FoliageEntryIds = FoliageFacade.GetFoliageEntryIdsForBranch(BranchIndex);
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);

		if (BranchPoints.Num() == 0)
		{
			continue;
		}

		for (int32 FoliageEntryId : FoliageEntryIds)
		{
			// Find the closest point for current entry's pivot position
			const FVector3f PivotPosition = FoliageFacade.GetPivotPoint(FoliageEntryId);
			const int32* Closest = Algo::MinElement(BranchPoints, [PivotPosition, PointFacade](const int32& PointIndexA, const int32& PointIndexB)
				{
					return FVector3f::Dist(PointFacade.GetPosition(PointIndexA), PivotPosition) <
						FVector3f::Dist(PointFacade.GetPosition(PointIndexB), PivotPosition);
				});
			int32 ClosestPointIndex = Closest
				? *Closest
				: BranchPoints[0];

			Map.FindOrAdd(ClosestPointIndex).Add(FoliageEntryId);
		}
	}

	return Map;
}

FMeshPointUVData FPVMeshBuilder::ComputePointUVData(const int32 PointIndex, const int32 BranchPointIndex, const bool bPrimitiveIsTrunk,
                                               const TArray<int32>& PrimitivePoints, const PV::Facades::FPointFacade& PointFacade,
                                               const TManagedArray<FVector3f>& PointPositions, const float PointScale)
{
	// For non-trunk branches, the 0th branch ring is the fused point shared with the parent.
	// Its UV attributes belong to the parent, so read UV from BranchPoints[1] and extrapolate
	// backward by the B0→B1 segment length to recover the child's origin UV.
	const bool bRootBranchPoint = !bPrimitiveIsTrunk && BranchPointIndex == 0;

#if DO_ENSURE
	if (bRootBranchPoint)
	{
		ensureMsgf(PrimitivePoints.IsValidIndex(1),
			TEXT("Non-trunk branch at PrimitiveIndex 0 has fewer than 2 points"));
	}
#endif

	const int32 UVPointIndex = (bRootBranchPoint && PrimitivePoints.IsValidIndex(1)) ? PrimitivePoints[1] : PointIndex;

	float TextureCoordV = PointFacade.GetTextureCoordV(UVPointIndex);
	const float TextureCoordUOffset = PointFacade.GetTextureCoordUOffset(UVPointIndex);
	const FVector2f URange = PointFacade.GetURange(UVPointIndex);

	const float Interval = URange.GetMax() - URange.GetMin();
	const float URatio = FMath::IsNearlyZero(Interval) ? 1.0f : 1.0f / Interval;
	TextureCoordV /= URatio;

	if (bRootBranchPoint && PrimitivePoints.IsValidIndex(1))
	{
		// TextureCoordV decreases from root to tip (SetOutputUVs applies UV.Y *= -1), so ring 0
		// must have a higher value than ring 1 to sit "behind" it in texture space.
		// We extrapolate backward by ADDING the B0→B1 segment's UV contribution to ring 1's value.
		// TODO: We need to figure out why changing the root point flips all the branch UVs 
		const float DistToFirstChild = FVector3f::Dist(PointPositions[PointIndex], PointPositions[UVPointIndex]);
		const float SegLength = PointScale > 0.0f ? DistToFirstChild / (PointScale * UE_TWO_PI) : 0.0f;
		TextureCoordV += SegLength * Interval;
	}

	return { TextureCoordV, TextureCoordUOffset, URange };
}

void FPVMeshBuilder::GenerateBranchMeshData(const bool bPrimitiveIsTrunk, const int32 GenerationNumber, const int32 BranchIndex, const TArray<int32>& PrimitivePoints,
                                            const FDisplacementData& DisplacementData, const FPVMeshBuilderParams& MeshBuilderParams,
                                            const TArray<int32>& TargetMeshDivisions, const FManagedArrayCollection& Collection, const FManagedArrayCollection& InPlantProfileCollection,
                                            FLocalDynamicMeshData& OutLocalMeshData)
{
	if (PrimitivePoints.Num() < 2)
	{
		return;
	}
	
	const PV::Facades::FPointFacade PointFacade(Collection);
	const PV::Facades::FBudVectorsFacade BudVectorFacade(Collection);
	const PV::Facades::FPlantProfileFacade PlantProfileFacade(InPlantProfileCollection);

	bool bShouldApplyProfile =
		PlantProfileFacade.NumProfileEntries() > 0
		&& (MeshBuilderParams.ProfileDetails.bApplyToBranches || bPrimitiveIsTrunk);

	TArray<float> ProfilePoints;
	if (bShouldApplyProfile)
	{
		ProfilePoints = PlantProfileFacade.GetProfilePoints(0);
	}

	const auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(Collection);
	const auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(Collection);
	const auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(Collection);

	const float MaxPointScale = *Algo::MaxElement(PointScaleAttribute);
	const float MaxScaleRatio = 1.0f / (MaxPointScale * UE_TWO_PI);
	int32 PolyGroupIndex = 0;
	TArray<int32> CurrentIndices;
	TArray<int32> PreviousIndices;

	bool bShouldApplyDisplacement =
		DisplacementData.Values.Num() > 0
		&& DisplacementData.TextureWidth > 0
		&& DisplacementData.TextureHeight > 0
		&& GenerationNumber <= MeshBuilderParams.Displacement.GenerationUpperLimit;

	const PV::FBudDirectionAttributeConstView BudDirectionAttribute = PV::FBudDirectionAttribute::FindAttribute(Collection);
	const PV::FPointPositionAttributeConstView PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(Collection);

	const TArray<FVector3f> RootBudDirectionData = PV::AttributesHelper::GetBudDirection(
		BudDirectionAttribute, BranchPointsAttribute, BranchParentNumberAttribute, PointPositionAttribute, BranchIndex, 0);
	
	const PV::FBudDirectionConstView RootBudDirection(RootBudDirectionData);

	FVector3f PreviousUpVector = RootBudDirection.UpVector.GetUnsafeNormal();

	float Displacement_UV_V = 0.0f;
	FVector3f PreviousPosition = PointPositionAttribute[PrimitivePoints[0]];

	auto SafeDivide = [](const double InNumerator, const double InDenominator)
	{
		return FMath::IsNearlyZero(InDenominator) ? InNumerator : (InNumerator / InDenominator);
	};

	const float BaseLengthFromRoot = PointFacade.GetLengthFromRoot(PrimitivePoints[0]);
	const float TipLengthFromRoot = PointFacade.GetLengthFromRoot(PrimitivePoints.Last());

	FVector3f TangentVector = FVector3f::UpVector;
	for (int32 BranchPointIndex = 0; BranchPointIndex < PrimitivePoints.Num(); ++BranchPointIndex)
	{
		const int32& PointIndex = PrimitivePoints[BranchPointIndex];
		const float LengthFromRoot = PointFacade.GetLengthFromRoot(PointIndex);
		const FVector3f& PointPosition = PointPositionAttribute[PointIndex];

		const bool bIsFusedRootPoint = BranchPointIndex == 0 && !bPrimitiveIsTrunk;
		const float PointScale = bIsFusedRootPoint
			? PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, 0)
			: PointScaleAttribute[PointIndex];
		
		const auto [TextureCoordV, TextureCoordUOffset, NewRange] =
			ComputePointUVData(PointIndex, BranchPointIndex, bPrimitiveIsTrunk, PrimitivePoints, PointFacade, PointPositionAttribute.ManagedArray->GetConstArray(), PointScale);
		
		const FVector2f OldRange = FVector2f(0, 1.0f);
		
		Displacement_UV_V += FVector3f::Dist(PointPosition, PreviousPosition) * MaxScaleRatio * (MaxPointScale / PointScale);
		
		if (PrimitivePoints.IsValidIndex(BranchPointIndex - 1) && PrimitivePoints.IsValidIndex(BranchPointIndex + 1))
		[[likely]]
		{
			const FVector3f PreviousTangent = (PointPosition - PointPositionAttribute[PrimitivePoints[BranchPointIndex - 1]]).GetSafeNormal();
			const FVector3f NextTangent = (PointPositionAttribute[PrimitivePoints[BranchPointIndex + 1]] - PointPosition).GetSafeNormal();
			TangentVector = FVector3f::SlerpNormals(PreviousTangent, NextTangent, 0.5f);
		}
		else if (BranchPointIndex == PrimitivePoints.Num() - 1)
		[[unlikely]]
		{
			const int32 PreviousPointIndex = PrimitivePoints[BranchPointIndex - 1];
			const FVector3f& PreviousPointPosition = PointPositionAttribute[PreviousPointIndex];
			TangentVector = (PointPosition - PreviousPointPosition).GetSafeNormal();
		}
		else if (BranchPointIndex == 0)
		[[unlikely]]
		{
			const int32 NextPointIndex = PrimitivePoints[BranchPointIndex + 1];
			const FVector3f& NextPointPosition = PointPositionAttribute[NextPointIndex];
			TangentVector = (NextPointPosition - PointPosition).GetSafeNormal();
		}
		
		// Guard against near-zero tangent when consecutive branch points are coincident
		if (TangentVector.IsNearlyZero())
		{
			for (int32 SearchIndex = BranchPointIndex + 1; SearchIndex < PrimitivePoints.Num(); ++SearchIndex)
			{
				TangentVector = (PointPositionAttribute[PrimitivePoints[SearchIndex]] - PointPosition).GetSafeNormal();
				if (!TangentVector.IsNearlyZero())
				{
					break;
				}
			}
			if (TangentVector.IsNearlyZero())
			{
				const TArray<FVector3f> FallbackBudDirData = bIsFusedRootPoint
				? PV::AttributesHelper::GetBudDirection(BudDirectionAttribute, BranchPointsAttribute, BranchParentNumberAttribute, PointPositionAttribute, BranchIndex, 0)
				: BudDirectionAttribute[PointIndex].Array;
				TangentVector = PV::FBudDirectionConstView(FallbackBudDirData).Apical.GetSafeNormal();
			}
			if (TangentVector.IsNearlyZero())
			{
				TangentVector = FVector3f::UpVector;
			}
		}

		/*
		 * This creates a perpendicular Up Vector to the Current Tangent
		 * The current Tangent and Previous Up Vector is crossed to get a perpendicular vector to both
		 * Then that vector is crossed again with the Tangent Vector to get a third perpendicular vector which becomes the Up Vector
		 *       Tangent
		 *          |
		 *          | 90°
		 *      90° O------- Cross
		 *         / 90°
		 *        /
		 *       Up 
		 */
		FVector3f CrossVector = FVector3f::CrossProduct(TangentVector, PreviousUpVector);
		FVector3f UpVector = FVector3f::CrossProduct(CrossVector, TangentVector);

		if (UpVector.IsNearlyZero())
		{
			FVector Axis1, Axis2;
			FVector(TangentVector).FindBestAxisVectors(Axis1, Axis2);
			UpVector = FVector3f(Axis1);
		}
		
		PVE_DEBUG_LOCK()
			PVE_PARAM_DEBUG_DIRECTION_PARAM("UpVector", PointPosition, UpVector);
			PVE_PARAM_DEBUG_VECTOR_PARAM("TangentVector", PointPosition, PointPosition + TangentVector);
			PVE_PARAM_DEBUG_TEXT_PARAM("PointScale", PointPosition, FText::FromString(FString::Printf(TEXT("%f"), PointScale)));
		PVE_DEBUG_UNLOCK()

		PreviousUpVector = UpVector;

		FVector2f UV1 = FVector2f(0, 1.0 - Displacement_UV_V);
		const FVector2f UV2 = FVector2f(LengthFromRoot * 0.01, SafeDivide(PointScale, MaxPointScale));
		const float UVOGenerationOffset = GenerationNumber - 1;

		const float BranchGradient = FMath::GetMappedRangeValueClamped(
			FVector2f(BaseLengthFromRoot, TipLengthFromRoot),
			FVector2f(0.0f, 1.0f),
			LengthFromRoot);

		const float ProfileFallOff = MeshBuilderParams.ProfileDetails.FallOff.GetRichCurveConst()->Eval(BranchGradient);
		if (PointScale == 0.0f)
		[[unlikely]]
		{
			CurrentIndices.Add(OutLocalMeshData.Vertices.Emplace(PointPosition, TangentVector, FVector2f(UVOGenerationOffset, TextureCoordV), UV1, UV2, PointIndex));
		}
		else
		[[likely]]
		{
			const int32 PointDivisions = TargetMeshDivisions[PointIndex];
			const float RotationDelta = UE_TWO_PI / PointDivisions;
			for (int32 Division = 0; Division < PointDivisions; ++Division)
			{
				float UV_U = (1 - (static_cast<float>(Division) / PointDivisions)) + TextureCoordUOffset;
				UV_U = FMath::GetMappedRangeValueUnclamped(OldRange, NewRange, FMath::Wrap(UV_U, OldRange.X, OldRange.Y));
				UV_U += UVOGenerationOffset;

				float Displacement_UV_U = (static_cast<float>(Division) / PointDivisions);
				UV1.X = 1 - Displacement_UV_U;
				
				float ProfileMultiplier = 1.0f;
				if (bShouldApplyProfile)
				{
					ProfileMultiplier = GetProfileMultiplier(ProfilePoints, Displacement_UV_U);
					ProfileMultiplier = FMath::Lerp(1.0f, ProfileMultiplier * MeshBuilderParams.ProfileDetails.Scale, ProfileFallOff);
				}

				const FVector3f Direction = FQuat4f(TangentVector, RotationDelta * Division).RotateVector(UpVector).GetSafeNormal();
				FVector3f RadialPoint = PointPosition + (Direction * PointScale * ProfileMultiplier);

				if (bShouldApplyDisplacement)
				{
					const int32 X = FMath::Clamp(
						FMath::FloorToInt(
							Displacement_UV_U * MeshBuilderParams.Displacement.UVScale.X * (DisplacementData.TextureWidth - 1)
						) % DisplacementData.TextureWidth,
						0, DisplacementData.TextureWidth - 1
					);
					const int32 Y = FMath::Clamp(
						FMath::FloorToInt(
							Displacement_UV_V * MeshBuilderParams.Displacement.UVScale.Y * (DisplacementData.TextureHeight - 1)
						) % DisplacementData.TextureHeight,
						0, DisplacementData.TextureHeight - 1
					);

					const int32 Idx = Y * DisplacementData.TextureWidth + X;
					float DisplacementMultiplier =
						MeshBuilderParams.Displacement.Strength * (DisplacementData.Values[Idx] - MeshBuilderParams.Displacement.Bias);
					DisplacementMultiplier = DisplacementMultiplier * PointScale / MaxPointScale;

					RadialPoint = RadialPoint + (Direction * DisplacementMultiplier);
				}

				const FVector3f Normal = FVector3f((RadialPoint - PointPosition).GetUnsafeNormal());

				CurrentIndices.Add(OutLocalMeshData.Vertices.Emplace(RadialPoint, Normal, FVector2f(UV_U, TextureCoordV), UV1, UV2, PointIndex));
			}
		}

		FVector2f UV0 = FVector2f(FMath::GetMappedRangeValueUnclamped(OldRange, NewRange, FMath::Wrap(TextureCoordUOffset, OldRange.X, OldRange.Y)),
				TextureCoordV);
		UV0.X += UVOGenerationOffset;

		UV1.X = 0;
		
		const FLocalDynamicMeshData::FVertex FirstVertex = OutLocalMeshData.Vertices[CurrentIndices[0]];
		CurrentIndices.Add(OutLocalMeshData.Vertices.Emplace(FirstVertex.Position, FirstVertex.Normal,UV0, UV1, UV2 , PointIndex));

		PreviousPosition = PointPosition;
		
		TriangulateRings(PreviousIndices, CurrentIndices, PolyGroupIndex, OutLocalMeshData);

		PreviousIndices = MoveTemp(CurrentIndices);
		CurrentIndices.Empty();
	}
	
	// Add End-cap
	const int32 NumOfLastRingVertices = PreviousIndices.Num();
	if (NumOfLastRingVertices >= 3 && MeshBuilderParams.MeshDetails.AddEndCaps)
	{
		const int32 LastPointIndex = PrimitivePoints.Last();
		float CurrentLengthFromRoot = PointFacade.GetLengthFromRoot(LastPointIndex);
		float CurrentScale = PointScaleAttribute[LastPointIndex];
		const int32 CapDivisions = TargetMeshDivisions[LastPointIndex];
		float StepSize = CurrentScale * 0.5f;
		
		float CurrentRawTextureV = PointFacade.GetTextureCoordV(LastPointIndex);
		float LastTextureCoordUOffset = PointFacade.GetTextureCoordUOffset(LastPointIndex);
		FVector2f LastURange = PointFacade.GetURange(LastPointIndex);
		FVector2f OldRange = FVector2f(0, 1.0f);
		
		// URatio matches the main branch's computation
		float Interval = LastURange.Y - LastURange.X;
		float URatio = FMath::IsNearlyZero(Interval) ? 1.0f : 1.0f / Interval;
		
		const float UVOGenerationOffset = GenerationNumber - 1;
		
		for (int32 CapStep = 1; CapStep <= 3; ++CapStep)
		{
			FVector3f CurrentPosition = PreviousPosition + (TangentVector * StepSize);
			CurrentLengthFromRoot += StepSize;
			
			// Keeping displacement V and texture V aligned
			const float TextureScaleV = FMath::Max(CurrentScale, UE_KINDA_SMALL_NUMBER);
			const float CapStrideV = StepSize * MaxScaleRatio * (MaxPointScale / TextureScaleV);
			Displacement_UV_V += CapStrideV;
			CurrentRawTextureV += CapStrideV;
			float FinalTextureV = CurrentRawTextureV / URatio;
			
			StepSize *= 0.25f;
			
			if (CapStep == 3)
			{
				CurrentScale = 0.0f;
			}
			else
			{
				CurrentScale *= 0.5f; 
			}

			FVector2f UV1 = FVector2f(0, 1.0 - Displacement_UV_V);
			FVector2f UV2 = FVector2f(CurrentLengthFromRoot * 0.01, SafeDivide(CurrentScale, MaxPointScale));
			
			if (CurrentScale <= UE_KINDA_SMALL_NUMBER)
			[[unlikely]]
			{
				// A unique vertex for each tri at the tip.
				check(NumOfLastRingVertices >= 2);
				const float U_DivisionDenominator = static_cast<float>(NumOfLastRingVertices - 1);
				for (int32 Division = 0; Division < NumOfLastRingVertices; ++Division)
				{
					const float U_Division = static_cast<float>(Division) / U_DivisionDenominator;
					float UV_U = (1 - U_Division) + LastTextureCoordUOffset;
					UV_U = FMath::GetMappedRangeValueUnclamped(
							OldRange,
							LastURange,
							FMath::Wrap(UV_U, OldRange.X, OldRange.Y))
						+ UVOGenerationOffset;
					
					UV1.X = 1.0f - U_Division;
				
					CurrentIndices.Add(OutLocalMeshData.Vertices.Emplace(
						CurrentPosition,
						TangentVector,
						FVector2f(UV_U, FinalTextureV),
						UV1,
						UV2,
						static_cast<float>(LastPointIndex)
					));
				}
			}
			else
			[[likely]]
			{
				const float RotationDelta = UE_TWO_PI / CapDivisions;
				
				for (int32 Division = 0; Division < CapDivisions; ++Division)
				{
					float UV_U = (1 - (static_cast<float>(Division) / CapDivisions)) + LastTextureCoordUOffset;
					UV_U = FMath::GetMappedRangeValueUnclamped(
						OldRange,
						LastURange,
						FMath::Wrap(UV_U, OldRange.X, OldRange.Y));
					UV_U += UVOGenerationOffset;
					
					float Displacement_UV_U = (static_cast<float>(Division) / CapDivisions);
					UV1.X = 1 - Displacement_UV_U;
					
					const FVector3f Direction = FQuat4f(TangentVector, RotationDelta * Division).RotateVector(PreviousUpVector).GetSafeNormal();
					
					FVector3f RadialPoint = CurrentPosition + (Direction * CurrentScale);
					const FVector3f Normal = FVector3f((RadialPoint - CurrentPosition).GetUnsafeNormal());

					CurrentIndices.Add(OutLocalMeshData.Vertices.Emplace(
						RadialPoint, 
						Normal, 
						FVector2f(UV_U, FinalTextureV), 
						UV1,
						UV2,
						static_cast<float>(LastPointIndex)
					));
				}
				
				const auto FirstVertex = OutLocalMeshData.Vertices[CurrentIndices[0]];
				float UV_U = FMath::GetMappedRangeValueUnclamped(
					OldRange, LastURange, FMath::Wrap(LastTextureCoordUOffset, OldRange.X, OldRange.Y)
				);
				UV_U += UVOGenerationOffset;
				
				UV1.X = 0;

				CurrentIndices.Add(OutLocalMeshData.Vertices.Emplace(
					FirstVertex.Position,
					FirstVertex.Normal,
					FVector2f(UV_U, FinalTextureV),
					UV1,
					UV2,
					static_cast<float>(LastPointIndex)
				));
			}
			
			PreviousPosition = CurrentPosition;

			TriangulateRings(PreviousIndices, CurrentIndices, PolyGroupIndex, OutLocalMeshData);

			PreviousIndices = MoveTemp(CurrentIndices);
			CurrentIndices.Empty();
		}
	}
}

void FPVMeshBuilder::UpdateFoliagePivotPoints(const TSet<int32>& PointsToRemove, FManagedArrayCollection& OutCollection)
{
	const PV::Facades::FBranchFacade BranchFacade(OutCollection);
	const PV::Facades::FPointFacade PointFacade(OutCollection);
	PV::Facades::FFoliageFacade FoliageFacade(OutCollection);

	TMap<int32, TArray<int32>> PointsIndicesToFoliageIndicesMap = GetPointsIndicesToFoliageIndicesMap(OutCollection);

	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);

		if (BranchPoints.Num() == 0)
		{
			continue;
		}

		TArray<int32> SortedPointIndices = BranchPoints;
		SortedPointIndices.Sort([PointFacade](const int32& A, const int32& B)
			{
				return PointFacade.GetLengthFromRoot(A) < PointFacade.GetLengthFromRoot(B);
			});

		for (int32 i = 0; i < SortedPointIndices.Num(); ++i)
		{
			const int32 CurrentPointIndex = SortedPointIndices[i];
			if (PointsToRemove.Contains(CurrentPointIndex) &&
				i > 0 &&
				i < SortedPointIndices.Num() - 1 &&
				PointsIndicesToFoliageIndicesMap.Contains(CurrentPointIndex))
			{
				int32 j = i - 1;
				while (j > 0 && PointsToRemove.Contains(SortedPointIndices[j]))
				{
					j--;
				}
				const int32 PreviousPointIndex = SortedPointIndices[j];

				int32 k = i + 1;
				while (k < (SortedPointIndices.Num() - 1) && PointsToRemove.Contains(SortedPointIndices[k]))
				{
					k++;
				}
				const int32 NextPointIndex = SortedPointIndices[k];

				const float PreviousPointLengthFromRoot = PointFacade.GetLengthFromRoot(PreviousPointIndex);
				const float CurrentPointLengthFromRoot = PointFacade.GetLengthFromRoot(CurrentPointIndex);
				const float NextPointLengthFromRoot = PointFacade.GetLengthFromRoot(NextPointIndex);
				const float BlendValue = FMath::GetMappedRangeValueClamped(
					FVector2f(PreviousPointLengthFromRoot, NextPointLengthFromRoot),
					FVector2f(0.0f, 1.0f),
					CurrentPointLengthFromRoot
				);

				const FVector3f& CurrentPointPosition = PointFacade.GetPosition(CurrentPointIndex);
				const FVector3f& PreviousPointPosition = PointFacade.GetPosition(PreviousPointIndex);
				const FVector3f& NextPointPosition = PointFacade.GetPosition(NextPointIndex);

				const FVector3f InterpolatedPointPosition = FMath::Lerp(PreviousPointPosition, NextPointPosition, BlendValue);

				for (const int32 FoliageIndex : PointsIndicesToFoliageIndicesMap[CurrentPointIndex])
				{
					const FVector3f& FoliagePosition = FoliageFacade.GetPivotPoint(FoliageIndex);
					const FVector3f OffsetVector = FoliagePosition - CurrentPointPosition;
					const FVector3f UpdatedFoliagePosition = InterpolatedPointPosition + OffsetVector;

					FoliageFacade.SetPivotPoint(FoliageIndex, UpdatedFoliagePosition);
				}
			}
		}
	}
}

bool IsDisplacementTextureValid(const TObjectPtr<UTexture2D>& Texture)
{
#if WITH_EDITOR
	return Texture && Texture->Source.IsValid();
#else
	return false;
#endif
}

void FPVMeshBuilder::GetLocalDynamicMeshData(FManagedArrayCollection& Collection, const FManagedArrayCollection& InPlantProfileCollection, const FPVMeshBuilderParams& MeshBuilderParams,
                                             TArray<FLocalDynamicMeshData>& MeshesData)
{
	const PV::Facades::FBranchFacade BranchFacade(Collection);
	const PV::Facades::FPointFacade PointFacade(Collection);
	const PV::Facades::FPlantFacade PlantFacade(Collection);

	if (!BranchFacade.IsValid() || !PointFacade.IsValid() || !PlantFacade.IsValid())
	{
		UE_LOGF(LogProceduralVegetation, Warning, "Fail to evaluate, InCollection dose not match the schema.");
		return;
	}

	PV::FPointPositionAttributeConstView PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(Collection);
	PV::FPointScaleAttributeConstView PointScaleAttribute = PV::FPointScaleAttribute::FindAttribute(Collection);
	PV::FBranchPointsAttributeConstView BranchPointsAttribute = PV::FBranchPointsAttribute::FindAttribute(Collection);
	PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(Collection);
	PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute = PV::FBranchPlantNumberAttribute::FindAttribute(Collection);
	PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute = PV::FBudDevelopmentAttribute::FindAttribute(Collection);
	PV::FPointNjordPixelIndexAttributeConstView NjordPixelIndexAttribute = PV::FPointNjordPixelIndexAttribute::FindAttribute(Collection);

	if (!PV::ValidateAttributeCollection(
		PointPositionAttribute,
		PointScaleAttribute,
		BranchPointsAttribute,
		BranchParentNumberAttribute,
		BranchPlantNumberAttribute,
		BudDevelopmentAttribute,
		NjordPixelIndexAttribute
	))
	{
		return;
	}

	PV::FPointGroundGradientAttributeView GroundGradientAttribute = PV::FPointGroundGradientAttribute::AddAttribute(Collection);
	PV::FPointScaleGradientAttributeView PointScaleGradientAttribute = PV::FPointScaleGradientAttribute::AddAttribute(Collection);
	PV::FPointHullGradientAttributeView HullGradientAttribute = PV::FPointHullGradientAttribute::AddAttribute(Collection);
	PV::FPointMainTrunkGradientAttributeView MainTrunkGradientAttribute = PV::FPointMainTrunkGradientAttribute::AddAttribute(Collection);

	PV::AttributesHelper::ComputePointGroundGradient({ GroundGradientAttribute, PointPositionAttribute });
	PV::AttributesHelper::ComputePointScaleGradient({ PointScaleGradientAttribute, PointScaleAttribute });
	PV::AttributesHelper::ComputePointHullGradient({ HullGradientAttribute, PointPositionAttribute, BranchPointsAttribute, BranchParentNumberAttribute });
	PV::AttributesHelper::ComputePointMainTrunkGradient({ MainTrunkGradientAttribute, PointPositionAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchPlantNumberAttribute, BudDevelopmentAttribute });

	const float MaxPointScale = *Algo::MaxElement(PointScaleAttribute);

	const TSet<int32> HardPoints = CollectHardPoints(BranchFacade, PointFacade, PlantFacade, NjordPixelIndexAttribute);

	TArray<float> MeshDivisionsGradientsForPoints;
	TArray<float> DeltaModifiersForPoints;
	TSet<int32> PointsToRemove = ComputePointGradients(
		HullGradientAttribute, 
		MainTrunkGradientAttribute, 
		GroundGradientAttribute, 
		PointScaleGradientAttribute, 
		MeshBuilderParams, 
		HardPoints, 
		MaxPointScale, 
		MeshDivisionsGradientsForPoints,
		DeltaModifiersForPoints
	);

	PerformPathSimplification(BranchFacade, PointFacade, MeshBuilderParams, MaxPointScale, HardPoints, DeltaModifiersForPoints, PointsToRemove);

	// Point removal may update curvate of the branch.
	// Foliage instance pivot/attachment points need to be updated to compensate for that.
	if (PointsToRemove.Num() > 0)
	{
		UpdateFoliagePivotPoints(PointsToRemove, Collection);
	}

	const TArray<int32> TargetMeshDivisions = ComputeMeshDivisions(BranchFacade, MeshBuilderParams, MeshDivisionsGradientsForPoints,
		PointFacade.GetElementCount());

	MeshesData.SetNum(BranchFacade.GetElementCount());

	int32 DisplacementWidth = 0;
	int32 DisplacementHeight = 0;
	if (MeshBuilderParams.Displacement.Texture
		&& IsDisplacementTextureValid(MeshBuilderParams.Displacement.Texture)
		&& MeshBuilderParams.DisplacementValues.Num() > 0
		&& MeshBuilderParams.Displacement.Strength > 0.0f)
	{
#if WITH_EDITOR
		DisplacementWidth = MeshBuilderParams.Displacement.Texture->Source.GetSizeX();
		DisplacementHeight = MeshBuilderParams.Displacement.Texture->Source.GetSizeY();
#endif
		if (MeshBuilderParams.Displacement.Texture->SRGB)
		{
			UE_LOGF(LogProceduralVegetation, Warning,
				"Displacement texture selected has sRGB enabled! This will corrupt displacement data. Please disable it.");
		}
	}

	const FDisplacementData DisplacementData(MeshBuilderParams.DisplacementValues, DisplacementWidth, DisplacementHeight);

	ParallelFor(
		BranchFacade.GetElementCount(),
		[&](const int32 PrimitiveIndex)
			{
				const TArray<int32>& PrimitivePoints = BranchFacade.GetPoints(PrimitiveIndex).FilterByPredicate(
					[&PointsToRemove](const int32 PointIndex)
						{
							return !PointsToRemove.Contains(PointIndex);
						});

				const bool PrimitiveIsTrunk = PlantFacade.IsTrunkIndex(PrimitiveIndex);
				const int32 GenerationNumber = PV::Facades::FTreeFacade::GetBranchGenerationNumber(Collection, PrimitiveIndex);

				MeshesData[PrimitiveIndex].MaterialID = BranchFacade.GetBranchUVMaterial(PrimitiveIndex);

				GenerateBranchMeshData(PrimitiveIsTrunk, GenerationNumber, PrimitiveIndex, PrimitivePoints, DisplacementData, MeshBuilderParams, TargetMeshDivisions,
					Collection, InPlantProfileCollection, MeshesData[PrimitiveIndex]);
			});
}

bool FPVMeshBuilder::ExtractDisplacementData(const TObjectPtr<UTexture2D>& Texture, TArray<float>& OutValues, FString& OutError)
{
	OutValues.Empty();

#if WITH_EDITOR
	if (!Texture)
	{
		OutError = "ExtractDisplacementData: Texture is null.";
		UE_LOGF(LogProceduralVegetation, Warning, "%ls", *OutError);
		return false;
	}

	FTextureSource& Source = Texture->Source;
	if (!Source.IsValid())
	{
		OutError = FString::Printf(TEXT("ExtractDisplacementData: No source data found on texture %s."), *Texture->GetName());
		UE_LOGF(LogProceduralVegetation, Warning, "%ls", *OutError);
		return false;
	}

	if (!Source.AreAllBlocksPowerOfTwo())
	{
		OutError = FString::Printf(TEXT("ExtractDisplacementData: Texture %s is not power of two, Only Power of 2 textures are supported."), *Texture->GetName());
		UE_LOGF(LogProceduralVegetation, Warning, "%ls", *OutError);
		return false;
	}

	const int32 Width = Source.GetSizeX();
	const int32 Height = Source.GetSizeY();
	const int32 NumPix = Width * Height;
	OutValues.SetNumZeroed(NumPix);

	const uint8* RawData = Source.LockMipReadOnly(0);

	bool bSuccess = true;
	switch (Source.GetFormat())
	{
	case TSF_RGBA32F:
		{
			const float* FloatData = reinterpret_cast<const float*>(RawData);
			for (int32 i = 0; i < NumPix; ++i)
			{
				OutValues[i] = FloatData[i * 4];
			}
		}
		break;
	case TSF_R32F:
		{
			const float* FloatData = reinterpret_cast<const float*>(RawData);
			for (int32 i = 0; i < NumPix; ++i)
			{
				OutValues[i] = FloatData[i];
			}
		}
		break;
	case TSF_RGBA16F:
		{
			const FFloat16* HalfData = reinterpret_cast<const FFloat16*>(RawData);
			for (int32 i = 0; i < NumPix; ++i)
			{
				OutValues[i] = HalfData[i * 4].GetFloat();
			}
		}
		break;
	case TSF_R16F:
		{
			const FFloat16* HalfData = reinterpret_cast<const FFloat16*>(RawData);
			for (int32 i = 0; i < NumPix; ++i)
			{
				OutValues[i] = HalfData[i].GetFloat();
			}
		}
		break;
	case TSF_BGRA8:
		{
			const FColor* ColorData = reinterpret_cast<const FColor*>(RawData);
			constexpr float Inv255 = 1.0f / 255.0f;
			for (int32 i = 0; i < NumPix; ++i)
			{
				OutValues[i] = ColorData[i].R * Inv255;
			}
		}
		break;
		case TSF_G8:
		{
			const uint8* ColorData = reinterpret_cast<const uint8*>(RawData);
			constexpr float Inv255 = 1.0f / 255.0f;
			for (int32 i = 0; i < NumPix; ++i)
			{
				OutValues[i] = *ColorData * Inv255;
				ColorData++;
			}
		}
		break;
	default:
		OutError = FString::Printf(TEXT("ExtractDisplacementData: Unsupported source format %d on texture %s."), Source.GetFormat(), *Texture->GetName());
		UE_LOGF(LogProceduralVegetation, Warning, "%ls",*OutError);
		bSuccess = false;
		break;
	}


	Source.UnlockMip(0);
	return bSuccess;
#else
	OutError = "ExtractDisplacementData only works in editor builds (WITH_EDITOR).";
	UE_LOGF(LogProceduralVegetation, Warning, "%ls", *OutError);
	return false;
#endif
}

void FPVMeshBuilder::ApplyDaVinciRule(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams)
{
	if (FMath::IsNearlyZero(MeshBuilderParams.BranchRadius.DaVinciRuleStrength))
	{
		return;
	}

	auto BranchPointsAttr = PV::FBranchPointsAttribute::FindAttribute(OutCollection);
	auto BranchPlantNumberAttr = PV::FBranchPlantNumberAttribute::FindAttribute(OutCollection);
	auto PointScaleAttr = PV::FPointScaleAttribute::FindAttribute(OutCollection);
	auto BudLateralMeristemAttr = PV::FBudLateralMeristemAttribute::FindAttribute(OutCollection);
	auto SeedPScaleAttr = PV::FPointSeedPScaleAttribute::FindAttribute(OutCollection);
	auto BranchHierarchyNumberAttr = PV::FBranchHierarchyNumberAttribute::FindAttribute(OutCollection);

	if (!PV::ValidateAttributeCollection(BranchPointsAttr,
		BranchPlantNumberAttr,
		PointScaleAttr,
		BudLateralMeristemAttr,
		SeedPScaleAttr,
		BranchHierarchyNumberAttr))
	{
		return;
	}

	const int32 NumBranches = BranchPointsAttr.Num();

	TArray<int32> SortedBranchIndices;
	SortedBranchIndices.Reserve(NumBranches);
	for (int32 i = 0; i < NumBranches; ++i) { SortedBranchIndices.Add(i); }
	SortedBranchIndices.Sort([&BranchHierarchyNumberAttr](int32 A, int32 B)
	{
		return BranchHierarchyNumberAttr[A] < BranchHierarchyNumberAttr[B];
	});

	// Build per-plant branch lists and find the per-plant max Davinci pscale
	TMap<int32, float> PlantMaxDavinci;
	TMap<int32, TArray<int32>> PlantToBranches;
	for (int32 BranchIndex : SortedBranchIndices)
	{
		const int32 PlantNumber = BranchPlantNumberAttr[BranchIndex];
		PlantToBranches.FindOrAdd(PlantNumber).Add(BranchIndex);
		float& MaxDavinci = PlantMaxDavinci.FindOrAdd(PlantNumber);
		for (const int32 PointIndex : BranchPointsAttr[BranchIndex])
		{
		  MaxDavinci = FMath::Max(MaxDavinci, BudLateralMeristemAttr[PointIndex].Davinci);
		}
	}

	const float Bias = MeshBuilderParams.BranchRadius.DaVinciRuleStrength;
	for (auto& [PlantNumber, PlantBranches] : PlantToBranches)
	{
		float MaxDavinciPscale = PlantMaxDavinci[PlantNumber];
		if (FMath::IsNearlyZero(MaxDavinciPscale))
		{
			continue;
		}
		TArray<int32> ExcludePoints;

		float RootScale = 1.0f;
		
		for (const int32 BranchIndex : PlantBranches)
		{
			for (const int32 PointIndex : BranchPointsAttr[BranchIndex])
			{
				if(PointIndex == 0)
				{
					RootScale = PointScaleAttr[PointIndex];
				}
				
				if (!ExcludePoints.Contains(PointIndex))
				{
					float DavinciPscale = FMath::GetMappedRangeValueClamped(FVector2f(0.0f, MaxDavinciPscale), FVector2f(0.0f, 1.0f), BudLateralMeristemAttr[PointIndex].Davinci);
					DavinciPscale *= RootScale;
					PointScaleAttr[PointIndex] = FMath::Lerp(PointScaleAttr[PointIndex], DavinciPscale, Bias);
					ExcludePoints.Add(PointIndex);
				}
			}
		}
	}
}

void FPVMeshBuilder::ApplyBranchGenerationRamps(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams)
{
	if (MeshBuilderParams.BranchRadius.GenerationRamps.IsEmpty())
	{
		return;
	}

	auto BranchPointsAttr          = PV::FBranchPointsAttribute::FindAttribute(OutCollection);
	auto BranchHierarchyNumberAttr = PV::FBranchHierarchyNumberAttribute::FindAttribute(OutCollection);
	auto BranchSourceBudNumberAttr = PV::FBranchSourceBudNumberAttribute::FindAttribute(OutCollection);
	auto BranchChildrenAttr        = PV::FBranchChildrenAttribute::FindAttribute(OutCollection);
	auto BranchNumberAttr          = PV::FBranchNumberAttribute::FindAttribute(OutCollection);
	auto PointBudNumberAttr        = PV::FPointBudNumberAttribute::FindAttribute(OutCollection);
	auto PointScaleAttr            = PV::FPointScaleAttribute::FindAttribute(OutCollection);

	if (!ValidateAttributeCollection(BranchPointsAttr,
		BranchHierarchyNumberAttr,
		BranchSourceBudNumberAttr,
		BranchChildrenAttr,
		BranchNumberAttr,
		PointBudNumberAttr,
		PointScaleAttr))
	{
		return;
	}

	const int32 NumBranches = BranchPointsAttr.Num();
	const int32 NumPoints   = PointScaleAttr.Num();
	const int32 NumRamps    = MeshBuilderParams.BranchRadius.GenerationRamps.Num();

	// BranchNumber (logical ID) -> branch array index
	TMap<int32, int32> BranchNumberToIndex;
	BranchNumberToIndex.Reserve(NumBranches);
	for (int32 i = 0; i < NumBranches; ++i)
	{
		BranchNumberToIndex.Add(BranchNumberAttr[i], i);
	}

	// BudNumber -> branch indices of branches spawned by that bud
	TMap<int32, TArray<int32>> BudToChildBranches;
	for (int32 i = 0; i < NumBranches; ++i)
	{
		BudToChildBranches.FindOrAdd(BranchSourceBudNumberAttr[i]).Add(i);
	}

	// Working copy of pscale so we don't read back partially-written values mid-pass
	TArray<float> PScaleArray;
	PScaleArray.SetNum(NumPoints);
	for (int32 i = 0; i < NumPoints; ++i)
	{
		PScaleArray[i] = PointScaleAttr[i];
	}

	const TArray<float> OriginalPScaleArray = PScaleArray;

	// Track points that have already had a ramp applied across passes.
	// Junction points are shared between the last point of a parent branch (gen N)
	// and the first point of a child branch (gen N+1). Processing in ascending generation
	// order and skipping already-seen points ensures each junction is remapped exactly
	// once — by the parent pass that first encounters it.
	TSet<int32> GlobalProcessedPoints;
	GlobalProcessedPoints.Reserve(NumPoints);

	for (int32 RampIdx = 0; RampIdx < NumRamps; ++RampIdx)
	{
		const int32 LookupGeneration = RampIdx + 1; // generations are 1-based
		const bool  bIsLastRamp      = (RampIdx == NumRamps - 1);
		const FRichCurve* RichCurve  = MeshBuilderParams.BranchRadius.GenerationRamps[RampIdx].Ramp.GetRichCurveConst();
		if (!RichCurve)
		{
			continue;
		}

		float MinPScale = TNumericLimits<float>::Max();
		float MaxPScale = TNumericLimits<float>::Lowest();

		// Collect generation points, skipping any already claimed by a lower-generation pass
		TArray<int32> GenerationPoints;
		for (int32 BranchIdx = 0; BranchIdx < NumBranches; ++BranchIdx)
		{
			const int32 BranchGen = BranchHierarchyNumberAttr[BranchIdx];
			if (BranchGen == LookupGeneration || (bIsLastRamp && BranchGen > LookupGeneration))
			{
				for (const int32 Pt : BranchPointsAttr[BranchIdx])
				{
					if(BranchGen == LookupGeneration && OriginalPScaleArray.IsValidIndex(Pt))
					{
						MinPScale = FMath::Min(MinPScale, OriginalPScaleArray[Pt]);
						MaxPScale = FMath::Max(MaxPScale, OriginalPScaleArray[Pt]);
					}
					
					if (!GlobalProcessedPoints.Contains(Pt))
					{
						GenerationPoints.Add(Pt);
					}
				}
			}
		}

		if (GenerationPoints.IsEmpty())
		{
			continue;
		}

		// Register these points now so subsequent ramp passes skip them
		GlobalProcessedPoints.Append(GenerationPoints);

		// Find pscale range across this generation's (deduplicated) points
		

		if (MaxPScale <= 0.0f || FMath::IsNearlyEqual(MinPScale, MaxPScale))
		{
			continue;
		}

		// Exclusion set: prevents child propagation from double-visiting points
		// within a single ramp pass
		TSet<int32> ExclusionSet(GenerationPoints);

		for (const int32 GenPt : GenerationPoints)
		{
			const float BasePScale = OriginalPScaleArray[GenPt];

			// Ramp lookup: normalize pscale within generation range → ramp → fraction of MaxPScale
			const float RelativeT     = FMath::GetRangePct(MinPScale, MaxPScale, BasePScale);
			const float RampVal       = RichCurve->Eval(FMath::Clamp(RelativeT, 0.0f, 1.0f));
			const float AdjustedScale = RampVal * MaxPScale;
			const float Ratio         = AdjustedScale / BasePScale;

			PScaleArray[GenPt] = FMath::Lerp(MinPScale, MaxPScale, RampVal);//AdjustedScale;

			// Find child branches originating at this point's bud, plus their sub-children
			const int32 BudNum = PointBudNumberAttr[GenPt];
			const TArray<int32>* DirectChildren = BudToChildBranches.Find(BudNum);
			if (!DirectChildren)
			{
				continue;
			}

			TArray<int32> AllChildBranchIndices = *DirectChildren;
			for (const int32 ChildBranchIdx : *DirectChildren)
			{
				for (const int32 SubChildBranchNumber : BranchChildrenAttr[ChildBranchIdx])
				{
					if (const int32* SubChildIdx = BranchNumberToIndex.Find(SubChildBranchNumber))
					{
						AllChildBranchIndices.Add(*SubChildIdx);
					}
				}
			}

			// Propagate the scale ratio to child points not yet touched by another ancestor
			for (const int32 ChildBranchIdx : AllChildBranchIndices)
			{
				for (const int32 ChildPt : BranchPointsAttr[ChildBranchIdx])
				{
					bool bAlreadyInSet;
					ExclusionSet.Add(ChildPt, &bAlreadyInSet);
					if (!bAlreadyInSet)
					{
						PScaleArray[ChildPt] *= Ratio;
					}
				}
			}
		}
	}

	// Write adjusted scales back to the collection
	for (int32 i = 0; i < NumPoints; ++i)
	{
		PointScaleAttr[i] = PScaleArray[i];
	}
}

void FPVMeshBuilder::ApplyBranchGenerationScales(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams)
{
	if (MeshBuilderParams.BranchRadius.GenerationScales.IsEmpty())
	{
		return;
	}

	auto BranchPointsAttr          = PV::FBranchPointsAttribute::FindAttribute(OutCollection);
	auto BranchHierarchyNumberAttr = PV::FBranchHierarchyNumberAttribute::FindAttribute(OutCollection);
	auto PointScaleAttr            = PV::FPointScaleAttribute::FindAttribute(OutCollection);

	if (!BranchPointsAttr.IsValid() || !BranchHierarchyNumberAttr.IsValid() || !PointScaleAttr.IsValid())
	{
		return;
	}

	const int32 NumBranches     = BranchPointsAttr.Num();
	const int32 NumScaleEntries = MeshBuilderParams.BranchRadius.GenerationScales.Num();

	// Sort by ascending generation so parent branches claim shared junction points first.
	// A junction point is the last point of a gen-N branch AND the first point of a gen-N+1
	// branch. Visiting gen-N first ensures it is scaled exactly once by the parent's factor.
	TArray<int32> SortedBranchIndices;
	SortedBranchIndices.Reserve(NumBranches);
	for (int32 i = 0; i < NumBranches; ++i) { SortedBranchIndices.Add(i); }
	SortedBranchIndices.Sort([&BranchHierarchyNumberAttr](int32 A, int32 B)
	{
		return BranchHierarchyNumberAttr[A] < BranchHierarchyNumberAttr[B];
	});

	TSet<int32> ProcessedPoints;
	ProcessedPoints.Reserve(PointScaleAttr.Num());

	for (const int32 BranchIdx : SortedBranchIndices)
	{
		const int32 BranchGen   = BranchHierarchyNumberAttr[BranchIdx];
		const int32 EntryIdx    = FMath::Clamp(BranchGen - 1, 0, NumScaleEntries - 1);
		const float ScaleFactor = MeshBuilderParams.BranchRadius.GenerationScales[EntryIdx].Scale;

		for (const int32 Pt : BranchPointsAttr[BranchIdx])
		{
			bool bAlreadyProcessed;
			ProcessedPoints.Add(Pt, &bAlreadyProcessed);
			if (!bAlreadyProcessed)
			{
				PointScaleAttr[Pt] *= ScaleFactor;
			}
		}
	}
}

void FPVMeshBuilder::ApplyMinRadius(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams)
{
	if (FMath::IsNearlyZero(MeshBuilderParams.BranchRadius.MinRadius))
	{
		return;
	}

	auto PointScaleAttr = PV::FPointScaleAttribute::FindAttribute(OutCollection);
	if (!PointScaleAttr.IsValid() || PointScaleAttr.Num() == 0)
	{
		return;
	}

	float AbsMin = TNumericLimits<float>::Max();
	float AbsMax = TNumericLimits<float>::Lowest();
	for (int32 i = 0; i < PointScaleAttr.Num(); ++i)
	{
		const float Scale = PointScaleAttr[i];
		AbsMin = FMath::Min(AbsMin, Scale);
		AbsMax = FMath::Max(AbsMax, Scale);
	}

	if (FMath::IsNearlyEqual(AbsMin, AbsMax))
	{
		return;
	}

	// Remap the full distribution from [AbsMin, AbsMax] → [MinRadius, AbsMax],
	// matching the Houdini fit() behaviour: thinnest branch is raised to MinRadius,
	// thickest branch stays at AbsMax.
	const float MinRadius = MeshBuilderParams.BranchRadius.MinRadius * 100;
	for (int32 i = 0; i < PointScaleAttr.Num(); ++i)
	{
		const float T = FMath::GetRangePct(AbsMin, AbsMax, PointScaleAttr[i]);
		PointScaleAttr[i] = FMath::Lerp(MinRadius, AbsMax, T);
	}
}

