// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVMaterialSettings.h"
#include "UDynamicMesh.h"

#include "Curves/CurveFloat.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"

#include "Generators/MeshShapeGenerator.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Utils/PVAttributes.h"

#include "DataTypes/PVMeshBuilderParams.h"

#include "PVMeshBuilder.generated.h"

USTRUCT()
struct FPVMeshBuilderParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Skeleton Shaping", meta=(ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVMeshBuilderSkeletonShapingParams SkeletonShaping;

	UPROPERTY(EditAnywhere, Category="Branch Radius", meta=(ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVMeshBuilderBranchRadiusParams BranchRadius;

	UPROPERTY(EditAnywhere, Category="Profile Details", meta=(ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVMeshBuilderProfileDetailParams ProfileDetails;

	UPROPERTY(EditAnywhere, Category="Mesh Details", meta=(ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVMeshBuilderMeshDetailParams MeshDetails;

	UPROPERTY(EditAnywhere, Category="Material Details", meta=(PCG_NotOverridable))
	FPVMaterialSettings MaterialDetails;

	UPROPERTY(EditAnywhere, Category="Displacement", meta=(ShowOnlyInnerProperties, PCG_NotOverridable))
	FPVMeshBuilderDisplacementParams Displacement;

	UPROPERTY(Transient, NonTransactional)
	TArray<float> DisplacementValues;
};

struct FLocalDynamicMeshData
{
	struct FVertex
	{
		FVector3f Position;
		FVector3f Normal;
		FVector2f UV;
		FVector2f UV1;
		FVector2f UV2;
		float PointIndex;
	};

	TArray<FVertex> Vertices;
	TArray<FIntVector4> Triangles;
	int MaterialID = 0;
};

struct FDisplacementData
{
	const TArrayView<const float> Values;
	const int32 TextureWidth;
	const int32 TextureHeight;

	FDisplacementData(const TArrayView<const float>& InValues, const int32 InWidth, const int32 InHeight)
		: Values(InValues)
		, TextureWidth(InWidth)
		, TextureHeight(InHeight)
	{
	}
};

class FPVMeshGenerator : public UE::Geometry::FMeshShapeGenerator
{
	int32 VerticesCount = 0;
	int32 TriangleCount = 0;

	TArray<FLocalDynamicMeshData>* MeshesDatas = nullptr;

public:
	FPVMeshGenerator(const int32 InVerticesCount, const int32 InTriangleCount, TArray<FLocalDynamicMeshData>* InMeshesDatas)
		: VerticesCount(InVerticesCount)
		, TriangleCount(InTriangleCount)
		, MeshesDatas(InMeshesDatas)
	{
	}

	virtual FMeshShapeGenerator& Generate() override
	{
		SetBufferSizes(VerticesCount, TriangleCount, VerticesCount, TriangleCount);

		int32 VertexIndex = 0;
		int32 FaceIndex = 0;
		for (auto MeshIndex = MeshesDatas->Num() - 1; MeshIndex >= 0; --MeshIndex)
		{
			const FLocalDynamicMeshData& MeshesData = (*MeshesDatas)[MeshIndex];
			for (const FIntVector4& Triangle : MeshesData.Triangles)
			{
				const FIntVector4 OffsetTriangle = Triangle + FIntVector4(VertexIndex);
				const UE::Geometry::FIndex3i Indices = FIntVector(OffsetTriangle);
				Triangles[FaceIndex] = Indices;
				TriangleUVs[FaceIndex] = Indices;
				TriangleNormals[FaceIndex] = Indices;
				TrianglePolygonIDs[FaceIndex] = OffsetTriangle.W;
				++FaceIndex;
			}
			for (const FLocalDynamicMeshData::FVertex& Vertex : MeshesData.Vertices)
			{
				Vertices[VertexIndex] = static_cast<FVector>(Vertex.Position);
				Normals[VertexIndex] = Vertex.Normal;
				NormalParentVertex[VertexIndex] = VertexIndex;
				UVs[VertexIndex] = Vertex.UV;
				UVParentVertex[VertexIndex] = VertexIndex;
				++VertexIndex;
			}
		}

		return *this;
	}
};

struct FMeshPointUVData
{
	float TextureCoordV;
	float TextureCoordUOffset;
	FVector2f URange;
};

struct FPVMeshBuilder
{
	static void GenerateGeometryCollection(const FManagedArrayCollection& InSkeletonCollection, const FManagedArrayCollection& InPlantProfileCollection, const FPVMeshBuilderParams& MeshBuilderParams,
	                                       FGeometryCollection& OutGeometryCollection);

	static void AddPointsToSkeleton(FManagedArrayCollection& OutCollection, int32 SkeletonResolution);

	static void ApplyNoiseToSkeleton(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams);

	static void GenerateDynamicMesh(FManagedArrayCollection& Collection, const FManagedArrayCollection& InPlantProfileCollection, const FPVMeshBuilderParams& MeshBuilderParams,
	                                TObjectPtr<UDynamicMesh>& OutMesh);

	static bool ExtractDisplacementData(const TObjectPtr<UTexture2D>& Texture, TArray<float>& OutValues, FString& OutError);

private:
	static TSet<int32> CollectHardPoints(const PV::Facades::FBranchFacade& BranchFacade, const PV::Facades::FPointFacade& PointFacade,
	                                     const PV::Facades::FPlantFacade& PlantFacade, PV::FPointNjordPixelIndexAttributeConstView NjordPixelIndexAttribute);

	static TSet<int32> ComputePointGradients(PV::FPointHullGradientAttributeConstView HullGradientAttribute,
		                                     PV::FPointMainTrunkGradientAttributeConstView MainTrunkGradientAttribute,
		                                     PV::FPointGroundGradientAttributeConstView GroundGradientAttribute,
		                                     PV::FPointScaleGradientAttributeConstView ScaleGradientAttribute,
		                                     const FPVMeshBuilderParams& MeshBuilderParams,
	                                         const TSet<int32>& HardPoints, const float MaxPointScale, TArray<float>& OutMeshDivisionsGradients,
	                                         TArray<float>& OutDeltaModifiers);

	static float GetMaxDeltaBetweenHardPoints(const PV::Facades::FBranchFacade& BranchFacade, const PV::Facades::FPointFacade& PointFacade,
	                                          const TSet<int32>& HardPoints);

	static void PerformPathSimplification(const PV::Facades::FBranchFacade& BranchFacade,
	                                      const PV::Facades::FPointFacade& PointFacade, const FPVMeshBuilderParams& MeshBuilderParams,
	                                      const float MaxPointScale, const TSet<int32>& HardPoints, const TArray<float>& DeltaModifiers,
	                                      TSet<int32>& InOutPointsToRemove);

	static TArray<int32> ComputeMeshDivisions(const PV::Facades::FBranchFacade& BranchFacade, const FPVMeshBuilderParams& MeshBuilderParams,
	                                          const TArray<float>& MeshDivisionsGradients, const int32 PointCount);

	static void TriangulateRings(const TArray<int32>& PreviousIndices, const TArray<int32>& CurrentIndices, int32& InOutPolyGroupIndex,
	                             FLocalDynamicMeshData& OutMeshData);

	static float GetProfileMultiplier(const TArray<float>& InProfilePoints, const float ProfileUV_U);

	static TMap<int32, TArray<int32>> GetPointsIndicesToFoliageIndicesMap(const FManagedArrayCollection& Collection);

	static FMeshPointUVData ComputePointUVData(int32 PointIndex, int32 BranchPointIndex, bool bPrimitiveIsTrunk,
	                                       const TArray<int32>& PrimitivePoints, const PV::Facades::FPointFacade& PointFacade,
	                                       const TManagedArray<FVector3f>& PointPositions, float PointScale);

	static void GenerateBranchMeshData(const bool bPrimitiveIsTrunk, const int32 GenerationNumber, int32 BranchIndex, const TArray<int32>& PrimitivePoints,
	                                   const FDisplacementData& DisplacementData, const FPVMeshBuilderParams& MeshBuilderParams,
	                                   const TArray<int32>& TargetMeshDivisions, const FManagedArrayCollection& Collection, const FManagedArrayCollection& InPlantProfileCollection,
	                                   FLocalDynamicMeshData& OutLocalMeshData);

	static void UpdateFoliagePivotPoints(const TSet<int32>& PointsToRemove, FManagedArrayCollection& OutCollection);

	static void GetLocalDynamicMeshData(FManagedArrayCollection& Collection, const FManagedArrayCollection& InPlantProfileCollection, const FPVMeshBuilderParams& MeshBuilderParams,
	                                    TArray<FLocalDynamicMeshData>& MeshesData);

	static void ApplyDaVinciRule(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams);

	static void ApplyBranchGenerationRamps(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams);

	static void ApplyBranchGenerationScales(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams);

	static void ApplyMinRadius(FManagedArrayCollection& OutCollection, const FPVMeshBuilderParams& MeshBuilderParams);
};
