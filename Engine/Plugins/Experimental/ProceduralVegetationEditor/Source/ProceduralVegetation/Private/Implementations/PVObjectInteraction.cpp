// Copyright Epic Games, Inc. All Rights Reserved.
#include "PVObjectInteraction.h"

#include "MeshDescriptionAdapter.h"

#include "DataTypes/PVGrowthData.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "Engine/StaticMesh.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"

#include "Helpers/PVUtilities.h"

#include "Implementations/PVCarve.h"

FVector FPVObjectInteraction::GetTriangleNormal(FMeshDescriptionTriangleMeshAdapter& Mesh, const int32 TID, TOptional<FVector> BaryCoords)
{
	if (Mesh.HasNormals() && BaryCoords.IsSet())
	{
		FVector3f NormalA, NormalB, NormalC;
		Mesh.GetTriNormals(TID, NormalA, NormalB, NormalC);
		return FVector(BaryCoords->X * NormalA + BaryCoords->Y * NormalB + BaryCoords->Z * NormalC).GetUnsafeNormal();
	}
	else
	{
		FVector VertexA, VertexB, VertexC;
		Mesh.GetTriVertices(TID, VertexA, VertexB, VertexC);
		return UE::Geometry::VectorUtil::Normal(VertexA, VertexB, VertexC);
	}
}

void FPVObjectInteraction::ObjectInteraction(FManagedArrayCollection& OutCollection, FPVColliderParams Collider)
{
	const FTransform& Transform = Collider.Transform;

	const UStaticMesh* ColliderMesh = Collider.Mesh.LoadSynchronous();
	if (!ColliderMesh)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	const FMeshDescription* MeshDescription = ColliderMesh->GetMeshDescription(0);
	FMeshDescriptionTriangleMeshAdapter Mesh(MeshDescription);
	UE::Geometry::TMeshAABBTree3 Octree(&Mesh);

	PV::Facades::FPointFacade PointFacade(OutCollection);
	PV::Facades::FBranchFacade BranchFacade(OutCollection);
	PV::Facades::FFoliageFacade FoliageFacade(OutCollection);
	PV::Facades::FBudVectorsFacade BudVectorFacade = PV::Facades::FBudVectorsFacade(OutCollection);

	TArray<bool> PointsToRemove;
	PointsToRemove.Init(false, PointFacade.GetElementCount());
	TArray<bool> BranchesToRemove;
	BranchesToRemove.Init(false, BranchFacade.GetElementCount());

	TManagedArray<FVector3f>& PointPositions = PointFacade.ModifyPositions();
	TManagedArray<TArray<FVector3f>>& BudDirections = BudVectorFacade.ModifyBudDirections();

	const auto ApplyOffsetToBranchSuccessors = [&](const int32 BranchIndex, const FVector3f& Offset)
		{
			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
			for (int32 BranchPointIndex = 1; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
			{
				PointPositions[BranchPoints[BranchPointIndex]] += Offset;
			}

			const TArray<int32> BranchChildren = BranchFacade.GetChildren(BranchIndex);
			for (int32 BranchChild : BranchChildren)
			{
				const int32 ChildIndex = BranchFacade.GetBranchNumbers().Find(BranchChild);
				if (ChildIndex != INDEX_NONE)
				{
					const TArray<int32>& ChildBranchPoints = BranchFacade.GetPoints(ChildIndex);
					for (int32 ChildBranchPointIndex = 1; ChildBranchPointIndex < ChildBranchPoints.Num(); ++ChildBranchPointIndex)
					{
						PointPositions[ChildBranchPoints[ChildBranchPointIndex]] += Offset;
					}
				}
			}
		};

	const auto RemoveBranchSuccessors = [&](const int32 BranchIndex)
		{
			BranchesToRemove[BranchIndex] = true;
			for (const int32 BranchPoint : BranchFacade.GetPoints(BranchIndex))
			{
				PointsToRemove[BranchPoint] = true;
			}

			const TArray<int32> BranchChildren = BranchFacade.GetChildren(BranchIndex);
			for (int32 BranchChild : BranchChildren)
			{
				const int32 ChildIndex = BranchFacade.GetBranchNumbers().Find(BranchChild);
				if (ChildIndex != INDEX_NONE)
				{
					BranchesToRemove[ChildIndex] = true;
					for (const int32 ChildBranchPoint : BranchFacade.GetPoints(ChildIndex))
					{
						PointsToRemove[ChildBranchPoint] = true;
					}
				}
			}
		};
	
	const bool bIsTrimCollider = Collider.CollisionType == EPVCollisionType::TRIM_INSIDE || Collider.CollisionType == EPVCollisionType::TRIM_OUTSIDE;

	TArray<int32> BranchIndices;
	BranchIndices.Reserve(BranchFacade.GetElementCount());
	BranchFacade.GetSortedBranchIndicesByHierarchy(BranchIndices);

	for (int32 BranchIndex : BranchIndices)
	{
		PVE_OUTER_LOOP_DEBUG_CHECK(break);
		
		if (bIsTrimCollider && BranchesToRemove[BranchIndex])
		{
			continue;
		}

		const TArray<int32> BranchPoints = BranchFacade.GetPoints(BranchIndex);
		for (int32 BranchPointIndex = 1; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
		{
			const int32 PointIndex = BranchPoints[BranchPointIndex - 1];
			const int32 NextPointIndex = BranchPoints[BranchPointIndex];

			if (bIsTrimCollider && PointsToRemove[PointIndex])
			{
				continue;
			}

			const FVector PointPosition = static_cast<FVector>(PointPositions[PointIndex]);
			const FVector NextPointPosition = static_cast<FVector>(PointPositions[NextPointIndex]);
			const FVector SegmentVector = NextPointPosition - PointPosition;

			FRay3d Ray(PointPosition, SegmentVector);
			FRay3d InverseRay(Transform.InverseTransformPosition(Ray.Origin), Transform.InverseTransformVector(Ray.Direction));

			const float PointLFR = PointFacade.GetLengthFromRoot(PointIndex);

			const double MaxLength = SegmentVector.Length();
			const FVector MaxLengthVector = Ray.Direction * MaxLength;
			const double InverseMaxLength = Transform.InverseTransformVector(MaxLengthVector).Length();

			int32 HitTriangle = INDEX_NONE;
			FVector BaryCoords;
			if (double RayT; Octree.FindNearestHitTriangle(InverseRay, RayT, HitTriangle, BaryCoords, {InverseMaxLength}))
			{
				const FVector HitNormal = Transform.TransformVectorNoScale(GetTriangleNormal(Mesh, HitTriangle, BaryCoords)).
				                                    GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);
				const FVector InvertedNormal = -HitNormal;

				if (Collider.CollisionType == EPVCollisionType::AVOID)
				{
					const float PointScale = PointFacade.GetPointScale(PointIndex);

					const FVector HitPoint = Transform.TransformPosition(InverseRay.PointAt(RayT));
					const double DistanceAfterHit = FVector::Distance(HitPoint, Ray.PointAt(MaxLength));

					const FVector BiNormal = FVector::CrossProduct(Ray.Direction, InvertedNormal);
					const FVector Tangent = FVector::CrossProduct(InvertedNormal, BiNormal).GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);

					const FVector AvoidanceOffset = Tangent * DistanceAfterHit * (FMath::Abs(FVector::DotProduct(Ray.Direction, InvertedNormal)) + 1);
					const FVector AvoidancePosition = HitPoint + AvoidanceOffset + HitNormal * PointScale;
					const FVector AvoidanceDirection = (AvoidancePosition - PointPosition).GetSafeNormal(UE_SMALL_NUMBER, FVector::DownVector);

					const FVector OffsetPosition = PointPosition + AvoidanceDirection * FVector::Distance(PointPosition, NextPointPosition);
					const FVector3f Offset = FVector3f(OffsetPosition - NextPointPosition);

					PVE_LOOP_DEBUG_PARAMS_START()
						PVE_LOOP_DEBUG_DIRECTION_PARAM("Ray", Ray.Origin, MaxLengthVector);
						PVE_LOOP_DEBUG_DIRECTION_PARAM("Hit Normal", HitPoint, HitNormal);
						PVE_LOOP_DEBUG_DIRECTION_PARAM("Avoidance Offset", HitPoint, AvoidanceOffset);
						PVE_LOOP_DEBUG_DIRECTION_PARAM("Avoidance Direction", NextPointPosition, AvoidanceDirection);
						PVE_LOOP_DEBUG_VECTOR_PARAM("Avoidance Vector", PointPosition, AvoidancePosition);
						PVE_LOOP_DEBUG_DIRECTION_PARAM("Offset", NextPointPosition, OffsetPosition - NextPointPosition);
					PVE_LOOP_DEBUG_PARAMS_END()

					PVE_LOOP_DEBUG_STEP(break);

					const float SmoothnessFactor = 1.1f + (1.0f - Collider.SmoothnessFactor);

					const int32 StartIndex = FMath::Max(1, BranchPointIndex - Collider.SmoothnessAmount);
					FVector3f PreviousSiblingPosition;
					for (int32 SiblingPointIndex = StartIndex; SiblingPointIndex < BranchPointIndex; ++SiblingPointIndex)
					{
						const int32 SiblingPoint = BranchPoints[SiblingPointIndex];
						const int32 NextSiblingPoint = BranchPoints[SiblingPointIndex + 1];

						const float SiblingPointLFR = PointFacade.GetLengthFromRoot(SiblingPoint);
						const float NextSiblingPointLFR = PointFacade.GetLengthFromRoot(NextSiblingPoint);

						const FVector3f PointOffset = Offset / FMath::Pow(SmoothnessFactor, BranchPointIndex - SiblingPointIndex);

						if (SiblingPointIndex != StartIndex)
						{
							const FVector3f SiblingSegmentVector = PointPositions[SiblingPoint] - PreviousSiblingPosition;
							const FQuat4f Rotation = FQuat4f::FindBetween(SiblingSegmentVector, SiblingSegmentVector + PointOffset);

							TArray<FVector3f>& BudDirection = BudDirections[SiblingPoint];
							BudDirection[PV::Facades::BudDirectionsApical] = Rotation.RotateVector(BudDirection[PV::Facades::BudDirectionsApical]);
							BudDirection[PV::Facades::BudDirectionsAxillary] = Rotation.
								RotateVector(BudDirection[PV::Facades::BudDirectionsAxillary]);
							BudDirection[PV::Facades::BudDirectionsGuideCurve] = Rotation.RotateVector(
								BudDirection[PV::Facades::BudDirectionsGuideCurve]);
							BudDirection[PV::Facades::BudDirectionsUpVector] = Rotation.
								RotateVector(BudDirection[PV::Facades::BudDirectionsUpVector]);
						}

						PreviousSiblingPosition = PointPositions[SiblingPoint];
						PointPositions[SiblingPoint] += PointOffset;

						const TArray<int32> BranchChildren = BranchFacade.GetImmediateChildren(BranchIndex);
						for (int32 ChildBranch : BranchChildren)
						{
							const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch);
							if (ChildBranchIndex != INDEX_NONE)
							{
								const int32 RootPoint = BranchFacade.GetRootPoint(ChildBranchIndex);
								const float ChildRootPointLFR = PointFacade.GetLengthFromRoot(RootPoint);
								if (ChildRootPointLFR >= SiblingPointLFR && ChildRootPointLFR < NextSiblingPointLFR)
								{
									ApplyOffsetToBranchSuccessors(ChildBranchIndex, PointOffset);
								}
							}
						}
					}

					for (int32 SiblingPointIndex = BranchPointIndex; SiblingPointIndex < BranchPoints.Num(); ++SiblingPointIndex)
					{
						PointPositions[BranchPoints[SiblingPointIndex]] += Offset;
					}

					const FQuat4f Rotation = FQuat4f::FindBetween(FVector3f(SegmentVector), FVector3f(SegmentVector) + Offset);

					TArray<FVector3f>& BudDirection = BudDirections[NextPointIndex];
					BudDirection[PV::Facades::BudDirectionsApical] = Rotation.RotateVector(BudDirection[PV::Facades::BudDirectionsApical]);
					BudDirection[PV::Facades::BudDirectionsAxillary] = Rotation.RotateVector(BudDirection[PV::Facades::BudDirectionsAxillary]);
					BudDirection[PV::Facades::BudDirectionsGuideCurve] = Rotation.RotateVector(BudDirection[PV::Facades::BudDirectionsGuideCurve]);
					BudDirection[PV::Facades::BudDirectionsUpVector] = Rotation.RotateVector(BudDirection[PV::Facades::BudDirectionsUpVector]);

					const TArray<int32> BranchChildren = BranchFacade.GetImmediateChildren(BranchIndex);
					for (int32 ChildBranch : BranchChildren)
					{
						const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch);
						if (ChildBranchIndex != INDEX_NONE)
						{
							const int32 RootPoint = BranchFacade.GetRootPoint(ChildBranchIndex);
							if (PointFacade.GetLengthFromRoot(RootPoint) > PointLFR)
							{
								ApplyOffsetToBranchSuccessors(ChildBranchIndex, Offset);
							}
						}
					}
				}
				else if (bIsTrimCollider)
				{
					const double RayDot = FVector::DotProduct(Ray.Direction, HitNormal);
					if (Collider.CollisionType == EPVCollisionType::TRIM_OUTSIDE ? RayDot > 0 : RayDot < 0)
					{
						for (int32 SiblingPointIndex = BranchPointIndex; SiblingPointIndex < BranchPoints.Num(); ++SiblingPointIndex)
						{
							PointsToRemove[BranchPoints[SiblingPointIndex]] = true;
						}
						if (BranchPointIndex <= 1)
						{
							PointsToRemove[BranchPoints[0]] = true;
							BranchesToRemove[BranchIndex] = true;
						}

						const float Ratio = static_cast<float>(BranchPointIndex) / static_cast<float>(BranchPoints.Num());

						// Rescale the branches based on the cut point
						for (int32 Index = !BranchFacade.IsTrunk(BranchIndex); Index <= BranchPointIndex; ++Index)
						{
							PointFacade.SetPointScale(BranchPoints[Index], PointFacade.GetPointScale(BranchPoints[Index]) * Ratio);
						}

						for (int32 ChildBranch : BranchFacade.GetChildren(BranchIndex))
						{
							if (const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch); ChildBranchIndex != INDEX_NONE)
							{
								const TArray<int32>& ChildBranchPoints = BranchFacade.GetPoints(ChildBranchIndex);
								for (int32 ChildBranchPointIndex = 1; ChildBranchPointIndex < ChildBranchPoints.Num(); ++ChildBranchPointIndex)
								{
									const int32 ChildBranchPoint = ChildBranchPoints[ChildBranchPointIndex];
									PointFacade.SetPointScale(ChildBranchPoint, PointFacade.GetPointScale(ChildBranchPoint) * Ratio);
								}
							}
						}

						// Remove all branch children that are after the cut point
						const TArray<int32> BranchChildren = BranchFacade.GetImmediateChildren(BranchIndex);
						for (int32 ChildBranch : BranchChildren)
						{
							const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch);
							if (ChildBranchIndex != INDEX_NONE)
							{
								const int32 RootPoint = BranchFacade.GetRootPoint(ChildBranchIndex);
								const float ChildRootPointLFR = PointFacade.GetLengthFromRoot(RootPoint);
								if (ChildRootPointLFR >= PointLFR)
								{
									RemoveBranchSuccessors(ChildBranchIndex);
								}
							}
						}

						break;
					}
				}
			}
		}
	}

	// Remove all the ToRemove entries using the same algorithm being used in Carve algorithm
	if (bIsTrimCollider)
	{
		TArray<bool> FoliageInstancesToRemove;
		PV::Facades::FTreeFacade::RemoveEntriesAndReIndexAttributes(
			OutCollection,
			PointsToRemove,
			BranchesToRemove,
			FoliageInstancesToRemove
		);
	}
#endif
}
