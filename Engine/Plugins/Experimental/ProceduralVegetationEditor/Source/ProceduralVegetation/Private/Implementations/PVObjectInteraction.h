// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PVObjectInteractionSettings.h"

struct FMeshDescriptionTriangleMeshAdapter;

struct FPVObjectInteraction
{
	static FVector GetTriangleNormal(
		FMeshDescriptionTriangleMeshAdapter& Mesh,
		const int32 TID,
		TOptional<FVector> BaryCoords = TOptional<FVector>()
	);
	
	static void ObjectInteraction(FManagedArrayCollection& OutCollection, FPVColliderParams Collider);
};
