// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVRecomputePointScale.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "Curves/RichCurve.h"

void PV::ComputePointScales_SmoothTaper(const FComputePointScales_SmoothTaperAttributes& InAttributes, float MaxTaperRateMultiplier)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputePointScales_SmoothTaper);

	if (!InAttributes.IsValid())
	{
		return;
	}

	using namespace PV::PlantTraversalHelper;

	RecursiveWalkBranches<int32>(
		InAttributes.BranchParentNumber,
		InAttributes.BranchChildren,
		InAttributes.BranchNumber,
		INDEX_NONE,
		[&](int32 BranchIndex, int32& ParentBranch)->EForEachResult
		{
			const TArray<int32>& BranchPoints = InAttributes.BranchPoints[BranchIndex];
			if (BranchPoints.Num() < 2)
			{
				return EForEachResult::Continue;
			}

			const bool bIsRootBranch = ParentBranch == INDEX_NONE;

			int32 StartIndex = bIsRootBranch ? 0 : 1;

			// For non-root branches, the junction (index 0) is shared with the parent and already updated by it.
			// Use it as the taper left anchor only when its scale is lower than the child's first own point,
			// which would otherwise create an upward scale jump at the junction.
			const float JunctionScale = InAttributes.PointScale[BranchPoints[0]];
			const float ChildFirstPointScale = InAttributes.PointScale[BranchPoints[StartIndex]];
			const bool bUseJunctionAsAnchor = !bIsRootBranch && JunctionScale > 0 && JunctionScale < ChildFirstPointScale;
			const int32 AnchorIndex = bUseJunctionAsAnchor ? 0 : StartIndex;

			float CurrentPointScale = InAttributes.PointScale[BranchPoints[AnchorIndex]];
			float CurrentLengthFromRoot = InAttributes.PointLengthFromRoot[BranchPoints[AnchorIndex]];
			if (CurrentPointScale <= 0)
			{
				for (int32 i = StartIndex; i < BranchPoints.Num(); ++i)
				{
					const float PointScale = InAttributes.PointScale[BranchPoints[i]];
					if (PointScale > 0)
					{
						CurrentPointScale = PointScale;
						CurrentLengthFromRoot = InAttributes.PointLengthFromRoot[BranchPoints[i]];
						for (int32 j = StartIndex; j < i; ++j)
						{
							InAttributes.PointScale[BranchPoints[j]] = CurrentPointScale;
						}
						StartIndex = i;
						break;
					}
				}
			}

			const float TipTargetScale = 0.01f;
			const float TipLengthFromRoot = InAttributes.PointLengthFromRoot[BranchPoints.Last()];

			int32 BranchPointIndex = bUseJunctionAsAnchor ? 0 : StartIndex;
			while (BranchPointIndex < BranchPoints.Num() - 1)
			{
				float NextValidScale = -1;
				float NextValidScale_LengthFromRoot = -1;
				int32 NextValidScale_Index = INDEX_NONE;

				for (int32 i = BranchPointIndex + 1; i < BranchPoints.Num(); ++i)
				{
					const int32 PointIndex = BranchPoints[i];
					const float PointScale = InAttributes.PointScale[PointIndex];
					if (PointScale > 0 && PointScale <= CurrentPointScale)
					{
						const float CandidateLengthFromRoot = InAttributes.PointLengthFromRoot[PointIndex];
						const float RemainingLength = FMath::Max(TipLengthFromRoot - CurrentLengthFromRoot, UE_SMALL_NUMBER);
						const float DistToCandidate = FMath::Max(CandidateLengthFromRoot - CurrentLengthFromRoot, UE_SMALL_NUMBER);
						const float ExpectedTaperRate = (CurrentPointScale - TipTargetScale) / RemainingLength;
						const float ImpliedTaperRate = (CurrentPointScale - PointScale) / DistToCandidate;
						if (ImpliedTaperRate <= MaxTaperRateMultiplier * ExpectedTaperRate)
						{
							NextValidScale = PointScale;
							NextValidScale_LengthFromRoot = CandidateLengthFromRoot;
							NextValidScale_Index = i;
							break;
						}
					}
				}

				if (NextValidScale_Index == INDEX_NONE)
				{
					NextValidScale = 0.01f;
					NextValidScale_Index = BranchPoints.Num() - 1;
					NextValidScale_LengthFromRoot = InAttributes.PointLengthFromRoot[BranchPoints[NextValidScale_Index]];
					InAttributes.PointScale[BranchPoints[NextValidScale_Index]] = NextValidScale;
				}

				for (int32 i = BranchPointIndex + 1; i < NextValidScale_Index; ++i)
				{
					const int32 PointIndex = BranchPoints[i];
					const float PointLengthFromRoot = InAttributes.PointLengthFromRoot[PointIndex];
					const float Alpha = FMath::GetMappedRangeValueClamped(FVector2D(CurrentLengthFromRoot, NextValidScale_LengthFromRoot), FVector2D(0, 1), PointLengthFromRoot);
					const float PointScale = FMath::Lerp(CurrentPointScale, NextValidScale, Alpha);
					InAttributes.PointScale[PointIndex] = PointScale;
				}

				CurrentPointScale = NextValidScale;
				CurrentLengthFromRoot = NextValidScale_LengthFromRoot;
				BranchPointIndex = NextValidScale_Index;
			}

			ParentBranch = BranchIndex;

			return EForEachResult::Continue;
		}
	);
}

void PV::ComputePointScales_UserTrunkScale(
	const FComputePointScales_UserTrunkScaleAttributes& InAttributes,
	float TrunkScale,
	const FRichCurve* TaperProfile
)
{
	if (!InAttributes.IsValid())
	{
		return;
	}

	for (int32 PointIndex = 0; PointIndex < InAttributes.PointScale.Num(); ++PointIndex)
	{
		const float PlantGradient = InAttributes.PlantGradient[PointIndex];
		InAttributes.PointScale[PointIndex] = TaperProfile 
			? TaperProfile->Eval(PlantGradient) * TrunkScale
			: InAttributes.PlantGradient[PointIndex] * TrunkScale;
	}
}

void PV::ComputePointScales_MaxScaleAsTrunkScale(
	const FComputePointScales_MaxScaleAsTrunkScale& InAttributes,
	float ScaleMultiplier,
	const FRichCurve* TaperProfile
)
{
	if (!InAttributes.IsValid())
	{
		return;
	}

	PV::PlantTraversalHelper::ForEachPlant(
		InAttributes.BranchPlantNumber,
		[&](int32 PlantNumber) -> PV::PlantTraversalHelper::EForEachResult
		{
			float MaxScale = 0.f;
			PV::PlantTraversalHelper::ForEachPlantPoint(
				InAttributes.BranchPlantNumber,
				InAttributes.BranchPoints,
				InAttributes.BranchParentNumber,
				PlantNumber,
				[&](int32 /*BranchIndex*/, int32 PointIndex) -> PV::PlantTraversalHelper::EForEachResult
				{
					MaxScale = FMath::Max(MaxScale, InAttributes.PointScale[PointIndex]);
					return PV::PlantTraversalHelper::EForEachResult::Continue;
				}
			);

			MaxScale *= ScaleMultiplier;

			PV::PlantTraversalHelper::ForEachPlantPoint(
				InAttributes.BranchPlantNumber,
				InAttributes.BranchPoints,
				InAttributes.BranchParentNumber,
				PlantNumber,
				[&](int32 /*BranchIndex*/, int32 PointIndex) -> PV::PlantTraversalHelper::EForEachResult
				{
					const float PlantGradient = InAttributes.PlantGradient[PointIndex];
					InAttributes.PointScale[PointIndex] = TaperProfile 
						? TaperProfile->Eval(PlantGradient) * MaxScale
						: InAttributes.PlantGradient[PointIndex] * MaxScale;
					return PV::PlantTraversalHelper::EForEachResult::Continue;
				}
			);

			return PV::PlantTraversalHelper::EForEachResult::Continue;
		}
	);
}
