// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Utils/PVAttributes.h"

struct FRichCurve;

namespace PV
{
	struct FComputePointScales_SmoothTaperAttributes
	{
		PV::FPointScaleAttributeView PointScale;
		PV::FPointLengthFromRootAttributeConstView PointLengthFromRoot;
		PV::FBranchPointsAttributeConstView BranchPoints;
		PV::FBranchParentNumberAttributeConstView BranchParentNumber;
		PV::FBranchChildrenAttributeConstView BranchChildren;
		PV::FBranchNumberAttributeConstView BranchNumber;

		FComputePointScales_SmoothTaperAttributes(
			PV::FPointScaleAttributeView InPointScale,
			PV::FPointLengthFromRootAttributeConstView InPointLengthFromRoot,
			PV::FBranchPointsAttributeConstView InBranchPoints,
			PV::FBranchParentNumberAttributeConstView InBranchParentNumber,
			PV::FBranchChildrenAttributeConstView InBranchChildren,
			PV::FBranchNumberAttributeConstView InBranchNumber)
			: PointScale(InPointScale)
			, PointLengthFromRoot(InPointLengthFromRoot)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
			, BranchChildren(InBranchChildren)
			, BranchNumber(InBranchNumber)
		{
		}

		FComputePointScales_SmoothTaperAttributes(FManagedArrayCollection& InCollection)
			: PointScale(PV::FPointScaleAttribute::FindAttribute(InCollection))
			, PointLengthFromRoot(PV::FPointLengthFromRootAttribute::FindAttribute(InCollection))
			, BranchPoints(PV::FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(PV::FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(PV::FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchNumber(PV::FBranchNumberAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(
				PointScale, 
				PointLengthFromRoot, 
				BranchPoints, 
				BranchParentNumber, 
				BranchChildren, 
				BranchNumber
			);
		}
	};

	// Smooths the existing point scales across the branch hierarchy and writes the results back to the PointScale attribute.
	// Expects the PointScale attribute to already contain valid scale values prior to calling this function.
	// MaxTaperRateMultiplier controls how aggressively anomalous values are rejected: lower = stricter smoothing.
	void ComputePointScales_SmoothTaper(
		const FComputePointScales_SmoothTaperAttributes& InAttributes, 
		float MaxTaperRateMultiplier = 3.0f
	);

	struct FComputePointScales_UserTrunkScaleAttributes
	{
		FPointScaleAttributeView PointScale;
		FPointPlantGradientAttributeConstView PlantGradient;

		FComputePointScales_UserTrunkScaleAttributes(
			FPointScaleAttributeView InPointScale,
			FPointPlantGradientAttributeConstView InPlantGradient)
			: PointScale(InPointScale)
			, PlantGradient(InPlantGradient)
		{}

		FComputePointScales_UserTrunkScaleAttributes(FManagedArrayCollection& InCollection)
			: PointScale(FPointScaleAttribute::FindAttribute(InCollection))
			, PlantGradient(FPointPlantGradientAttribute::FindAttribute(InCollection))
		{}

		bool IsValid() const
		{
			return ValidateAttributeCollection(
				PointScale,
				PlantGradient
			);
		}
	};

	// Sets each point's scale to TaperProfile(PlantGradient) * TrunkScale, where TaperProfile is evaluated at each point's PlantGradient to shape the taper.
	// If TaperProfile is null, falls back to PlantGradient * TrunkScale.
	void ComputePointScales_UserTrunkScale(
		const FComputePointScales_UserTrunkScaleAttributes& InAttributes,
		float TrunkScale,
		const FRichCurve* TaperProfile
	);

	struct FComputePointScales_MaxScaleAsTrunkScale
	{
		FPointScaleAttributeView PointScale;
		FPointPlantGradientAttributeConstView PlantGradient;
		FBranchPlantNumberAttributeConstView BranchPlantNumber;
		FBranchPointsAttributeConstView BranchPoints;
		FBranchParentNumberAttributeConstView BranchParentNumber;
		
		FComputePointScales_MaxScaleAsTrunkScale(
			FPointScaleAttributeView InPointScale,
			FPointPlantGradientAttributeConstView InPlantGradient,
			FBranchPlantNumberAttributeConstView InBranchPlantNumber,
			FBranchPointsAttributeConstView InBranchPoints,
			FBranchParentNumberAttributeConstView InBranchParentNumber)
			: PointScale(InPointScale)
			, PlantGradient(InPlantGradient)
			, BranchPlantNumber(InBranchPlantNumber)
			, BranchPoints(InBranchPoints)
			, BranchParentNumber(InBranchParentNumber)
		{
		}

		FComputePointScales_MaxScaleAsTrunkScale(FManagedArrayCollection& InCollection)
			: PointScale(FPointScaleAttribute::FindAttribute(InCollection))
			, PlantGradient(FPointPlantGradientAttribute::FindAttribute(InCollection))
			, BranchPlantNumber(FBranchPlantNumberAttribute::FindAttribute(InCollection))
			, BranchPoints(FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(FBranchParentNumberAttribute::FindAttribute(InCollection))
		{
		}

		bool IsValid() const
		{
			return ValidateAttributeCollection(
				PointScale,
				PlantGradient,
				BranchPlantNumber,
				BranchPoints,
				BranchParentNumber
			);
		}
	};

	// Sets each point's scale to TaperProfile(PlantGradient) * (MaxPointScale * ScaleMultiplier), where MaxPointScale is the largest existing scale on the same plant.
	// If TaperProfile is null, falls back to PlantGradient * (MaxPointScale * ScaleMultiplier).
	void ComputePointScales_MaxScaleAsTrunkScale(
		const FComputePointScales_MaxScaleAsTrunkScale& InAttributes,
		float ScaleMultiplier,
		const FRichCurve* TaperProfile
	);
};
