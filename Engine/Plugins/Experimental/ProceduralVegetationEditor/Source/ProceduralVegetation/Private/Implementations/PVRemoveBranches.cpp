// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVRemoveBranches.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"
#include "Helpers/PVAttributesHelper.h"

namespace PV::Facades
{
	static constexpr int BranchBudLightDetectedIndex = 2;
	static constexpr int BudDevelopmentGenerationIndex = 0;
	static constexpr int BudDevelopmentAgeIndex = 2;

	void GatherBranchRemovalData(const FManagedArrayCollection& Collection, const FBranchFacade& InBranchFacade, const FPointFacade& InPointFacade, const FPlantFacade& InPlantFacade,
	                             const ERemoveBranchesBasis InBasis, TArray<float>& OutRemovalData)
	{
		const auto PointScaleAttribute = PV::FPointScaleAttribute::GetAttribute(Collection);
		const auto BranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(Collection);
		const auto BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(Collection);

		const int32 TotalNumberOfBranches = InBranchFacade.GetElementCount();
		OutRemovalData.Init(-1.0f, TotalNumberOfBranches);

		for (const TMap<int32, int32> PlantNumbersToTrunkIDs = InPlantFacade.GetPlantNumbersToTrunkIndicesMap();
		     const TPair<int32, int32> Pair : PlantNumbersToTrunkIDs)
		{
			const int32 PlantNumber = Pair.Key;
			const int32 TrunkIndex = Pair.Value;
			const TArray<int32> BranchIndices = InPlantFacade.GetBranchIndices(PlantNumber);

			check(TrunkIndex != INDEX_NONE);

			if (TrunkIndex == INDEX_NONE)
				continue;

			const TArray<int32>& TrunkPoints = InBranchFacade.GetPoints(TrunkIndex);

			check(TrunkPoints.Num() > 0);

			if (TrunkPoints.Num() == 0)
				continue;

			const int32 RootPointIndex = TrunkPoints[0];
			const int32 RootHierarchyGeneration =  InBranchFacade.GetHierarchyGenerationNumber(TrunkIndex);
			const TArray<int>& RootBudDevelopment = InPointFacade.GetBudDevelopment(RootPointIndex);

			float MaxValue = std::numeric_limits<float>::lowest();

			for (const int32 BranchIndex : BranchIndices)
			{
				const TArray<int32>& BranchPoints = InBranchFacade.GetPoints(BranchIndex);

				if (BranchPoints.Num() == 0)
				{
					continue;
				}

				if (InPlantFacade.IsTrunkIndex(BranchIndex))
				{
					continue;
				}

				if (InBasis == ERemoveBranchesBasis::Length)
				{
					const float BranchRootPointLFR = InPointFacade.GetLengthFromRoot(BranchPoints[0]);
					const float BranchLastPointLFR = InPointFacade.GetLengthFromRoot(BranchPoints[BranchPoints.Num() - 1]);

					OutRemovalData[BranchIndex] = BranchLastPointLFR - BranchRootPointLFR;
				}
				else if (InBasis == ERemoveBranchesBasis::Radius)
				{
					OutRemovalData[BranchIndex] = PV::AttributesHelper::GetBranchPointScale(PointScaleAttribute, BranchPointsAttribute, BranchParentNumberAttribute, BranchIndex, 0);
				}
				else if (InBasis == ERemoveBranchesBasis::Light)
				{
					if (BranchPoints.Num() < 2)
					{
						continue;
					}

					const TArray<float>& PointBudLightDetected = InPointFacade.GetBudLightDetected(BranchPoints[BranchPoints.Num() - 2]);

					// TODO: Some entries dont have complete data. Need to update the JSON to fix it. For now, setting -1 value to keep track of inaccurate data. 
					OutRemovalData[BranchIndex] = PointBudLightDetected.Num() > BranchBudLightDetectedIndex
						? PointBudLightDetected[BranchBudLightDetectedIndex]
						: -1.0f;
				}
				else if (InBasis == ERemoveBranchesBasis::Age)
				{
					const TArray<int>& PointBudDevelopment = InPointFacade.GetBudDevelopment(BranchPoints[BranchPoints.Num() - 1]);

					const int RootBudAge = RootBudDevelopment[BudDevelopmentAgeIndex];
					OutRemovalData[BranchIndex] = RootBudAge > 0
						? static_cast<float>(PointBudDevelopment[BudDevelopmentAgeIndex]) / static_cast<float>(RootBudAge)
						: -1.0f;
				}
				else if (InBasis == ERemoveBranchesBasis::Generation)
				{
					const int32 BranchHierarchyGeneration =  InBranchFacade.GetHierarchyGenerationNumber(BranchIndex);

					if (BranchHierarchyGeneration > 0)
					{
						OutRemovalData[BranchIndex] = RootHierarchyGeneration / static_cast<float>(BranchHierarchyGeneration);
					}
				}

				MaxValue = FMath::Max(OutRemovalData[BranchIndex], MaxValue);
			}

			if (MaxValue > 1)
			{
				for (const int32 BranchIndex : BranchIndices)
				{
					OutRemovalData[BranchIndex] = OutRemovalData[BranchIndex] / MaxValue;
				}
			}
		}
	}

	void FindBranchesToRemove(const FBranchFacade& InBranchFacade, const TArray<float>& RemovalData, const float& InThreshold,
	                          TArray<int>& OutBranchesToRemove)
	{
		const int32 NumBranches = InBranchFacade.GetElementCount();
		OutBranchesToRemove.Reserve(NumBranches / 2);

		for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
		{
			if (const float Val = RemovalData[BranchIndex];
				Val > 0.0f && Val < InThreshold)
			{
				OutBranchesToRemove.Add(BranchIndex);
			}
		}
	}
}

void FPVRemoveBranches::ApplyRemoveBranches(const ERemoveBranchesBasis InBasis, const float InThreshold, FManagedArrayCollection& OutCollection)
{
	const PV::Facades::FBranchFacade BranchFacade(OutCollection);
	const PV::Facades::FPointFacade PointFacade(OutCollection);
	const PV::Facades::FPlantFacade PlantFacade(OutCollection);

	if (!PointFacade.IsValid() || !BranchFacade.IsValid())
	{
		return;
	}

	TArray<float> OutRemovalData;
	GatherBranchRemovalData(OutCollection, BranchFacade, PointFacade, PlantFacade, InBasis, OutRemovalData);

	TArray<int> OutBranchesToRemove;
	FindBranchesToRemove(BranchFacade, OutRemovalData, InThreshold, OutBranchesToRemove);

	PV::Facades::FTreeFacade::RemoveBranches(BranchFacade, OutBranchesToRemove, OutCollection);
}
