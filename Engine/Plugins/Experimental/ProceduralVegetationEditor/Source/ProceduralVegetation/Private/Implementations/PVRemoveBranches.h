// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVRemoveBranches.generated.h"

struct FManagedArrayCollection;

UENUM()
enum class ERemoveBranchesBasis : uint8
{
	Length,
	Radius,
	Light,
	Age,	
	Generation
};

struct FPVRemoveBranches
{
	static void ApplyRemoveBranches(const ERemoveBranchesBasis InBasis, const float InThreshold, FManagedArrayCollection& OutCollection);
};
