// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVRotateBranches.h"
#include "Helpers/PVTransformHelper.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "Helpers/PVAttributesHelper.h"

void FPVRotateBranches::ApplyRotateBranches(const FPVRotateBranchesParams& InRotateBranchesParams, FManagedArrayCollection& OutCollection)
{
	using namespace PV::Transform;

	const FRotateBranchPointsAttributeCollection Attributes(OutCollection);
	if (!Attributes.IsValid())
	{
		return;
	}

	PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute = PV::FBudDevelopmentAttribute::FindAttribute(OutCollection);
	PV::FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::FindAttribute(OutCollection);
	if (!PV::ValidateAttributeCollection(BudDevelopmentAttribute, PointLengthFromRootAttribute))
	{
		return;
	}

	PV::FPointPlantGradientAttributeView PointPlantGradientAttribute = PV::FPointPlantGradientAttribute::AddAttribute(OutCollection);
	PV::AttributesHelper::ComputePointPlantGradient({
		PointPlantGradientAttribute,
		Attributes.BranchPoints,
		Attributes.BranchParentNumber,
		Attributes.BranchNumber,
		Attributes.BranchChildren
	});


	const int32 StartGeneration = InRotateBranchesParams.StartGeneration;
	const int32 EndGeneration = InRotateBranchesParams.NumGenerations <= 0
		? MAX_int32
		: InRotateBranchesParams.StartGeneration + InRotateBranchesParams.NumGenerations;

	FRandomStream RandomStream(InRotateBranchesParams.RandomSeed);
	int32 CurrentBranchIndex = INDEX_NONE;

	for (const int32 TrunkIndex : PV::PlantTraversalHelper::GetTrunkIndices(Attributes.BranchParentNumber))
	{
		const TArray<int32>& CurrentBranchPoints = Attributes.BranchPoints[TrunkIndex];
		if (CurrentBranchPoints.Num() == 0)
		{
			continue;
		}

		struct FPointState
		{
			int32 PrevBranchIndex = INDEX_NONE;
			int32 PrevBranchPointIndex = INDEX_NONE;
			float RotationSign = 1;
		};

		const auto CalcRotationFunction = [&](const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result, FPointState& PointState)
		{
			if (Params.BranchIndex != CurrentBranchIndex)
			{
				CurrentBranchIndex = Params.BranchIndex;
			}

			if (PointState.PrevBranchIndex != Params.BranchIndex && Params.BranchIndex != TrunkIndex)
			{
				const int32 Generation = BudDevelopmentAttribute[Params.PointIndex].Generation;
				if (Generation < StartGeneration || Generation > EndGeneration)
				{
					return;
				}

				const float PlantGradient = PointPlantGradientAttribute[Params.ParentPointIndex];
				const float BranchGradient = PV::AttributesHelper::GetBranchPointBranchGradient(PointLengthFromRootAttribute, Attributes.BranchPoints, PointState.PrevBranchIndex, PointState.PrevBranchPointIndex);
				const float PlantGradientMultiplier = InRotateBranchesParams.PlantGradientMultiplier.EditorCurveData.Eval(1.f - PlantGradient);
				const float BranchGradientMultiplier = InRotateBranchesParams.BranchGradientMultiplier.EditorCurveData.Eval(1.f - BranchGradient);

				float BlendedMultiplier = 0.f;
				switch (InRotateBranchesParams.GradientBlendMode)
				{
				case EPVGradientBlendMode::Multiply:
					BlendedMultiplier = PlantGradientMultiplier * BranchGradientMultiplier;
					break;
				case EPVGradientBlendMode::Add:
					BlendedMultiplier = FMath::Clamp(PlantGradientMultiplier + BranchGradientMultiplier, -1, 1);
					break;
				case EPVGradientBlendMode::Min:
					BlendedMultiplier = FMath::Min(PlantGradientMultiplier, BranchGradientMultiplier);
					break;
				case EPVGradientBlendMode::Max:
					BlendedMultiplier = FMath::Max(PlantGradientMultiplier, BranchGradientMultiplier);
					break;
				case EPVGradientBlendMode::Lerp:
				default:
					BlendedMultiplier = FMath::Lerp(PlantGradientMultiplier, BranchGradientMultiplier, InRotateBranchesParams.BranchGradientBias);
					break;
				}

				const float RandomAngleRad = InRotateBranchesParams.Randomness > 0
					? RandomStream.FRandRange(-UE_PI, UE_PI) * InRotateBranchesParams.Randomness
					: 0.f;
				const float RotationDeg = (FMath::DegreesToRadians(InRotateBranchesParams.Rotation * PointState.RotationSign) + RandomAngleRad) * BlendedMultiplier;
				const FVector3f& ParentApical = Attributes.BudDirection[Params.ParentPointIndex].Apical;
				Result.Rotation = FQuat4f(ParentApical, RotationDeg);
			}

			if (InRotateBranchesParams.bAlternatingRotations && Params.ChildBranchIndices.Num() > 0)
			{
				PointState.RotationSign *= -1;
			}

			PointState.PrevBranchIndex = Params.BranchIndex;
			PointState.PrevBranchPointIndex = Params.BranchPointIndex;
		};

		RecursiveRotateBranchPoints<FPointState>(Attributes, TrunkIndex, FPointState(), CalcRotationFunction);
	}
}
