// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Utils/PVFloatRamp.h"
#include "PVRotateBranches.generated.h"

UENUM()
enum class EPVGradientBlendMode : uint8
{
	Multiply,
	Add,
	Min,
	Max,
	Lerp
};

USTRUCT()
struct FPVRotateBranchesParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (Units = Degrees, ClampMin = -180.0f, ClampMax = 180.0f, UIMin = -180.0f, UIMax = 180.0f, Tooltip="Base rotation angle applied to each branch in degrees.\n\nPositive = counter-clockwise around the parent axis (right-hand rule). Negative = clockwise. The actual per-branch rotation may be modified by ramps and randomness."))
	float Rotation = 0;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = 1, UIMin = 1, UIMax = 20, Tooltip="First branch generation to receive the rotation.\n\n1 = trunk (no-op since the trunk has no parent). 2 = first lateral branches. Higher numbers skip lower-order branches entirely. Use to apply rotation only to twigs or only to main branches."))
	int32 StartGeneration = 1;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = 0, UIMin = 0, UIMax = 20, Tooltip="How many generations from StartGeneration to affect (0 = unlimited).\n\n0 = all generations from StartGeneration onward. 1 = only the StartGeneration tier. Use to limit the rotation's depth — e.g. rotate only first-order side branches without touching their children."))
	int32 NumGenerations = 0;

	UPROPERTY(EditAnywhere, Category="Settings", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=-1.0f, YAxisMax=1.0f, Tooltip="Scales rotation amount based on branch position along the whole plant.\n\nX = plant gradient (0 = root, 1 = tip). Y = rotation multiplier from -1 to 1. Use to reverse rotation direction in different plant zones, or to suppress rotation near the base."))
	FPVFloatRamp PlantGradientMultiplier;

	UPROPERTY(EditAnywhere, Category="Settings", meta=(XAxisMin=0.0f, XAxisMax=1.0f, YAxisMin=-1.0f, YAxisMax=1.0f, Tooltip="Scales rotation amount based on branch position along its parent.\n\nX = branch gradient (0 = branch root, 1 = branch tip). Y = rotation multiplier. Use to fade rotation in or out along each branch."))
	FPVFloatRamp BranchGradientMultiplier;

	UPROPERTY(EditAnywhere, Category="Settings", meta=(Tooltip="How the plant and branch gradient multipliers combine.\n\nMultiply / Add / Min / Max / Lerp. Lerp blends them by `BranchGradientBias`."))
	EPVGradientBlendMode GradientBlendMode = EPVGradientBlendMode::Lerp;

	UPROPERTY(EditAnywhere, Category="Settings", meta=(ClampMin=0, ClampMax=1, UIMin=0, UIMax=1, EditCondition="GradientBlendMode==EPVGradientBlendMode::Lerp", EditConditionHides, Tooltip="Lerp blend between plant gradient (0) and branch gradient (1).\n\n0 = use only the plant gradient. 1 = use only the branch gradient. 0.5 = equal mix."))
	float BranchGradientBias = 0.f;

	UPROPERTY(EditAnywhere, Category = "Settings", meta=(Tooltip="Alternate rotation direction between successive axillary branches.\n\nWhen on, branches alternate +angle, -angle, +angle, -angle as you walk along the parent. Turn off for uniform same-direction twist."))
	bool bAlternatingRotations = true;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = 0, ClampMax = 1, UIMin = 0, UIMax = 1, Tooltip="Random variation added to each branch's rotation.\n\n0 = perfectly uniform rotation. 1 = full random offset. Use to break perfect symmetry and create natural-looking variation."))
	float Randomness = 0.f;

	UPROPERTY(EditAnywhere, Category = "Settings", meta=(Tooltip="Seed for the random rotation offsets.\n\nChange to get a different random pattern without changing the strength."))
	int32 RandomSeed = 0;

	FPVRotateBranchesParams()
	{
		PlantGradientMultiplier.InitializeLinearCurve();
		BranchGradientMultiplier.InitializeLinearCurve();
	}
};

struct FPVRotateBranches
{
	static void ApplyRotateBranches(const FPVRotateBranchesParams& InRotateBranchesParams, FManagedArrayCollection& OutCollection);
};
