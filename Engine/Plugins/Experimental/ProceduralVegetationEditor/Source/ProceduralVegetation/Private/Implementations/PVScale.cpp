// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVScale.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"

void FPVScale::ApplyScale(const float InManualScale, FManagedArrayCollection& OutCollection)
{
	PV::Facades::FBranchFacade BranchFacade(OutCollection);
	PV::Facades::FPointFacade PointFacade(OutCollection);
	PV::Facades::FPlantFacade PlantFacade(OutCollection);

	if (!PointFacade.IsValid() || !BranchFacade.IsValid() || !PlantFacade.IsValid())
	{
		return;
	}

	for (const TMap<int32, int32> PlantNumbersToTrunkIDs = PlantFacade.GetPlantNumbersToTrunkIndicesMap();
	     const TPair<int32, int32> Pair : PlantNumbersToTrunkIDs)
	{
		const int32 PlantNumber = Pair.Key;
		const int32 TrunkIndex = Pair.Value;
		const TArray<int32> BranchIndices = PlantFacade.GetBranchIndices(PlantNumber);

		check(TrunkIndex != INDEX_NONE);

		if (TrunkIndex == INDEX_NONE)
			continue;

		const TArray<int32>& TrunkPoints = BranchFacade.GetPoints(TrunkIndex);

		check(TrunkPoints.Num() > 0);

		if (TrunkPoints.Num() == 0)
			continue;

		const int32 RootPointIndex = TrunkPoints[0];
		const FVector3f& PivotPos = PointFacade.GetPosition(RootPointIndex);

		for (const int32 BranchIndex : BranchIndices)
		{
			const bool IsFirstGeneration = BranchFacade.IsTrunk(BranchIndex);
			const int SourceBudNumber = BranchFacade.GetBranchSourceBudNumber(BranchIndex);
			
			for (const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
				 const int32& PointIndex : BranchPoints)
			{
				const int BudNumber = PointFacade.GetBudNumber(PointIndex);
				
				if (IsFirstGeneration || BudNumber != SourceBudNumber)
				{
					FVector3f PointPos = PointFacade.GetPosition(PointIndex);
					float LFS = PointFacade.GetLengthFromSeed(PointIndex);
					float LFR = PointFacade.GetLengthFromRoot(PointIndex);

					PointPos -= PivotPos;
					PointPos *= InManualScale;
					PointPos += PivotPos;

					LFS *= InManualScale;
					LFR *= InManualScale;
				
					PointFacade.ModifyPositions()[PointIndex] = PointPos;
					PointFacade.ModifyLengthFromSeeds()[PointIndex] = LFS;
					PointFacade.ModifyLengthFromRoots()[PointIndex] = LFR;
					PointFacade.ModifyPointScales()[PointIndex] *= InManualScale;
				}
			}
		}
	}
}
