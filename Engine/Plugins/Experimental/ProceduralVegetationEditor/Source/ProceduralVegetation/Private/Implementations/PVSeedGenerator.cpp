// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSeedGenerator.h"
#include "PVGrowerData.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"
#include "Utils/PCGValueRange.h"

#define LOCTEXT_NAMESPACE "PVSeedGenerator"

TArray<FTransform> FPVConvertToSeedPointImplementation::FindNeighbours(const FTransform& InTransform, const UPCGPointData* PointData, float MaxDist, int MaxPoints)
{
	TArray<FTransform> Neighbours;

	TConstPCGValueRange<FTransform> PointTransforms = PointData->GetConstTransformValueRange();
	for (const FTransform& Point : PointTransforms)
	{
		if (Point.GetLocation() != InTransform.GetLocation() && Neighbours.Num() < MaxPoints
			&& FVector::Dist(Point.GetLocation(), InTransform.GetLocation()) < MaxDist)
		{
			Neighbours.Add(Point);
		}
	}
	
	return Neighbours;
}

void FPVConvertToSeedPointImplementation::SetBaseDirection(const UPCGPointData* PointData, const FPVConvertToSeedPointParams& Params, TArray<TArray<FVector3f>>& DirectionArray, int PrimaryAxis /*= 0*/)
{
	TConstPCGValueRange<FTransform> PointTransforms = PointData->GetConstTransformValueRange();

	float MinDistance = FLT_MAX;
	float MaxDistance = -FLT_MAX;
	
	for (const FTransform& Point : PointTransforms)
	{
		float Distance = FVector::Dist(Point.GetLocation(), FVector::Zero());
		
		MinDistance = FMath::Min(Distance, MinDistance);
		MaxDistance = FMath::Max(Distance, MaxDistance);
	}

	int SeedPointCount = 0;
	for (const FTransform& Point : PointTransforms)
	{
		TArray<FTransform> Neighbours = FindNeighbours(Point, PointData, 200, 5);
		FVector AvoidanceDirection = FVector::ZeroVector;
		
		for (const FTransform& Neighbour : Neighbours)
		{
			FVector NearAvoidanceDirection = (Point.GetLocation() - Neighbour.GetLocation()).GetSafeNormal();
			AvoidanceDirection += NearAvoidanceDirection;
		}

		if (!Neighbours.IsEmpty())
		{
			AvoidanceDirection = AvoidanceDirection / Neighbours.Num();
		}

		FVector ApicalDirection;
		FVector AxillaryDirection;
		FVector BorderAdjustedApicalDirection;
		FVector UpVector;
		FVector CentroidDirection = Point.GetLocation().GetSafeNormal();
		//float RandomSeed = SeedPointCount + 1;
		
		float CenterReduction = FMath::GetMappedRangeValueClamped(FVector2f(0, MaxDistance), FVector2f(0, 1), FVector::Dist(Point.GetLocation(), FVector::Zero()));
		
		if (PrimaryAxis == 0)
		{
			AxillaryDirection = FMath::Lerp(CentroidDirection,AvoidanceDirection,Params.AvoidanceBias).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);
			ApicalDirection = FVector::UpVector;
			UpVector = FVector::CrossProduct(AxillaryDirection,ApicalDirection);

			FVector AxillaryAdjustedApicalDirection = FMath::Lerp(ApicalDirection,AxillaryDirection, Params.Avoidance);
			BorderAdjustedApicalDirection =  FMath::Lerp(ApicalDirection,AxillaryAdjustedApicalDirection,CenterReduction);
			BorderAdjustedApicalDirection =  FMath::Lerp(AxillaryAdjustedApicalDirection,BorderAdjustedApicalDirection,Params.AvoidanceCenterReduction);
		}
		else
		{
			ApicalDirection = FVector::UpVector;
			ApicalDirection =  FMath::Lerp(ApicalDirection,AvoidanceDirection,Params.Avoidance).GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);
			UpVector = FVector::RightVector;
			AxillaryDirection = FVector::CrossProduct(UpVector,ApicalDirection).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

			BorderAdjustedApicalDirection = ApicalDirection;
		}

		//float ApicalRandomAngle = FMath::DegreesToRadians(Params.ApicalRandom);
		//FVector RandomAngleApicalDirection = sample_direction_cone(BorderAdjustedApicalDirection, ApicalRandomAngle,FVector2f(FMath::FRand(RandomSeed)));

		FQuat Dihedral = FQuat::FindBetweenVectors(ApicalDirection, BorderAdjustedApicalDirection);
		AxillaryDirection = Dihedral.RotateVector(AxillaryDirection);
		UpVector = Dihedral.RotateVector(UpVector);

		ApicalDirection = BorderAdjustedApicalDirection;

		DirectionArray.Add(FPVBudDirection{ApicalDirection,AxillaryDirection,ApicalDirection, ApicalDirection, ApicalDirection, UpVector}.GetDataArray());
		SeedPointCount++;
	}

}

void FPVConvertToSeedPointImplementation::FillCollection(FManagedArrayCollection& Collection, const FPVConvertToSeedPointParams& Settings, const UPCGPointData* SeedPoints)
{
	PV::Facades::FPointFacade PointFacade = PV::Facades::FPointFacade(Collection);
	PV::Facades::FBranchFacade BranchFacade = PV::Facades::FBranchFacade(Collection);

	TArray<FVector3f> Points;
	TArray<float> LengthFromRoots;
	TArray<float> PointScales;
	TArray<float> SeedPScales;
	TArray<float> SeedPScaleRatios;
	TArray<TArray<FVector3f>> BudDirections;
	TArray<TArray<float>> BudHormoneLevels;
	TArray<float> PlantGradients;
	TArray<float> LengthFromSeeds;
	TArray<int32> BudNumbers;
	TArray<TArray<float>> BudLightDetected;
	TArray<TArray<int32>> BudDevelopment;
	TArray<TArray<int32>> BudStatus;
	TArray<TArray<float>> BudLateralMeristem;
	TArray<float> NjordPixelIndices;

	TArray<TArray<int32>> AllParents;
	TArray<TArray<int32>> AllChildren;
	TArray<TArray<int32>> AllPoints;
	TArray<int32> BranchNumbers;
	TArray<int32> BranchSourceBudNumbers;
	TArray<int32> BranchHierarchyNumbers;
	TArray<int32> BranchGenerationNumbers;
	TArray<int32> BranchParentNumbers;
	TArray<int32> PlantNumbers;
	
	int PointIndex = 0;
	TConstPCGValueRange<FTransform> PointTransforms = SeedPoints->GetConstTransformValueRange();

	float MinScale = FLT_MAX;
	float MaxScale = -FLT_MAX;
	for (const FTransform& Point : PointTransforms)
	{
		MinScale = FMath::Min(Point.GetScale3D().X, MinScale);
		MaxScale = FMath::Max(Point.GetScale3D().X, MaxScale);
	}

	int BudNumber = 1;
	int PlantNumber = 0;
	int BranchNumber = 1;
	for (const FTransform& Point : PointTransforms)
	{ 
		Points.Add(FVector3f(Point.GetLocation()));
		LengthFromRoots.Add(0);
		PointScales.Add(Point.GetScale3D().X);
		SeedPScales.Add(Point.GetScale3D().X);
		float PScaleRatio = FMath::GetMappedRangeValueClamped(FVector2f(0, MaxScale), FVector2f(0, 1), Point.GetScale3D().X);
		SeedPScaleRatios.Add(PScaleRatio);
		LengthFromSeeds.Add(0);
		BudNumbers.Add(BudNumber);

		FPVBudHormoneLevels HormoneLevels;
		FPVBudStatus Status;
		FPVBudLateralMeristem LateralMeristem;
		FPVBudLightDetected LightDetected;
		FPVBudDevelopment Development;
		
		LateralMeristem = {0,1,0,0,0,0,0};
		Status = FPVBudStatus{1,0,0,1,1,0,0,0,0,0};
		HormoneLevels = FPVBudHormoneLevels{1,1, 0,0,0 };
		LightDetected = {0,0,0,0};
		Development = { 1,0,0,0,0,1};
		
		BudHormoneLevels.Add(HormoneLevels.GetDataArray());
		BudDevelopment.Add(Development.GetDataArray());
		BudLightDetected.Add(LightDetected.GetDataArray());
		BudStatus.Add(Status.GetDataArray());
		BudLateralMeristem.Add(LateralMeristem.GetDataArray());
		
		AllParents.Add({0});
		AllPoints.Add({ Points.Num() - 1 });
		BranchNumbers.Add(BudNumber);
		PlantNumbers.Add(PlantNumber);
		BranchSourceBudNumbers.Add(BudNumber);
		BranchParentNumbers.Add(0);
		BranchGenerationNumbers.Add(1);
		BranchHierarchyNumbers.Add(1);
		
		BudNumber++;
		PlantNumber++;
		BranchNumber++;
	}

	SetBaseDirection(SeedPoints, Settings, BudDirections);

	PointFacade.AddElements(Points.Num());
	PointFacade.FillPositions(Points);
	PointFacade.FillLFR(LengthFromRoots);
	PointFacade.FillPointScales(PointScales);
	PointFacade.FillBudDirections(BudDirections);
	PointFacade.FillBudStatus(BudStatus);
	PointFacade.FillBudLateralMeristem(BudLateralMeristem);
	PointFacade.FillBudHormoneLevels(BudHormoneLevels);
	PointFacade.FillLengthFromSeeds(LengthFromSeeds);
	PointFacade.FillBudNumbers(BudNumbers);
	PointFacade.FillBudLightDetected(BudLightDetected);
	PointFacade.FillBudDevelopment(BudDevelopment);
	PointFacade.FillSeedPScale(SeedPScales);
	PointFacade.FillSeedPScaleRatio(SeedPScaleRatios);

	BranchFacade.AddElements(AllParents.Num());
	BranchFacade.FillParents(AllParents);
	BranchFacade.FillChildren(AllChildren);
	BranchFacade.FillBranchPoints(AllPoints);
	BranchFacade.FillBranchNumbers(BranchNumbers);
	BranchFacade.FillBranchSourceBudNumber(BranchSourceBudNumbers);
	BranchFacade.FillBranchHierarchyNumber(BranchHierarchyNumbers);
	BranchFacade.FillBranchParentNumbers(BranchParentNumbers);
	BranchFacade.FillPlantNumbers(PlantNumbers);

	check(PointFacade.IsValid());
	check(BranchFacade.IsValid());
}

#undef LOCTEXT_NAMESPACE
