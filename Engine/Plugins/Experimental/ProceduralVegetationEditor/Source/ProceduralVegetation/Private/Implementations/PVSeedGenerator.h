// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Data/PCGPointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "PVSeedGenerator.generated.h"

USTRUCT()
struct  FPVSeedPoint
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Seed")
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Seed")
	FVector ApicalDirection = FVector::UpVector;

	UPROPERTY(EditAnywhere, Category = "Seed")
	FVector AxillaryDirection = FVector::ForwardVector;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 BranchNumber = 1;

	UPROPERTY(EditAnywhere, Category = "Seed")
	float LengthFromRoot = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 PlantNumber = 1;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedBranchParentNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedBranchSourceBudNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedGeneration = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedMaxBranchNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedMaxBudNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedPlantNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	float SeedPScale = 0.1f;

	UPROPERTY(EditAnywhere, Category = "Seed")
	float SeedPScaleRatio = 1.0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	uint32 SeedType = 0;

	UPROPERTY(EditAnywhere, Category = "Seed")
	FVector UpVector = FVector::RightVector;
};

UCLASS()
class UPVSeedGenerator : public UObject
{
	GENERATED_BODY()

public:
	const FPVSeedPoint& GetSeedPoint() const { return SeedPoint; }

private:
	UPROPERTY(EditAnywhere, Category = "Seed")
	FPVSeedPoint SeedPoint;
};

USTRUCT()
struct FPVConvertToSeedPointParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = AvoidanceSettings , meta = (UIMin = "0.0", UIMax = "1.0", Tooltip="Strength of the avoidance bend applied to each seed's initial growth direction.\n\n0 = seeds grow straight up regardless of position. Higher values bend each seed's initial direction away from a reference. Use AvoidanceBias to choose whether the reference is the cluster center or nearby seeds."))
	float Avoidance = 0;

	UPROPERTY(EditAnywhere, Category = AvoidanceSettings, meta = (UIMin = "0.0", UIMax = "1.0", Tooltip="Mix between avoiding the cluster center (0) and avoiding neighbors (1).\n\n0 = seeds bend away from the cluster center. 1 = seeds bend away from nearby seeds. 0.5 = equal mix."))
	float AvoidanceBias = 0;

	UPROPERTY(EditAnywhere, Category = AvoidanceSettings, meta = (UIMin = "0.0", UIMax = "1.0", Tooltip="Taper the avoidance bend so central seeds bend less than outer seeds.\n\n0 = all seeds bend equally regardless of position. 1 = central seeds grow closer to straight up, outer seeds bend fully toward the avoidance direction. Tapers smoothly from 0 at the cluster center to 1 at the outermost seed."))
	float AvoidanceCenterReduction = 0;

	UPROPERTY(VisibleAnywhere, Category = AvoidanceSettings, meta = (UIMin = "0.0", UIMax = "360.0", Tooltip="Auto-computed random apical direction range applied to each seed (degrees)."))
	float ApicalRandom = 0;

	UPROPERTY(VisibleAnywhere, Category = AvoidanceSettings, meta = (UIMin = "0.0", UIMax = "360.0", Tooltip="Auto-computed random axillary direction range applied to each seed (degrees)."))
	float AxillaryRandom = 0;
};

struct FPVConvertToSeedPointImplementation
{
	static TArray<FTransform> FindNeighbours(const FTransform& InTransform, const UPCGPointData* PointData, float MaxDist, int MaxPoints);
	
	static void SetBaseDirection(const UPCGPointData* PointData, const FPVConvertToSeedPointParams& Params, TArray<TArray<FVector3f>>& DirectionArray, int PrimaryAxis = 0);
	
	static void FillCollection(FManagedArrayCollection& Collection, const FPVConvertToSeedPointParams& Params, const UPCGPointData* SeedPoints);
};
