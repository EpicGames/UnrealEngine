// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSimplifyPlantSkeleton.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "Polyline3.h"
#include "Implementations/PVCarve.h"

void PV::SimplifyPlantSkeleton(FManagedArrayCollection& InOutCollection, float SimplificationAmount)
{
	using namespace PV::PlantTraversalHelper;
	using namespace UE::Geometry;

	const PV::FPointPositionAttributeConstView PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(InOutCollection);
	const PV::FBranchChildrenAttributeConstView BranchChildrenAttribute = PV::FBranchChildrenAttribute::FindAttribute(InOutCollection);
	const PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(InOutCollection);
	const PV::FBranchNumberAttributeConstView BranchNumberAttribute = PV::FBranchNumberAttribute::FindAttribute(InOutCollection);
	const PV::FBranchPointsAttributeConstView BranchPointsAttribute = PV::FBranchPointsAttribute::FindAttribute(InOutCollection);
	const PV::FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::FindAttribute(InOutCollection);

	if (!ValidateAttributeCollection(
		PointPositionAttribute,
		BranchChildrenAttribute,
		BranchParentNumberAttribute,
		BranchNumberAttribute,
		BranchPointsAttribute,
		PointLengthFromRootAttribute))
	{
		return;
	}

	SimplificationAmount = FMath::Clamp(SimplificationAmount, 0.f, 1.f);

	if (SimplificationAmount <= 0.0f)
	{
		return;
	}

	const FBranchNumberToIndexTable BranchNumberToIndex(BranchNumberAttribute);
	const auto GetNumAxillaryBranches = [&](int32 BranchIndex, int32 BranchPointIndex)
	{
		return GetBranchPointChildBranchIndices(
			BranchChildrenAttribute,
			BranchParentNumberAttribute,
			BranchNumberToIndex,
			BranchPointsAttribute,
			BranchIndex,
			BranchPointIndex
		).Num();
	};

	struct FBranchSegment
	{
		int32 BranchIndex;
		int32 StartIndex;
		int32 EndIndex;
	};
	TArray<FBranchSegment> Segments;
	Segments.Reserve(PointPositionAttribute.Num());

	for (int32 BranchIndex = 0; BranchIndex < BranchPointsAttribute.Num(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttribute[BranchIndex];
		int32 SegmentStartIdx = 0;
		for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
		{
			if (GetNumAxillaryBranches(BranchIndex, BranchPointIndex) > 0)
			{
				Segments.Add({BranchIndex, SegmentStartIdx, BranchPointIndex});
				SegmentStartIdx = BranchPointIndex;
			}
		}
		if (BranchPoints.Num() > 0)
		{
			Segments.Add({BranchIndex, SegmentStartIdx, BranchPoints.Num() - 1});
		}
	}

	// Pre-pass: find the maximum perpendicular deviation of any interior point from the
	// line connecting its segment's endpoints, across all segments. This is the exact
	// tolerance at which Douglas-Peucker would remove all interior points from every
	// segment, making it the correct normalisation basis for SimplificationAmount = 1.
	float GlobalMaxDeviationSq = 0.0f;
	for (const FBranchSegment& Segment : Segments)
	{
		if (Segment.EndIndex - Segment.StartIndex < 2)
		{
			continue;
		}
		const TArray<int32>& BranchPoints = BranchPointsAttribute[Segment.BranchIndex];
		const FSegment3f SegLine(
			PointPositionAttribute[BranchPoints[Segment.StartIndex]],
			PointPositionAttribute[BranchPoints[Segment.EndIndex]]);
		for (int32 i = Segment.StartIndex + 1; i < Segment.EndIndex; ++i)
		{
			GlobalMaxDeviationSq = FMath::Max(
				GlobalMaxDeviationSq,
				SegLine.DistanceSquared(PointPositionAttribute[BranchPoints[i]]));
		}
	}

	const float LineDeviationTolerance = FMath::IsNearlyZero(GlobalMaxDeviationSq)
		? UE_KINDA_SMALL_NUMBER
		: SimplificationAmount * FMath::Sqrt(GlobalMaxDeviationSq);

	TArray<bool> PointsToRemove;
	PointsToRemove.Init(false, PointPositionAttribute.Num());

	TArray<bool> BranchesToRemove;
	BranchesToRemove.Init(false, BranchNumberAttribute.Num());

	TArray<int32> NewVertexIndices;
	UE::Geometry::FPolyline3f Polyline;
	for (const FBranchSegment& Segment : Segments)
	{
		const TArray<int32>& BranchPoints = BranchPointsAttribute[Segment.BranchIndex];
		Polyline.Clear(Segment.EndIndex - Segment.StartIndex + 1);
		for (int32 i = Segment.StartIndex; i <= Segment.EndIndex; ++i)
		{
			Polyline.AppendVertex(PointPositionAttribute[BranchPoints[i]]);
		}
		const float ClusterTolerance = 0.001f;
		Polyline.Simplify(ClusterTolerance, LineDeviationTolerance, NewVertexIndices);
		for (int32 i = 1; i < NewVertexIndices.Num(); ++i)
		{
			const int32 PrevBranchPointIndex = Segment.StartIndex + NewVertexIndices[i - 1];
			const int32 ThisBranchPointIndex = Segment.StartIndex + NewVertexIndices[i];
			for (int32 BranchPointIndex = PrevBranchPointIndex + 1; BranchPointIndex < ThisBranchPointIndex; ++BranchPointIndex)
			{
				PointsToRemove[BranchPoints[BranchPointIndex]] = true;
			}
		}
	}

	const FManagedArrayCollection SourceCollection = InOutCollection;
	TArray<bool> FoliageInstancesToRemove;
	FPVCarve::RemoveEntriesAndRecomputeAttributes(
		InOutCollection,
		SourceCollection,
		PointsToRemove,
		BranchesToRemove,
		FoliageInstancesToRemove
	);
}
