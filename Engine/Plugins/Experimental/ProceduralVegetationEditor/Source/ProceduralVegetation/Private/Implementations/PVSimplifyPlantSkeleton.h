// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace PV
{
	extern void SimplifyPlantSkeleton(FManagedArrayCollection& InOutCollection, float SimplificationAmount);
};
