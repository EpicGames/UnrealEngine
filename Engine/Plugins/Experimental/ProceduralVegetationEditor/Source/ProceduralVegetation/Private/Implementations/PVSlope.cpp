// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSlope.h"
#include "Helpers/PVTransformHelper.h"
#include "Helpers/PVPlantTraversalHelper.h"

namespace PVSlope
{
	FQuat4f CalculateBranchRotation(float SlopeDirection, float SlopeAngle)
	{
		const float SlopeDirectionInRadians = FMath::DegreesToRadians(SlopeDirection);
		const float SlopeAngleInRadians = FMath::DegreesToRadians(SlopeAngle);

		const FQuat4f YawRotation = FQuat4f(FVector3f::UpVector, SlopeDirectionInRadians);
		const FQuat4f LeanRotation = FQuat4f(FVector3f::RightVector, SlopeAngleInRadians);
		return YawRotation * LeanRotation;
	}

	float CalculateTreeHeight(
		PV::FBranchChildrenAttributeConstView BranchChildrenAttribute,
		PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute,
		PV::FBranchNumberAttributeConstView BranchNumberAttribute,
		PV::FBranchPointsAttributeConstView BranchPointsAttribute,
		PV::FPointPositionAttributeConstView PointPositionAttribute,
		int32 TrunkIndex)
	{
		using namespace PV::PlantTraversalHelper;

		const TArray<int32>& CurrentBranchPoints = BranchPointsAttribute[TrunkIndex];
		if (CurrentBranchPoints.Num() == 0)
		{
			return 0;
		}

		const FVector3f& RootPosition = PointPositionAttribute[CurrentBranchPoints[0]];

		float BranchLengthSquared = 0;
		RecursiveWalkBranchPoints(
			BranchChildrenAttribute,
			BranchParentNumberAttribute,
			BranchNumberAttribute,
			BranchPointsAttribute,
			TrunkIndex,
			[&](const FRecursiveWalkBranchPointsParams& Params)->EForEachResult
			{
				const FVector3f& PointPosition = PointPositionAttribute[Params.PointIndex];
				const float PointDistSquared = (PointPosition - RootPosition).SizeSquared();
				if (PointDistSquared > BranchLengthSquared)
				{
					BranchLengthSquared = PointDistSquared;
				}
				return EForEachResult::Continue;
			}
		);

		return BranchLengthSquared > 0 ? FMath::Sqrt(BranchLengthSquared) : 0.f;
	}
}

void FPVSlope::ApplySlope(const FPVSlopeParams& InSlopeParams, FManagedArrayCollection& OutCollection)
{
	using namespace PV::Transform;

	const FRotateBranchPointsAttributeCollection Attributes(OutCollection);
	if (!Attributes.IsValid())
	{
		return;
	}

	for (const int32 TrunkIndex : PV::PlantTraversalHelper::GetTrunkIndices(Attributes.BranchParentNumber))
	{
		const TArray<int32>& CurrentBranchPoints = Attributes.BranchPoints[TrunkIndex];
		if (CurrentBranchPoints.Num() == 0)
		{
			continue;
		}

		const float TreeHeight = PVSlope::CalculateTreeHeight(
			Attributes.BranchChildren,
			Attributes.BranchParentNumber,
			Attributes.BranchNumber,
			Attributes.BranchPoints,
			Attributes.PointPosition,
			TrunkIndex
		);
		const FVector3f TrunkPosition = Attributes.PointPosition[CurrentBranchPoints[0]];
		const FVector3f TrunkPivot = InSlopeParams.TrunkPivotPoint == EPVSlopeTrunkPivotPoint::Origin ? FVector3f::ZeroVector : TrunkPosition;
		const auto CalcRotationFunction = [&](const FRotateBranchPointParams& Params, FRotateBranchPointResult& Result)
		{
			const FVector3f Position = Attributes.PointPosition[Params.PointIndex];
			const float TreeAngleAlpha = 1.0 - (TreeHeight > 0 ? ((Position - TrunkPosition).Size() / TreeHeight) : 0);
			const float AngleStrength = FMath::Clamp(FMath::Pow(TreeAngleAlpha, InSlopeParams.BendStrength), 0.0, 1.0);

			Result.Rotation = PVSlope::CalculateBranchRotation(InSlopeParams.SlopeDirection, InSlopeParams.SlopeAngle * AngleStrength);
			Result.bAbsoluteRotation = true;
		};

		RecursiveRotateBranchPoints(Attributes, TrunkIndex, CalcRotationFunction, TrunkPivot);
	}
}
