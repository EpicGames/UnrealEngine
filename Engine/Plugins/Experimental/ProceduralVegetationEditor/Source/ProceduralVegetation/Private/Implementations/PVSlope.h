// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "PVSlope.generated.h"

namespace PV::Facades
{
	class FPointFacade;
	class FBranchFacade;
}

UENUM()
enum class EPVSlopeTrunkPivotPoint
{
	Origin,
	Trunk
};

USTRUCT()
struct FPVSlopeParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Slope", meta = (Units = Degrees, ClampMin = -90.0f, ClampMax = 90.0f, UIMin = -90.0f, UIMax = 90.0f, Tooltip="Slope angle in degrees (positive = uphill, negative = downhill).\n\nThe angle of the ground the plant is growing on. 0 = flat. Positive = sloping up in the slope direction. Negative = sloping down."))
	float SlopeAngle = 0;

	UPROPERTY(EditAnywhere, Category = "Slope", meta = (Units = Degrees, ClampMin = -180.0f, ClampMax = 180.0f, UIMin = -180.0f, UIMax = 180.0f, Tooltip="Compass direction of the slope's downhill side.\n\nHeading angle of the slope's downhill direction in degrees. 0 = north, 90 = east, -90 = west, ±180 = south."))
	float SlopeDirection = 0;

	UPROPERTY(EditAnywhere, Category = "Slope", meta = (ClampMin = 0.0f, ClampMax = 100.0f, UIMin = 0.0f, UIMax = 10.0f, Tooltip="How strongly the plant bends back toward vertical as it grows.\n\n0 = no bending (plant grows perpendicular to slope and stays angled). Higher values = aggressive uprighting."))
	float BendStrength = 2.0;

	UPROPERTY(EditAnywhere, Category = "Slope", meta=(Tooltip="Rotate around world origin or plant base (useful with multi seed plants)\n\n`Origin` = Pivot the ground rotation around the world origin. `Trunk` = Pivot the ground rotation around the plant base (does not move plant base)."))
	EPVSlopeTrunkPivotPoint TrunkPivotPoint = EPVSlopeTrunkPivotPoint::Origin;
};

struct FPVSlope
{
	static void ApplySlope(const FPVSlopeParams& InSlopeParams, FManagedArrayCollection& OutCollection);
};
