// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVTrunkTextureSetup.h"
#include "ImageUtils.h"
#include "PVTrunkTextureSetupCS.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "TextureResource.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Helpers/PVUtilities.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "UObject/SavePackage.h"
#include "Params//PVTrunkTextureSetupParams.h"

#define LOCTEXT_NAMESPACE "PVTrunkTextureSetup"

IMPLEMENT_GLOBAL_SHADER(FPVTrunkTextureSetupDilationCS, "/Plugins/Experimental/ProceduralVegetationEditor/PVTrunkTextureSetupCS.usf", "DilationCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FPVTrunkTextureSetupScaleCS, "/Plugins/Experimental/ProceduralVegetationEditor/PVTrunkTextureSetupCS.usf", "ScaleCS", SF_Compute);

TArray<FString> FPVTrunkTextureSetupImplementation::GetChannels(const TArray<FPVTextureGenerationParams>& Array)
{
	TArray<FString> Channels;

	for (auto Generation : Array)
	{
		for (auto Channel : Generation.GetChannelNames())
		{
			Channels.AddUnique(Channel);
		}
	}
	return Channels;
}

UTexture* FPVTrunkTextureSetupImplementation::CreateTexture(int Width, int Height, bool bCanCreateUAV, FName Name)
{
	UTextureRenderTarget2D* RenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), Name);
	RenderTargetTexture->ClearColor = FColor::Black;
	RenderTargetTexture->bCanCreateUAV = bCanCreateUAV;
	RenderTargetTexture->bSupportsUAV = bCanCreateUAV;
	RenderTargetTexture->InitAutoFormat(Width, Height);
	RenderTargetTexture->UpdateResourceImmediate(true);
	
	return RenderTargetTexture;
}

TMap<FString, TObjectPtr<UTexture>> FPVTrunkTextureSetupImplementation::CreateTextures(int32 Resolution,const TArray<FString>& ChannelNames, const FString& AssetNamePrefix, bool bVirtualTextureStreaming)
{
	TMap<FString, TObjectPtr<UTexture>> TextureSet;
	for (const auto& ChannelName : ChannelNames)
	{
		FString Name = AssetNamePrefix + ChannelName;
		UTexture* Texture = CreateTexture(Resolution, Resolution, true, FName(Name));
		Texture->VirtualTextureStreaming = bVirtualTextureStreaming;
		TextureSet.Add(ChannelName, Texture);
	}
	return TextureSet;
}

FRHITexture* FPVTrunkTextureSetupImplementation::GetRHITexture(UTexture* Texture)
{
	if (UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(Texture))
	{
		const FTextureRenderTargetResource* Resource = RenderTarget->GetRenderTargetResource();//>GetResource();
		if (!Resource)
		{
			return nullptr;
		}
	
		return Resource->GetRenderTargetTexture();
	}
	
	if (Texture && Texture->GetResource())
	{
		const FTextureResource* Resource = Texture->GetResource();
		
		return Resource->GetTexture2DRHI();
	}
	
	return nullptr;
}

FRDGTextureSRVRef FPVTrunkTextureSetupImplementation::GetTextureSRVRef(FRDGBuilder& GraphBuilder,UTexture* InputTexture,const TCHAR* Name)
{
	FRHITexture* RHITexture = GetRHITexture(InputTexture);
	TRefCountPtr<IPooledRenderTarget> InputRenderTarget = CreateRenderTarget(RHITexture, Name);
	FRDGTextureRef InputTextureRef = GraphBuilder.RegisterExternalTexture(InputRenderTarget);
	
	FRDGTextureSRVRef InputTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(InputTextureRef, 0));
	return InputTextureSRV;
}

FRDGTextureUAVRef FPVTrunkTextureSetupImplementation::GetTextureUAVRef(FRDGBuilder& GraphBuilder,UTexture* Texture,const TCHAR* Name)
{
	FRHITexture* RHITexture = GetRHITexture(Texture);
	
	TRefCountPtr<IPooledRenderTarget> RenderTarget = CreateRenderTarget(RHITexture, Name);
	
	FRDGTextureRef TextureRDGRef = GraphBuilder.RegisterExternalTexture(RenderTarget);
	FRDGTextureUAVDesc Descriptor(TextureRDGRef);
	Descriptor.MipLevel = 0;
	
	FRDGTextureUAVRef TextureUAV = GraphBuilder.CreateUAV(Descriptor);
	return TextureUAV;
}

void FPVTrunkTextureSetupImplementation::DilationPass(FRDGBuilder& GraphBuilder, float DilationFactor, float TileX, float TileY, UTexture* InputTexture, UTexture* OutputTexture)
{
	float ScaleX = 1.0 - (DilationFactor * 2.0);
	
	FRDGTextureSRVRef InputTextureSRV = GetTextureSRVRef(GraphBuilder, InputTexture, TEXT("PVDilationInputTexture"));
	FRDGTextureUAVRef RWOutputTextureUAV = GetTextureUAVRef(GraphBuilder, OutputTexture, TEXT("PVDilationOutputTexture"));
		
	FPVTrunkTextureSetupDilationCS::FParameters* Params = GraphBuilder.AllocParameters<FPVTrunkTextureSetupDilationCS::FParameters>();
	Params->OffsetX = DilationFactor;
	Params->ScaleX = ScaleX;
	Params->TileX = TileX;
	Params->TileY = TileY;
	Params->Source = InputTextureSRV;
	Params->SourceSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap>::GetRHI();
	Params->Target = RWOutputTextureUAV;
	Params->TargetSize = FIntPoint(OutputTexture->GetSurfaceWidth(), OutputTexture->GetSurfaceHeight());
	TShaderMapRef<FPVTrunkTextureSetupDilationCS> PVTrunkTextureSetupDilationCS(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(Params->TargetSize, FIntPoint(8,8));
	//FIntVector DispatchCount(FMath::DivideAndRoundUp(Params->TargetSize.X, 8),FMath::DivideAndRoundUp(Params->TargetSize.Y, 8),1);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("PVDilation"), PVTrunkTextureSetupDilationCS, Params, GroupCount);
	
}

void FPVTrunkTextureSetupImplementation::ScaleAndCombineTexturePass(FRDGBuilder& GraphBuilder, UTexture* TargetTexture, UTexture* InputTexture, const float WidthRatio, const float OffsetX, const FPVTextureGenerationParams& Generation)
{
	FRDGTextureSRVRef InputTextureSRV = GetTextureSRVRef(GraphBuilder, InputTexture, TEXT("PVDilationInputTexture"));
	FRDGTextureUAVRef RWOutputTextureUAV = GetTextureUAVRef(GraphBuilder, Cast<UTextureRenderTarget2D>(TargetTexture), TEXT("PVTrimSheetTexture"));
		
	FPVTrunkTextureSetupScaleCS::FParameters* Params = GraphBuilder.AllocParameters<FPVTrunkTextureSetupScaleCS::FParameters>();
	Params->OffsetX = OffsetX;
	Params->ScaleX = Generation.XScale / WidthRatio;
	Params->TileX = Generation.BaseTileX;
	Params->TileY = Generation.BaseTileY;
	Params->Source = InputTextureSRV;
	Params->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	Params->Target = RWOutputTextureUAV;
	Params->TargetSize = FIntPoint(TargetTexture->GetSurfaceWidth(), TargetTexture->GetSurfaceHeight());
	TShaderMapRef<FPVTrunkTextureSetupScaleCS> PVTrunkTextureSetupScaleCS(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(Params->TargetSize, FIntPoint(8,8));
	//FIntVector DispatchCount(FMath::DivideAndRoundUp(Params->TargetSize.X, 8),FMath::DivideAndRoundUp(Params->TargetSize.Y, 8),1);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("PVScale"), PVTrunkTextureSetupScaleCS, Params, GroupCount);
}

void FPVTrunkTextureSetupImplementation::AddChannelTextureToTextureSet(const float InOffsetX, const FPVTextureGenerationParams& Generation, const FPVTextureChannelParams& Channel, UTexture* Texture)
{
	UTexture* ChannelTexture = Channel.Texture;
	UTexture* DilatedTexture = CreateTexture(ChannelTexture->GetSurfaceWidth(), ChannelTexture->GetSurfaceHeight(), true, NAME_None);
	
	ENQUEUE_RENDER_COMMAND(RunPVScaleAndCombinePass)
	([ &Generation, &InOffsetX, ChannelTexture, DilatedTexture, Texture](FRHICommandListImmediate& RHICmdList)
	{
		FRDGBuilder GraphBuilder(RHICmdList);
		float WidthRatio = ChannelTexture->GetSurfaceWidth() > 0 ? static_cast<float>(ChannelTexture->GetSurfaceHeight() / static_cast<float>(ChannelTexture->GetSurfaceWidth())) : 1.0;
		DilationPass(GraphBuilder, Generation.Dilation, Generation.BaseTileX, Generation.BaseTileY, ChannelTexture, DilatedTexture);
		ScaleAndCombineTexturePass(GraphBuilder, Texture, DilatedTexture, WidthRatio, InOffsetX, Generation);
		GraphBuilder.Execute();
	});

	FlushRenderingCommands();
}

void FPVTrunkTextureSetupImplementation::AddGenerationIntoChannel(const FString& ChannelName, UTexture* Texture, const FPVTrunkTextureSetupParams& Params)
{
	float XOffset = 0.0f;
	for (auto Generation : Params.Generations)
	{
		//Get the Channel with name
		if (const FPVTextureChannelParams* Channel = Generation.GetChannel(ChannelName))
		{
			if (Channel->Texture != nullptr)
			{
				AddChannelTextureToTextureSet(XOffset, Generation, *Channel, Texture);
				XOffset += Generation.XScale / Channel->WidthRatio;
			}
		}
	}
}

void FPVTrunkTextureSetupImplementation::AddGenerationIntoChannel(const TMap<FString, TObjectPtr<UTexture>>& Map,
	const FPVTrunkTextureSetupParams& Params)
{
	for (auto Pair : Map)
	{
		AddGenerationIntoChannel(Pair.Key, Pair.Value, Params);
	}
}


FString FPVTrunkTextureSetupImplementation::NormalizeString(FString InString)
{
	InString = InString.ToLower();
	InString.ReplaceInline(TEXT(" "), TEXT(""));
	InString.ReplaceInline(TEXT("_"), TEXT(""));
	InString.ReplaceInline(TEXT("-"), TEXT(""));
	return InString;
}

EPVTextureChannel FPVTrunkTextureSetupImplementation::GetTextureChannelFromString(const FString& InString)
{
	FString NormalizedString = NormalizeString(InString);

	static TArray<FString> BaseColorNames = {"basecolor", "albedo", "alb", "diffuse", "col", "color"};
	static TArray<FString> AONames = {"ao", "ambientocclusion", "occlusion", "occ"};
	static TArray<FString> NormalNames = {"normal", "norm", "nor"};
	static TArray<FString> DisplacementNames = {"displacement", "disp", "height"};
	static TArray<FString> BumpNames = {"bump", "bum"};
	static TArray<FString> CavityNames = {"cavity", "cav"};
	static TArray<FString> GlossNames = {"gloss", "gls"};
	static TArray<FString> RoughnessNames = {"roughness", "rough"};
	static TArray<FString> SpecularNames = {"specular", "spec"};
	
	if (BaseColorNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::BaseColor;
	}
	else if (AONames.Contains(NormalizedString))
	{
		return EPVTextureChannel::AO;
	}
	else if (NormalNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Normal;
	}
	else if (DisplacementNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Displacement;
	}
	else if (BumpNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Bump;
	}
	else if (CavityNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Cavity;
	}
	else if (GlossNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Gloss;
	}
	else if (RoughnessNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Roughness;
	}
	else if (SpecularNames.Contains(NormalizedString))
	{
		return EPVTextureChannel::Specular;
	}

	return EPVTextureChannel::Custom;
}

TObjectPtr<UTexture> FPVTrunkTextureSetupImplementation::GetChannelTextureForMaterialParam(FPVTrunkTextureSetupInfo& TrunkTextureSetupInfo, const FString MaterialParamName)
{
	EPVTextureChannel Channel = GetTextureChannelFromString(MaterialParamName);
	FString ChannelName = StaticEnum<EPVTextureChannel>()->GetNameStringByValue((int64)Channel);

	if (TrunkTextureSetupInfo.Channels.Contains(ChannelName))
	{
		return TrunkTextureSetupInfo.Channels[ChannelName];
	}
	else if (TrunkTextureSetupInfo.Channels.Contains(MaterialParamName))
	{
		return TrunkTextureSetupInfo.Channels[MaterialParamName];
	}

	return nullptr;
}

void FPVTrunkTextureSetupImplementation::SetupMaterial(TObjectPtr<UMaterialInterface> Material, FPVTrunkTextureSetupInfo& TrunkTextureSetupInfo)
{
	TObjectPtr<UMaterialInstanceConstant> MaterialInstance = Cast<UMaterialInstanceConstant>(Material);
	if (MaterialInstance)
	{
#if WITH_EDITOR
		MaterialInstance->Modify();
		TArray<FMaterialParameterInfo> Infos;
		TArray<FGuid> Guids;
		MaterialInstance->GetAllTextureParameterInfo(Infos, Guids);

		for (auto Info : Infos)
		{
			UTexture* Texture = GetChannelTextureForMaterialParam(TrunkTextureSetupInfo, Info.Name.ToString());
			if (Texture)
			{
				MaterialInstance->SetTextureParameterValueEditorOnly(Info, Texture);
			}
		}

		bool bDirty = MaterialInstance->MarkPackageDirty();
		FPropertyChangedEvent EmptyPropertyUpdateStruct(NULL);
		MaterialInstance->PostEditChangeProperty(EmptyPropertyUpdateStruct);
#endif
	}
	
	TrunkTextureSetupInfo.Material = Material;
}

void FPVTrunkTextureSetupImplementation::CreateTrimSheetTexture( FPVTrunkTextureSetupParams& Params, FPVTrunkTextureSetupInfo& TrunkTextureSetupInfo)
{
	TrunkTextureSetupInfo.Channels.Empty();
	TrunkTextureSetupInfo.GenerationUVs.Empty();
	//Create one texture per Channel with given Resolution
	int32 Resolution = static_cast<int32>(Params.Resolution);
	TArray<FString> Channels = GetChannels(Params.Generations);

	TMap<FString, TObjectPtr<UTexture>> TextureSet = CreateTextures(Resolution, Channels, Params.AssetNamePrefix, Params.bVirtualTexture);
	AddGenerationIntoChannel(TextureSet, Params);

	float PreviousOffsetX = 0.0f;
	for (auto Generation : Params.Generations)
	{
		float Scale = Generation.XScale / Params.GetWidthRatio(Generation);
		FPVGenerationUVRange UV;
		float DilationFactor = (Generation.Dilation * Scale);
		UV.OffsetXStart = PreviousOffsetX + DilationFactor;
		UV.OffsetXEnd = PreviousOffsetX + (Scale - DilationFactor);
		UV.DilationFactor = DilationFactor;
		TrunkTextureSetupInfo.GenerationUVs.Add(UV);
		PreviousOffsetX += Scale;
	}
	
	TrunkTextureSetupInfo.Channels = MoveTemp(TextureSet);
	TrunkTextureSetupInfo.Resolution = Resolution;

	SetupMaterial(Params.Material, TrunkTextureSetupInfo);

	TrunkTextureSetupInfo.PreviewChannelName = StaticEnum<EPVTextureChannel>()->GetNameStringByValue((int64)Params.PreviewChannel);
}

UTexture2D* FPVTrunkTextureSetupImplementation::ConvertRenderTargetToTexture2D(UTextureRenderTarget2D* InRenderTarget, const FString& Path, FString& OutError)
{
	if (!InRenderTarget)
	{
		OutError = "Cannot convert a null render target to a Texture2D.";
		return nullptr;
	}
	
#if WITH_EDITOR
	FString AssetName = InRenderTarget->GetName();
	FString PackageName = Path / AssetName;
	FString ObjectPath = PackageName + TEXT(".") + AssetName;

	if (!PV::Utilities::ValidateAssetPathAndName(AssetName, Path, UTexture2D::StaticClass(), OutError))
	{
		return nullptr;
	}

	UTexture2D* Texture = LoadObject<UTexture2D>(nullptr, ObjectPath);
	UPackage* Package = nullptr;

	if (Texture)
	{
		Package = Texture->GetPackage();
	}
	else
	{
		Package = CreatePackage(*PackageName);
		Texture = NewObject<UTexture2D>(Package, FName(AssetName), RF_Public | RF_Standalone);
		FAssetRegistryModule::AssetCreated(Texture);
	}
	
	if (!InRenderTarget->UpdateTexture(Texture))
	{
		return Texture;
	}
	
	Texture->Modify();
	Texture->VirtualTextureStreaming = InRenderTarget->VirtualTextureStreaming;
	bool bDirty = Texture->MarkPackageDirty();
	Texture->PostEditChange();
	Texture->UpdateResource();
	
	const FString PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
	FSavePackageArgs SavePackageArgs;
	SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;

	const bool bSave = UPackage::SavePackage(Package, Texture, *PackageFileName, SavePackageArgs);

	return Texture;
#else
	return nullptr;
#endif
}

#undef LOCTEXT_NAMESPACE

