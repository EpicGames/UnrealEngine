// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphBuilder.h"
#include "TextureResource.h"
#include "DataTypes/PVTrunkTextureSetupData.h"
#include "Params/PVTrunkTextureSetupParams.h"

class UTexture2D;
class UTexture;
class UTextureRenderTarget2D;
class UMaterialInterface;

struct FPVTrunkTextureSetupImplementation
{
	static TArray<FString> GetChannels(const TArray<FPVTextureGenerationParams>& Array);
	static UTexture* CreateTexture(int Width, int Height, bool bCanCreateUAV, FName Name);
	static TMap<FString, TObjectPtr<UTexture>> CreateTextures(int32 Resolution, const TArray<FString>& ChannelNames, const FString& AssetNamePrefix, bool bVirtualTextureStreaming);
	static FRHITexture* GetRHITexture(UTexture* Texture);
	static FRDGTextureSRVRef GetTextureSRVRef(FRDGBuilder& GraphBuilder, UTexture* InputTexture, const TCHAR* Name);
	static FRDGTextureUAVRef GetTextureUAVRef(FRDGBuilder& GraphBuilder, UTexture* Texture, const TCHAR* Name);

	static void DilationPass(FRDGBuilder& GraphBuilder, float DilationFactor, float TileX, float TileY, UTexture* InputTexture, UTexture* OutputTexture);
	static void ScaleAndCombineTexturePass(FRDGBuilder& GraphBuilder, UTexture* TargetTexture, UTexture* InputTexture, const float WidthRatio, const float OffsetX,
	                            const FPVTextureGenerationParams& Generation);
	static void AddChannelTextureToTextureSet(const float InOffsetX, const FPVTextureGenerationParams& Generation, const FPVTextureChannelParams& Channel, UTexture* Texture);
	static void AddGenerationIntoChannel(const FString& ChannelName, UTexture* Texture, const FPVTrunkTextureSetupParams& Params);
	static void AddGenerationIntoChannel(const TMap<FString, TObjectPtr<UTexture>>& Map, const FPVTrunkTextureSetupParams& Params);
	
	static FString NormalizeString(FString InString);
	static EPVTextureChannel GetTextureChannelFromString(const FString& InString);
	static TObjectPtr<UTexture>  GetChannelTextureForMaterialParam(FPVTrunkTextureSetupInfo& TrunkTextureSetupInfo, FString MaterialParamName);
	static void SetupMaterial(TObjectPtr<UMaterialInterface> Material, FPVTrunkTextureSetupInfo& TrunkTextureSetupInfo);
	static void CreateTrimSheetTexture( FPVTrunkTextureSetupParams& Params, FPVTrunkTextureSetupInfo& TrunkTextureSetupInfo);
	static UTexture2D* ConvertRenderTargetToTexture2D(UTextureRenderTarget2D* InRenderTarget, const FString& Path, FString& OutError);
};
