// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

class FPVTrunkTextureSetupDilationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPVTrunkTextureSetupDilationCS);
	SHADER_USE_PARAMETER_STRUCT(FPVTrunkTextureSetupDilationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER(float, OffsetX)
	SHADER_PARAMETER(float, ScaleX)
	SHADER_PARAMETER(float, TileX)
	SHADER_PARAMETER(float, TileY)
	SHADER_PARAMETER(FIntPoint, TargetSize)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, Source)
	SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Target)
	END_SHADER_PARAMETER_STRUCT()
};

class FPVTrunkTextureSetupScaleCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPVTrunkTextureSetupScaleCS);
	SHADER_USE_PARAMETER_STRUCT(FPVTrunkTextureSetupScaleCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER(float, OffsetX)
	SHADER_PARAMETER(float, ScaleX)
	SHADER_PARAMETER(float, TileX)
	SHADER_PARAMETER(float, TileY)
	SHADER_PARAMETER(FIntPoint, TargetSize)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, Source)
	SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Target)
	END_SHADER_PARAMETER_STRUCT()
};


