// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVGrowthDataDiffSettings.h"
 
#include "ProceduralVegetationModule.h"
 
#include "DataTypes/PVGrowthData.h"
#include "Utils/PVAttributes.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "PVCommon.h"
 
#define LOCTEXT_NAMESPACE "PVGrowthDataDiffSettings"

namespace PV::DiffHelpers
{
	struct FGrowthDataHierarchyAttributes
	{
		FPointPositionAttributeConstView PointPosition;
		FBranchPointsAttributeConstView BranchPoints;
		FBranchParentNumberAttributeConstView BranchParentNumber;
		FBranchParentsAttributeConstView BranchParents;
		FBranchNumberAttributeConstView BranchNumber;
		FBranchChildrenAttributeConstView BranchChildren;
		FBranchHierarchyNumberAttributeConstView BranchHierarchyNumber;
		FBranchPlantNumberAttributeConstView BranchPlantNumber;

		FGrowthDataHierarchyAttributes() = default;
		FGrowthDataHierarchyAttributes(const FManagedArrayCollection& InCollection)
			: PointPosition(FPointPositionAttribute::FindAttribute(InCollection))
			, BranchPoints(FBranchPointsAttribute::FindAttribute(InCollection))
			, BranchParentNumber(FBranchParentNumberAttribute::FindAttribute(InCollection))
			, BranchParents(FBranchParentsAttribute::FindAttribute(InCollection))
			, BranchNumber(FBranchNumberAttribute::FindAttribute(InCollection))
			, BranchChildren(FBranchChildrenAttribute::FindAttribute(InCollection))
			, BranchHierarchyNumber(FBranchHierarchyNumberAttribute::FindAttribute(InCollection))
			, BranchPlantNumber(FBranchPlantNumberAttribute::FindAttribute(InCollection))
		{}
	};

	bool HasSameGrowthDataHierarchy(
		const FGrowthDataHierarchyAttributes& AttributesA,
		const FGrowthDataHierarchyAttributes& AttributesB)
	{
		if (AttributesA.PointPosition.IsValid() != AttributesB.PointPosition.IsValid()
			|| AttributesA.BranchPoints.IsValid() != AttributesB.BranchPoints.IsValid()
			|| AttributesA.BranchParentNumber.IsValid() != AttributesB.BranchParentNumber.IsValid()
			|| AttributesA.BranchParents.IsValid() != AttributesB.BranchParents.IsValid()
			|| AttributesA.BranchNumber.IsValid() != AttributesB.BranchNumber.IsValid()
			|| AttributesA.BranchChildren.IsValid() != AttributesB.BranchChildren.IsValid()
			|| AttributesA.BranchHierarchyNumber.IsValid() != AttributesB.BranchHierarchyNumber.IsValid()
			|| AttributesA.BranchPlantNumber.IsValid() != AttributesB.BranchPlantNumber.IsValid())
		{
			return false;
		}

		if (AttributesA.PointPosition.Num() != AttributesB.PointPosition.Num()
			|| AttributesA.BranchPoints.Num() != AttributesB.BranchPoints.Num()
			|| AttributesA.BranchParentNumber.Num() != AttributesB.BranchParentNumber.Num()
			|| AttributesA.BranchParents.Num() != AttributesB.BranchParents.Num()
			|| AttributesA.BranchNumber.Num() != AttributesB.BranchNumber.Num()
			|| AttributesA.BranchChildren.Num() != AttributesB.BranchChildren.Num()
			|| AttributesA.BranchHierarchyNumber.Num() != AttributesB.BranchHierarchyNumber.Num()
			|| AttributesA.BranchPlantNumber.Num() != AttributesB.BranchPlantNumber.Num())
		{
			return false;
		}

		for (int32 i = 0; i < AttributesA.PointPosition.Num(); ++i)
		{
			if (!AttributesA.PointPosition[i].Equals(AttributesB.PointPosition[i]))
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchPoints.Num(); ++i)
		{
			if (AttributesA.BranchPoints[i] != AttributesB.BranchPoints[i])
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchParentNumber.Num(); ++i)
		{
			if (AttributesA.BranchParentNumber[i] != AttributesB.BranchParentNumber[i])
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchParents.Num(); ++i)
		{
			if (AttributesA.BranchParents[i] != AttributesB.BranchParents[i])
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchNumber.Num(); ++i)
		{
			if (AttributesA.BranchNumber[i] != AttributesB.BranchNumber[i])
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchChildren.Num(); ++i)
		{
			if (AttributesA.BranchChildren[i] != AttributesB.BranchChildren[i])
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchHierarchyNumber.Num(); ++i)
		{
			if (AttributesA.BranchHierarchyNumber[i] != AttributesB.BranchHierarchyNumber[i])
			{
				return false;
			}
		}

		for (int32 i = 0; i < AttributesA.BranchPlantNumber.Num(); ++i)
		{
			if (AttributesA.BranchPlantNumber[i] != AttributesB.BranchPlantNumber[i])
			{
				return false;
			}
		}

		return true;
	}

	template<typename TAttributeType>
	TAttributeType DiffAttributeValue(const TAttributeType& A, const TAttributeType& B) { return A - B; }

	template<typename TAttributeType>
	TArray<TAttributeType> DiffAttributeValue(const TArray<TAttributeType>& A, const TArray<TAttributeType>& B)
	{ 
		TArray<TAttributeType> DiffArray;
		if (A.Num() != B.Num())
		{
			return DiffArray;
		}

		DiffArray.SetNum(A.Num());
		for (int32 i = 0; i < A.Num(); ++i)
		{
			DiffArray[i] = DiffAttributeValue<TAttributeType>(A[i], B[i]);
		}

		return DiffArray;
	}

	template<> bool DiffAttributeValue(const bool& A, const bool& B) { return A == B; }
	template<> FTransform DiffAttributeValue<FTransform>(const FTransform& A, const FTransform& B) 
	{
		return FTransform(
			DiffAttributeValue(A.GetRotation(), B.GetRotation()),
			DiffAttributeValue(A.GetLocation(), B.GetLocation()),
			DiffAttributeValue(A.GetScale3D(), B.GetScale3D())
		);
	}
	template<> FTransform3f DiffAttributeValue<FTransform3f>(const FTransform3f& A, const FTransform3f& B) 
	{ 
		return FTransform3f(
			DiffAttributeValue(A.GetRotation(), B.GetRotation()),
			DiffAttributeValue(A.GetLocation(), B.GetLocation()),
			DiffAttributeValue(A.GetScale3D(), B.GetScale3D())
		);
	}
	template<> TSet<int32> DiffAttributeValue<TSet<int32>>(const TSet<int32>& A, const TSet<int32>& B)
	{ 
		return A.Difference(B);
	}

	template<typename TAttributeType>
	void DiffAttributes_Internal(
		const FManagedArrayCollection& CollectionA,
		const FManagedArrayCollection& CollectionB,
		FManagedArrayCollection& ResultCollection,
		const FName& InAttributeName,
		const FName& InGroupName
	)
	{
		if (CollectionA.NumElements(InGroupName) != CollectionB.NumElements(InGroupName))
		{
			return;
		}

		ResultCollection.CopyAttribute(CollectionA, InAttributeName, InGroupName);

		const TManagedArray<TAttributeType>& AttributeA = CollectionA.GetAttribute<TAttributeType>(InAttributeName, InGroupName);
		const TManagedArray<TAttributeType>& AttributeB = CollectionB.GetAttribute<TAttributeType>(InAttributeName, InGroupName);
		TManagedArray<TAttributeType>& ResultAttribute = ResultCollection.ModifyAttribute<TAttributeType>(InAttributeName, InGroupName);

		for (int32 i = 0; i < AttributeA.Num(); ++i)
		{
			ResultAttribute[i] = DiffAttributeValue(AttributeA[i], AttributeB[i]);
		}
	}

	void DiffAttributes(
		const FManagedArrayCollection& CollectionA, 
		const FManagedArrayCollection& CollectionB, 
		FManagedArrayCollection& ResultCollection,
		const FName& InAttributeName, 
		const FName& InGroupName
	)
	{
		const FManagedArrayCollection::EArrayType ArrayTypeA = CollectionA.GetAttributeType(InAttributeName, InGroupName);
		const FManagedArrayCollection::EArrayType ArrayTypeB = CollectionB.GetAttributeType(InAttributeName, InGroupName);
		if (ArrayTypeA != ArrayTypeB)
		{
			return;
		}

		switch (ArrayTypeA)
		{
		case FManagedArrayCollection::EArrayType::FVectorType:
			return DiffAttributes_Internal<FVector3f>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVectorType:
			return DiffAttributes_Internal<FIntVector>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector2DType:
			return DiffAttributes_Internal<FVector2f>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FLinearColorType:
			return DiffAttributes_Internal<FLinearColor>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FInt32Type:
			return DiffAttributes_Internal<int32>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FTransformType:
			return DiffAttributes_Internal<FTransform>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FFloatType:
			return DiffAttributes_Internal<float>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FQuatType:
			return DiffAttributes_Internal<FQuat4f>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntArrayType:
			return DiffAttributes_Internal<TSet<int32>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FUInt8Type:
			return DiffAttributes_Internal<uint8>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector2DArrayType:
			return DiffAttributes_Internal<TArray<FVector2f>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FDoubleType:
			return DiffAttributes_Internal<double>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector4Type:
			return DiffAttributes_Internal<FIntVector4>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector3dType:
			return DiffAttributes_Internal<FVector3d>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector2Type:
			return DiffAttributes_Internal<FIntVector2>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector2ArrayType:
			return DiffAttributes_Internal<TArray<FIntVector2>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FInt32ArrayType:
			return DiffAttributes_Internal<TArray<int32>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FFloatArrayType:
			return DiffAttributes_Internal<TArray<float>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector4fType:
			return DiffAttributes_Internal<FVector4f>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FFVectorArrayType:
			return DiffAttributes_Internal<TArray<FVector3f>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FTransform3fType:
			return DiffAttributes_Internal<FTransform3f>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector3ArrayType:
			return DiffAttributes_Internal<TArray<FIntVector3>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector4fArrayType:
			return DiffAttributes_Internal<TArray<FVector4f>>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FUintVector2Type:
			return DiffAttributes_Internal<FUintVector2>(CollectionA, CollectionB, ResultCollection, InAttributeName, InGroupName);
		
		// Not trivially diffable
		case FManagedArrayCollection::EArrayType::FStringType:
		case FManagedArrayCollection::EArrayType::FGuidType:
		case FManagedArrayCollection::EArrayType::FVectorArrayPointerType:
		case FManagedArrayCollection::EArrayType::FVectorArrayUniquePointerType:
		case FManagedArrayCollection::EArrayType::FUObjectArrayType:
		case FManagedArrayCollection::EArrayType::FNameType:
		case FManagedArrayCollection::EArrayType::FSoftObjectPathType:
		case FManagedArrayCollection::EArrayType::FBoolType:

		// Unused in PVE
		case FManagedArrayCollection::EArrayType::FMeshSectionType:
		case FManagedArrayCollection::EArrayType::FBoneNodeType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3PointerType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3UniquePointerType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3SerializablePtrType:
		case FManagedArrayCollection::EArrayType::FFBVHParticlesFloat3PointerType:
		case FManagedArrayCollection::EArrayType::FFBVHParticlesFloat3UniquePointerType:
		case FManagedArrayCollection::EArrayType::FTPBDRigidParticleHandle3fPtrType:
		case FManagedArrayCollection::EArrayType::FTPBDGeometryCollectionParticleHandle3fPtrType:
		case FManagedArrayCollection::EArrayType::FTGeometryParticle3fUniquePtrType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3ThreadSafeSharedPointerType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3SharedPointerType:
		case FManagedArrayCollection::EArrayType::FTPBDRigidClusteredParticleHandle3fPtrType:
		case FManagedArrayCollection::EArrayType::FFConvexUniquePtrType:
		case FManagedArrayCollection::EArrayType::FTPBDRigidParticle3fUniquePtrType:
		case FManagedArrayCollection::EArrayType::FFImplicitObjectRefCountedPtrType:
		case FManagedArrayCollection::EArrayType::FFConvexRefCountedPtrType:
		case FManagedArrayCollection::EArrayType::FPMatrix33dType:
		case FManagedArrayCollection::EArrayType::FLinearCurveType:
		case FManagedArrayCollection::EArrayType::FFVector3fNestedArrayType:
			return;
		default:
			ensureAlwaysMsgf(false, TEXT("Unknown attribute type"));
			break;
		}
	}
};

namespace PVGrowthDataDiffPins
{
	const static FName InputA = FName(TEXT("A"));
	const static FName InputB = FName(TEXT("B"));
};

#if WITH_EDITOR
FLinearColor UPVGrowthDataDiffSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Development;
}

FText UPVGrowthDataDiffSettings::GetCategoryOverride() const
{
	return PV::Categories::Development;
}


FName UPVGrowthDataDiffSettings::GetDefaultNodeName() const
{ 
	return FName(TEXT("GrowthDataDiff")); 
}

FText UPVGrowthDataDiffSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Growth Data Diff"); 
}

FText UPVGrowthDataDiffSettings::GetNodeTooltipText() const
{ 
	return LOCTEXT("NodeTooltip", "Diff two sets of grwoth data");
}
#endif

UPVGrowthDataDiffSettings::UPVGrowthDataDiffSettings()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bOnlyExposeInDebugMode = true;
		bExposeToLibrary = false;
	}
#endif
}
 
TArray<FPCGPinProperties> UPVGrowthDataDiffSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& PinA = Properties.Emplace_GetRef(PVGrowthDataDiffPins::InputA, GetInputPinTypeIdentifier());
	PinA.SetRequiredPin();
	PinA.SetAllowMultipleConnections(false);
	PinA.bAllowMultipleData = false;

	FPCGPinProperties& PinB = Properties.Emplace_GetRef(PVGrowthDataDiffPins::InputB, GetInputPinTypeIdentifier());
	PinB.SetRequiredPin();
	PinB.SetAllowMultipleConnections(false);
	PinB.bAllowMultipleData = false;

	return Properties;
}

FPCGElementPtr UPVGrowthDataDiffSettings::CreateElement() const
{
	return MakeShared<FPVGrowthDataDiffElement>();
}

FPCGDataTypeIdentifier UPVGrowthDataDiffSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}
 
FPCGDataTypeIdentifier UPVGrowthDataDiffSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}
 
bool FPVGrowthDataDiffElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowthDataDiffElement::Execute);
 
	check(InContext);
 
	const UPVGrowthDataDiffSettings* Settings = InContext->GetInputSettings<UPVGrowthDataDiffSettings>();
	check(Settings);
	
	const TArray<FPCGTaggedData> PinInputA = InContext->InputData.GetInputsByPin(PVGrowthDataDiffPins::InputA);
	const TArray<FPCGTaggedData> PinInputB = InContext->InputData.GetInputsByPin(PVGrowthDataDiffPins::InputB);

	if (PinInputA.IsEmpty() || PinInputB.IsEmpty())
	{
		return true;
	}

	const UPVGrowthData* GrowthDataA = Cast<UPVGrowthData>(PinInputA[0].Data);
	const UPVGrowthData* GrowthDataB = Cast<UPVGrowthData>(PinInputB[0].Data);
	if (!GrowthDataA || !GrowthDataB)
	{
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}

	const FManagedArrayCollection& CollectionA = GrowthDataA->GetCollection();
	const FManagedArrayCollection& CollectionB = GrowthDataB->GetCollection();

	if (!PV::Utilities::IsValidGrowthData(CollectionA) || !PV::Utilities::IsValidGrowthData(CollectionB))
	{
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}

	if (!PV::DiffHelpers::HasSameGrowthDataHierarchy(CollectionA, CollectionB))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("DifferentHierarchy", "Plant hierarchy is not the same"), InContext);
		return true;
	}

	FManagedArrayCollection DiffCollection;

	// Copy core hierarchy attributes
	PV::FPointPositionAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchPointsAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchParentNumberAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchParentsAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchNumberAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchChildrenAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchHierarchyNumberAttribute::CopyAttribute(CollectionA, DiffCollection);
	PV::FBranchPlantNumberAttribute::CopyAttribute(CollectionA, DiffCollection);

	// Diff required attributes
	const static TArray<FManagedArrayCollection::FManagedType> AttributesToDiff = {
		PV::FBranchSourceBudNumberAttribute::ManagedType,
		PV::FPointLengthFromRootAttribute::ManagedType,
		PV::FPointLengthFromSeedAttribute::ManagedType,
		PV::FPointScaleGradientAttribute::ManagedType,
		PV::FPointHullGradientAttribute::ManagedType,
		PV::FPointMainTrunkGradientAttribute::ManagedType,
		PV::FPointGroundGradientAttribute::ManagedType,
		PV::FPointScaleAttribute::ManagedType,
		PV::FBudDirectionAttribute::ManagedType,
		PV::FBudStatusAttribute::ManagedType,
		PV::FBudLateralMeristemAttribute::ManagedType,
		PV::FBudHormoneLevelsAttribute::ManagedType,
		PV::FPointPlantGradientAttribute::ManagedType,
		PV::FPointBudNumberAttribute::ManagedType,
		PV::FPointNjordPixelIndexAttribute::ManagedType,
		PV::FBudLightDetectedAttribute::ManagedType,
		PV::FBudDevelopmentAttribute::ManagedType,
	};

	for (const auto& ManagedType : AttributesToDiff)
	{
		if (CollectionA.GetAttributeType(ManagedType.Name, ManagedType.Group) != ManagedType.Type
			|| CollectionB.GetAttributeType(ManagedType.Name, ManagedType.Group) != ManagedType.Type)
		{
			continue;
		}

		PV::DiffHelpers::DiffAttributes(CollectionA, CollectionB, DiffCollection, ManagedType.Name, ManagedType.Group);
	}

	UPVGrowthData* OutGrowthData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
	OutGrowthData->Initialize(MoveTemp(DiffCollection));
	InContext->OutputData.TaggedData.Emplace(OutGrowthData);

	return true; 
}

#undef LOCTEXT_NAMESPACE
