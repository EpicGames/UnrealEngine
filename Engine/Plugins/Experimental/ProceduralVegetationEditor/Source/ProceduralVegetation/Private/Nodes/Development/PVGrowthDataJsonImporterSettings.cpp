// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVGrowthDataJsonImporterSettings.h"
 
#include "ProceduralVegetationModule.h"
 
#include "DataTypes/PVGrowthData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PVJSONHelper.h"
#include "Utils/PVAttributes.h"
#include "Helpers/PVAttributesHelper.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVGrowthDataJsonImporterSettings"

#if WITH_EDITOR
FLinearColor UPVGrowthDataJsonImporterSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Development;
}

FText UPVGrowthDataJsonImporterSettings::GetCategoryOverride() const
{
	return PV::Categories::Development;
}


FName UPVGrowthDataJsonImporterSettings::GetDefaultNodeName() const
{ 
	return FName(TEXT("GrowthDataJsonImporter")); 
}

FText UPVGrowthDataJsonImporterSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Growth Data Json Importer"); 
}

FText UPVGrowthDataJsonImporterSettings::GetNodeTooltipText() const
{ 
	return LOCTEXT("NodeTooltip", "Imports growth data from a json file");
}
#endif

UPVGrowthDataJsonImporterSettings::UPVGrowthDataJsonImporterSettings()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bOnlyExposeInDebugMode = true;
		bExposeToLibrary = false;
	}
#endif
}
 
TArray<FPCGPinProperties> UPVGrowthDataJsonImporterSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGElementPtr UPVGrowthDataJsonImporterSettings::CreateElement() const
{
	return MakeShared<FPVGrowthDataJsonImporterElement>();
}
 
FPCGDataTypeIdentifier UPVGrowthDataJsonImporterSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}
 
bool FPVGrowthDataJsonImporterElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowthDataJsonImporterElement::Execute);
 
	check(InContext);
 
	const UPVGrowthDataJsonImporterSettings* Settings = InContext->GetInputSettings<UPVGrowthDataJsonImporterSettings>();
	check(Settings);
	
	FString ErrorMessage;
	FManagedArrayCollection Collection;
	bool bSuccess = PV::JSON::LoadGrowthDataJsonToCollection(Collection, Settings->GrowthDataFile.FilePath, ErrorMessage);
	if (!bSuccess)
	{
		UE_LOGF(LogProceduralVegetation, Error, "%ls", *ErrorMessage);
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}

	UPVGrowthData* OutGrowthData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
	OutGrowthData->Initialize(MoveTemp(Collection));
	InContext->OutputData.TaggedData.Emplace(OutGrowthData);

	return true; 
}

#undef LOCTEXT_NAMESPACE
