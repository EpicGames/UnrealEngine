// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVMockTreeImporterSettings.h"
 
#include "ProceduralVegetationModule.h"
 
#include "DataTypes/PVGrowthData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Tests/PVMockTree.h"
#include "PVCommon.h"
 
#define LOCTEXT_NAMESPACE "PVMockTreeImporterSettings"

#if WITH_EDITOR
FLinearColor UPVMockTreeImporterSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Development;
}

FText UPVMockTreeImporterSettings::GetCategoryOverride() const
{
	return PV::Categories::Development;
}


FName UPVMockTreeImporterSettings::GetDefaultNodeName() const 
{ 
	return FName(TEXT("ProceduralVegetationMockTreeImporter")); 
}

FText UPVMockTreeImporterSettings::GetDefaultNodeTitle() const 
{ 
	return LOCTEXT("NodeTitle", "Mock Tree Importer"); 
}

FText UPVMockTreeImporterSettings::GetNodeTooltipText() const 
{ 
	return LOCTEXT("NodeTooltip", "Create Procedural Vegetation Skeleton based of the Mock Tree"); 
}
#endif

UPVMockTreeImporterSettings::UPVMockTreeImporterSettings()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bOnlyExposeInDebugMode = true;
		bExposeToLibrary = false;
	}
#endif
}
 
TArray<FPCGPinProperties> UPVMockTreeImporterSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}
 
FPCGElementPtr UPVMockTreeImporterSettings::CreateElement() const
{
	return MakeShared<FPVMockTreeImporterElement>();
}
 
FPCGDataTypeIdentifier UPVMockTreeImporterSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}
 
bool FPVMockTreeImporterElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMockTreeImporterElement::Execute);
 
	check(InContext);
 
	const UPVMockTreeImporterSettings* Settings = InContext->GetInputSettings<UPVMockTreeImporterSettings>();
	check(Settings);
	
	FManagedArrayCollection Collection;
#if WITH_DEV_AUTOMATION_TESTS
	Collection = MoveTemp(*PVMockTreeCollection::CreateCollection());
#endif

	UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
	OutManagedArrayCollectionData->Initialize(MoveTemp(Collection));
		
	InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
	
	return true; 
}

#undef LOCTEXT_NAMESPACE
