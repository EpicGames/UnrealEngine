// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVRecomputeGrowthDataAttributesSettings.h"
 
#include "DataTypes/PVGrowthData.h"
#include "Helpers/PVAttributesHelper.h"
#include "PVCommon.h"
 
#define LOCTEXT_NAMESPACE "PVRecomputeGrowthDataAttributesSettings"

#if WITH_EDITOR
FLinearColor UPVRecomputeGrowthDataAttributesSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Development;
}

FText UPVRecomputeGrowthDataAttributesSettings::GetCategoryOverride() const
{
	return PV::Categories::Development;
}


FName UPVRecomputeGrowthDataAttributesSettings::GetDefaultNodeName() const
{ 
	return FName(TEXT("RecomputeGrowthDataAttributes")); 
}

FText UPVRecomputeGrowthDataAttributesSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Recompute Growth Data Attributes"); 
}

FText UPVRecomputeGrowthDataAttributesSettings::GetNodeTooltipText() const
{ 
	return LOCTEXT("NodeTooltip", "Recomputes all growth data attributes using the plant skeleton");
}
#endif

UPVRecomputeGrowthDataAttributesSettings::UPVRecomputeGrowthDataAttributesSettings()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bOnlyExposeInDebugMode = true;
		bExposeToLibrary = false;
	}
#endif
}
 
FPCGElementPtr UPVRecomputeGrowthDataAttributesSettings::CreateElement() const
{
	return MakeShared<FPVRecomputeGrowthDataAttributesElement>();
}

FPCGDataTypeIdentifier UPVRecomputeGrowthDataAttributesSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}
 
FPCGDataTypeIdentifier UPVRecomputeGrowthDataAttributesSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}
 
bool FPVRecomputeGrowthDataAttributesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVRecomputeGrowthDataAttributesElement::Execute);
 
	check(InContext);
 
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();

			PV::AttributesHelper::RecomputeAllGrowthDataAttributes(OutCollection);

			UPVGrowthData* OutGrowthData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			OutGrowthData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutGrowthData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}
	
	return true; 
}

#undef LOCTEXT_NAMESPACE
