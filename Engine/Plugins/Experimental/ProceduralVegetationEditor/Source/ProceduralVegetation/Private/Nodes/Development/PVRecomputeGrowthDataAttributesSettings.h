// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Nodes/PVBaseSettings.h"

#include "PVRecomputeGrowthDataAttributesSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVRecomputeGrowthDataAttributesSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVRecomputeGrowthDataAttributesSettings();

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;
};

class FPVRecomputeGrowthDataAttributesElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
