// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDistributionBaseSettings.h"
#include "PVCommon.h"

#if WITH_EDITOR
FLinearColor UPVDistributionBaseSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Foliage;
}

FText UPVDistributionBaseSettings::GetCategoryOverride() const
{
	return PV::Categories::DistributionSettings;
}

#endif
