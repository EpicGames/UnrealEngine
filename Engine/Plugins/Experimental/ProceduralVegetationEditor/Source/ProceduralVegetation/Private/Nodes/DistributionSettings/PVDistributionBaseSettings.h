// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "PVDistributionBaseSettings.generated.h"

UCLASS(BlueprintType, Abstract, ClassGroup = (Procedural))
class UPVDistributionBaseSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::None }; }
#endif
};
