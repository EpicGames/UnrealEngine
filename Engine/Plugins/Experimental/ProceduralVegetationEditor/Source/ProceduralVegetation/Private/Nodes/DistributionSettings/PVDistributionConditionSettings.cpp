// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDistributionConditionSettings.h"
#include "DataTypes/PVDistributionSettingsData.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVDistributionConditionSettings"

#if WITH_EDITOR
FText UPVDistributionConditionSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Distribution Condition Settings");
}
#endif

FPCGDataTypeIdentifier UPVDistributionConditionSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionConditionSettings::AsId() };
}

FPCGElementPtr UPVDistributionConditionSettings::CreateElement() const
{
	return MakeShared<FPVDistributionConditionElement>();
}

bool FPVDistributionConditionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDistributionConditionElement::Execute);

	check(InContext);

	const UPVDistributionConditionSettings* Settings = InContext->GetInputSettings<UPVDistributionConditionSettings>();
	check(Settings);

	UPVDistributionConditionSettingsData* OutData = FPCGContext::NewObject_AnyThread<UPVDistributionConditionSettingsData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	return true;
}

#undef LOCTEXT_NAMESPACE
