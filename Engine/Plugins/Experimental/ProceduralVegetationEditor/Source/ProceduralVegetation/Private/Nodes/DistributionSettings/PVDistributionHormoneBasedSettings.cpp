// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDistributionHormoneBasedSettings.h"
#include "DataTypes/PVDistributionSettingsData.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVDistributionHormoneBasedSettings"

#if WITH_EDITOR
FText UPVDistributionHormoneBasedSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Distribution Hormone Based Settings");
}
#endif

FPCGDataTypeIdentifier UPVDistributionHormoneBasedSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionHormoneBasedSettings::AsId() };
}

FPCGElementPtr UPVDistributionHormoneBasedSettings::CreateElement() const
{
	return MakeShared<FPVDistributionHormoneBasedElement>();
}

bool FPVDistributionHormoneBasedElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDistributionHormoneBasedElement::Execute);

	check(InContext);

	const UPVDistributionHormoneBasedSettings* Settings = InContext->GetInputSettings<UPVDistributionHormoneBasedSettings>();
	check(Settings);

	UPVDistributionHormoneBasedSettingsData* OutData = FPCGContext::NewObject_AnyThread<UPVDistributionHormoneBasedSettingsData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	return true;
}

#undef LOCTEXT_NAMESPACE
