// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDistributionParametricSettings.h"
#include "DataTypes/PVDistributionSettingsData.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVDistributionParametricSettings"

#if WITH_EDITOR
FText UPVDistributionParametricSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Distribution Parametric Settings");
}
#endif

FPCGDataTypeIdentifier UPVDistributionParametricSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionParametricSettings::AsId() };
}

FPCGElementPtr UPVDistributionParametricSettings::CreateElement() const
{
	return MakeShared<FPVDistributionParametricElement>();
}

bool FPVDistributionParametricElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDistributionParametricElement::Execute);

	check(InContext);

	const UPVDistributionParametricSettings* Settings = InContext->GetInputSettings<UPVDistributionParametricSettings>();
	check(Settings);

	UPVDistributionParametricSettingsData* OutData = FPCGContext::NewObject_AnyThread<UPVDistributionParametricSettingsData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	return true;
}

#undef LOCTEXT_NAMESPACE
