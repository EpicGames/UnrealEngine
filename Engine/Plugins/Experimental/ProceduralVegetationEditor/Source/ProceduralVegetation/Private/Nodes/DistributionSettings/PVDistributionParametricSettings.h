// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVDistributionBaseSettings.h"
#include "DataTypes/PVDistributionParams.h"
#include "PVDistributionParametricSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVDistributionParametricSettings : public UPVDistributionBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("DistributionParametricSettings")); }
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override { return TArray<FPCGPinProperties>(); }
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Parametric Settings", meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPVDistributionParametricParams Params;
};

class FPVDistributionParametricElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
