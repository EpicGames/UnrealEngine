// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDistributionVectorSettings.h"
#include "DataTypes/PVDistributionSettingsData.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVDistributionVectorSettings"

#if WITH_EDITOR
FText UPVDistributionVectorSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Distribution Vector Settings");
}
#endif

FPCGDataTypeIdentifier UPVDistributionVectorSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionVectorSettings::AsId() };
}

FPCGElementPtr UPVDistributionVectorSettings::CreateElement() const
{
	return MakeShared<FPVDistributionVectorElement>();
}

bool FPVDistributionVectorElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDistributionVectorElement::Execute);

	check(InContext);

	const UPVDistributionVectorSettings* Settings = InContext->GetInputSettings<UPVDistributionVectorSettings>();
	check(Settings);

	UPVDistributionVectorSettingsData* OutData = FPCGContext::NewObject_AnyThread<UPVDistributionVectorSettingsData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	return true;
}

#undef LOCTEXT_NAMESPACE
