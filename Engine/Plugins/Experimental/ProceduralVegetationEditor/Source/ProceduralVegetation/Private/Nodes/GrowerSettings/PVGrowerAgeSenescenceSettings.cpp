// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerAgeSenescenceSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerAgeSenescenceSettings"

#if WITH_EDITOR
FText UPVGrowerAgeSenescenceSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Age Senescence Settings"); 
}
#endif

FPCGDataTypeIdentifier UPVGrowerAgeSenescenceSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerAgeSenescence::AsId() };
}

FPCGElementPtr UPVGrowerAgeSenescenceSettings::CreateElement() const
{
	return MakeShared<FPVAgeSenescenceElement>();
}

bool FPVAgeSenescenceElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVAgeSenescenceElement::Execute);

	check(InContext);

	const UPVGrowerAgeSenescenceSettings* Settings = InContext->GetInputSettings<UPVGrowerAgeSenescenceSettings>();
	check(Settings);
	
	UPVGrowerAgeSenescenceData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerAgeSenescenceData>(InContext);
	OutData->Params =  Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE