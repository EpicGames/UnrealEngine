// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVGrowerBaseSettings.h"
#include "DataTypes/PVGrowerParams.h"
#include "Nodes/PVBaseSettings.h"
#include "PVGrowerAgeSenescenceSettings.generated.h"


UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerAgeSenescenceSettings : public UPVGrowerBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GrowerAgeSenescence")); }
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category="AgeSenescence", meta=(ShowOnlyInnerProperties, PCG_Overridable, InnerCategoryToggle))
	FPVAgeSenescenceParams Params;
};

class FPVAgeSenescenceElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
