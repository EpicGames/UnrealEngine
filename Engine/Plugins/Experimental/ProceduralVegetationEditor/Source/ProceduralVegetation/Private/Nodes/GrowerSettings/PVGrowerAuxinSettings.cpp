// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerAuxinSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerAuxinSettings"

#if WITH_EDITOR
FText UPVGrowerAuxinSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Auxin Settings"); 
}
#endif

FString UPVGrowerAuxinSettings::GetAdditionalTitleInformation() const
{
	return ParamsWithTargets.Targets.ToString();
}

FPCGDataTypeIdentifier UPVGrowerAuxinSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerAuxin::AsId() };
}

FPCGElementPtr UPVGrowerAuxinSettings::CreateElement() const
{
	return MakeShared<FPVGrowerAuxinElement>();
}

bool FPVGrowerAuxinElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerAuxinElement::Execute);

	check(InContext);

	const UPVGrowerAuxinSettings* Settings = InContext->GetInputSettings<UPVGrowerAuxinSettings>();
	check(Settings);
	
	UPVGrowerAuxinData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerAuxinData>(InContext);
	OutData->ParamsWithTargets =  Settings->ParamsWithTargets;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE