// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerBaseSettings.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVGrowerBaseSettings"

#if WITH_EDITOR
FLinearColor UPVGrowerBaseSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Growth;
}

FText UPVGrowerBaseSettings::GetCategoryOverride() const
{
	return PV::Categories::GrowthSettings;
}



FText UPVGrowerBaseSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip","\n\nPress Ctrl + L to lock/unlock node output");
}

#endif

TArray<FPCGPinProperties> UPVGrowerBaseSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

#undef LOCTEXT_NAMESPACE
