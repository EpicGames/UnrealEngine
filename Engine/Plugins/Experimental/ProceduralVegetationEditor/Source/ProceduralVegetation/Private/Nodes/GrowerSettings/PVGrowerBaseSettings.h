// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "PVGrowerBaseSettings.generated.h"

UCLASS(BlueprintType, Abstract, ClassGroup = (Procedural))
class UPVGrowerBaseSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("UPVGrowerBaseSettings")); }
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::None }; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
};