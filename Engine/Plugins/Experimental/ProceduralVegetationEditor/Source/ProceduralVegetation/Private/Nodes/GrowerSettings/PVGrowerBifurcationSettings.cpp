// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerBifurcationSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerBifurcationSettings"

#if WITH_EDITOR
FText UPVGrowerBifurcationSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Bifurcation Settings"); 
}
#endif

FString UPVGrowerBifurcationSettings::GetAdditionalTitleInformation() const
{
	return ParamsWithTargets.Targets.ToString();
}

FPCGDataTypeIdentifier UPVGrowerBifurcationSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerBifurcation::AsId() };
}

FPCGElementPtr UPVGrowerBifurcationSettings::CreateElement() const
{
	return MakeShared<FPVGrowerBifurcationElement>();
}

bool FPVGrowerBifurcationElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerBifurcationElement::Execute);

	check(InContext);

	const UPVGrowerBifurcationSettings* Settings = InContext->GetInputSettings<UPVGrowerBifurcationSettings>();
	check(Settings);
	
	UPVGrowerBifurcationData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerBifurcationData>(InContext);
	OutData->ParamsWithTargets =  Settings->ParamsWithTargets;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE