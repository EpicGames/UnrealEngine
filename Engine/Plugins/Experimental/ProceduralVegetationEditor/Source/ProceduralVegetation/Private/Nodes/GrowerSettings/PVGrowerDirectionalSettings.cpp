// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerDirectionalSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerDirectionalSettings"

#if WITH_EDITOR
FText UPVGrowerDirectionalSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Directional Settings"); 
}
#endif

FString UPVGrowerDirectionalSettings::GetAdditionalTitleInformation() const
{
	return ParamsWithTargets.Targets.ToString();
}

FPCGDataTypeIdentifier UPVGrowerDirectionalSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerDirectional::AsId() };
}

FPCGElementPtr UPVGrowerDirectionalSettings::CreateElement() const
{
	return MakeShared<FPVGrowerDirectionalElement>();
}

bool FPVGrowerDirectionalElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerAngleElement::Execute);

	check(InContext);

	const UPVGrowerDirectionalSettings* Settings = InContext->GetInputSettings<UPVGrowerDirectionalSettings>();
	check(Settings);
	
	UPVGrowerDirectionalData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerDirectionalData>(InContext);
	OutData->ParamsWithTargets =  Settings->ParamsWithTargets;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE