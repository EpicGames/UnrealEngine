// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerFoliageSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerFoliageSettings"

#if WITH_EDITOR
FText UPVGrowerFoliageSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Foliage Settings"); 
}

FText UPVGrowerFoliageSettings::GetNodeTooltipText() const
{ 
	return LOCTEXT("NodeTooltip", 
		"\n\nPress Ctrl + L to lock/unlock node output"
	); 
}
#endif

TArray<FPCGPinProperties> UPVGrowerFoliageSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGDataTypeIdentifier UPVGrowerFoliageSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerFoliage::AsId() };
}

FPCGElementPtr UPVGrowerFoliageSettings::CreateElement() const
{
	return MakeShared<FPVGrowerFoliageElement>();
}


bool FPVGrowerFoliageElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerFoliageElement::Execute);

	check(InContext);

	const UPVGrowerFoliageSettings* Settings = InContext->GetInputSettings<UPVGrowerFoliageSettings>();
	check(Settings);
	
	UPVGrowerFoliageData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerFoliageData>(InContext);
	OutData->Params =  Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE