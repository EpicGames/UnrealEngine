// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVGrowerBaseSettings.h"
#include "DataTypes/PVGrowerParams.h"
#include "Implementations/PVGravity.h"
#include "PVGrowerFoliageSettings.generated.h"


UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerFoliageSettings : public UPVGrowerBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GrowerFoliageSettings")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::None }; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:

	UPROPERTY(EditAnywhere, Category="Foliage Settings", meta=(ShowOnlyInnerProperties, PCG_Overridable, InnerCategoryToggle))
	FPVFoliageParams Params;
};

class FPVGrowerFoliageElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
