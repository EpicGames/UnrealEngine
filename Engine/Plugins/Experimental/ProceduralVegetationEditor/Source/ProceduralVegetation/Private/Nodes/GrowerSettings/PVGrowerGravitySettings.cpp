// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerGravitySettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerGravitySettings"

#if WITH_EDITOR
FText UPVGrowerGravitySettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Gravity Settings"); 
}
#endif

FPCGDataTypeIdentifier UPVGrowerGravitySettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerGravity::AsId() };
}

FPCGElementPtr UPVGrowerGravitySettings::CreateElement() const
{
	return MakeShared<FPVGrowerGravityElement>();
}

bool FPVGrowerGravityElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerGravityElement::Execute);

	check(InContext);

	const UPVGrowerGravitySettings* Settings = InContext->GetInputSettings<UPVGrowerGravitySettings>();
	check(Settings);
	
	UPVGrowerGravityData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerGravityData>(InContext);
	OutData->Params =  Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE