// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerLightSenescenceSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerLightSenescenceSettings"

#if WITH_EDITOR
FText UPVGrowerLightSenescenceSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Light Senescence Settings"); 
}
#endif

FPCGDataTypeIdentifier UPVGrowerLightSenescenceSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerLightSenescence::AsId() };
}

FPCGElementPtr UPVGrowerLightSenescenceSettings::CreateElement() const
{
	return MakeShared<FPVLightSenescenceElement>();
}

bool FPVLightSenescenceElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVLightSenescenceElement::Execute);

	check(InContext);

	const UPVGrowerLightSenescenceSettings* Settings = InContext->GetInputSettings<UPVGrowerLightSenescenceSettings>();
	check(Settings);
	
	UPVGrowerLightSenescenceData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerLightSenescenceData>(InContext);
	OutData->Params =  Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE