// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVGrowerBaseSettings.h"
#include "DataTypes/PVGrowerParams.h"
#include "Nodes/PVBaseSettings.h"
#include "PVGrowerLightSenescenceSettings.generated.h"


UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerLightSenescenceSettings : public UPVGrowerBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GrowerLightSenescence")); }
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category="LightSenescence", meta=(ShowOnlyInnerProperties, PCG_Overridable, InnerCategoryToggle))
	FPVLightSenescenceParams Params;
};

class FPVLightSenescenceElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
