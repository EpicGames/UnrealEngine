// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerPhototropismSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerPhototropismSettings"

#if WITH_EDITOR
FText UPVGrowerPhototropismSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Phototropism Settings"); 
}
#endif

FString UPVGrowerPhototropismSettings::GetAdditionalTitleInformation() const
{
	return ParamsWithTargets.Targets.ToString();
}

FPCGDataTypeIdentifier UPVGrowerPhototropismSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerPhototropism::AsId() };
}

FPCGElementPtr UPVGrowerPhototropismSettings::CreateElement() const
{
	return MakeShared<FPVGrowerPhototropismElement>();
}

bool FPVGrowerPhototropismElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerPhototropismElement::Execute);

	check(InContext);

	const UPVGrowerPhototropismSettings* Settings = InContext->GetInputSettings<UPVGrowerPhototropismSettings>();
	check(Settings);
	
	UPVGrowerPhototropismData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerPhototropismData>(InContext);
	OutData->ParamsWithTargets =  Settings->ParamsWithTargets;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE