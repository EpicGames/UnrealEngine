// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVGrowerBaseSettings.h"
#include "DataTypes/PVGrowerParams.h"
#include "PVGrowerPhototropismSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerPhototropismSettings : public UPVGrowerBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GrowerPhototropismSettings")); }
	virtual FText GetDefaultNodeTitle() const override;
#endif

	virtual FString GetAdditionalTitleInformation() const override;
	
protected:
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:

	UPROPERTY(EditAnywhere, Category="PhototropismSettings", meta=(ShowOnlyInnerProperties, PCG_Overridable, InnerCategoryToggle))
	FPVGrowerPhototropismWithTargets ParamsWithTargets;
};

class FPVGrowerPhototropismElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
