// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerPhyllotaxySettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVPhyllotaxySettings"

#if WITH_EDITOR
FText UPVGrowerPhyllotaxySettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Phyllotaxy Settings"); 
}
#endif

FString UPVGrowerPhyllotaxySettings::GetAdditionalTitleInformation() const
{
	return ParamsWithTargets.Targets.ToString();
}

FPCGDataTypeIdentifier UPVGrowerPhyllotaxySettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerPhyllotaxy::AsId() };
}

FPCGElementPtr UPVGrowerPhyllotaxySettings::CreateElement() const
{
	return MakeShared<FPVGrowerPhyllotaxyElement>();
}

bool FPVGrowerPhyllotaxyElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVPhylotaxyElement::Execute);

	check(InContext);

	const UPVGrowerPhyllotaxySettings* Settings = InContext->GetInputSettings<UPVGrowerPhyllotaxySettings>();
	check(Settings);
	
	UPVGrowerPhyllotaxyData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerPhyllotaxyData>(InContext);
	OutData->ParamsWithTargets =  Settings->ParamsWithTargets;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE