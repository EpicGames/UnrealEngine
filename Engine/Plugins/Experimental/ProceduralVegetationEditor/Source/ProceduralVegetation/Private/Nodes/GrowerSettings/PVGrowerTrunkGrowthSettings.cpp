// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerTrunkGrowthSettings.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGrower.h"

#define LOCTEXT_NAMESPACE "PVGrowerTrunkGrowthSettings"

#if WITH_EDITOR
FText UPVGrowerTrunkGrowthSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower Growth Settings"); 
}
#endif

FPCGDataTypeIdentifier UPVGrowerTrunkGrowthSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerTrunkGrowth::AsId() };
}

FPCGElementPtr UPVGrowerTrunkGrowthSettings::CreateElement() const
{
	return MakeShared<FPVGrowerTrunkGrowthElement>();
}



bool FPVGrowerTrunkGrowthElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerTrunkGrowthElement::Execute);

	check(InContext);

	const UPVGrowerTrunkGrowthSettings* Settings = InContext->GetInputSettings<UPVGrowerTrunkGrowthSettings>();
	check(Settings);
	
	UPVGrowerTrunkGrowthData* OutData = FPCGContext::NewObject_AnyThread<UPVGrowerTrunkGrowthData>(InContext);
	OutData->Params =  Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);
	
	return true;
}

#undef LOCTEXT_NAMESPACE