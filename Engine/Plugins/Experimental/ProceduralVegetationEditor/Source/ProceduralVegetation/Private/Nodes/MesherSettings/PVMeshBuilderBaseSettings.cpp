// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderBaseSettings.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderBaseSettings"

#if WITH_EDITOR
FLinearColor UPVMeshBuilderBaseSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Mesh;
}

FText UPVMeshBuilderBaseSettings::GetCategoryOverride() const
{
	return PV::Categories::MeshBuilderSettings;
}



FText UPVMeshBuilderBaseSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "\n\nPress Ctrl + L to lock/unlock node output");
}

#endif

TArray<FPCGPinProperties> UPVMeshBuilderBaseSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

#undef LOCTEXT_NAMESPACE
