// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderBranchRadiusSettings.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderBranchRadiusSettings"

#if WITH_EDITOR
FText UPVMeshBuilderBranchRadiusSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh Builder Branch Radius Settings");
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderBranchRadiusSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderBranchRadius::AsId() };
}

FPCGElementPtr UPVMeshBuilderBranchRadiusSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderBranchRadiusSettingsElement>();
}

bool FPVMeshBuilderBranchRadiusSettingsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderBranchRadiusSettingsElement::Execute);

	check(InContext);

	const UPVMeshBuilderBranchRadiusSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderBranchRadiusSettings>();
	check(Settings);

	UPVMeshBuilderBranchRadiusData* OutData = FPCGContext::NewObject_AnyThread<UPVMeshBuilderBranchRadiusData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);

	return true;
}

#undef LOCTEXT_NAMESPACE
