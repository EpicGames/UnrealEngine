// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderDisplacementSettings.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"
#include "Implementations/PVMeshBuilder.h"
#include "Helpers/PCGSettingsHelpers.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderDisplacementSettings"

#if WITH_EDITOR
FText UPVMeshBuilderDisplacementSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh Builder Displacement Settings");
}

void UPVMeshBuilderDisplacementSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderDisplacementParams, Texture))
	{
		DisplacementWarnings.Empty();
		if (Params.Texture)
		{
			FPVMeshBuilder::ExtractDisplacementData(Params.Texture, CachedValues, DisplacementWarnings);
		}
		else
		{
			CachedValues.Empty();
		}

		Modify();
	}
}

void UPVMeshBuilderDisplacementSettings::PostLoad()
{
	Super::PostLoad();

	DisplacementWarnings.Empty();
	if (Params.Texture)
	{
		FPVMeshBuilder::ExtractDisplacementData(Params.Texture, CachedValues, DisplacementWarnings);
	}
	else
	{
		CachedValues.Empty();
	}
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderDisplacementSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderDisplacement::AsId() };
}

FPCGElementPtr UPVMeshBuilderDisplacementSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderDisplacementSettingsElement>();
}

bool FPVMeshBuilderDisplacementSettingsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderDisplacementSettingsElement::Execute);

	check(InContext);

	const UPVMeshBuilderDisplacementSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderDisplacementSettings>();
	check(Settings);

	UPVMeshBuilderDisplacementData* OutData = FPCGContext::NewObject_AnyThread<UPVMeshBuilderDisplacementData>(InContext);
	OutData->Params = Settings->Params;
	OutData->Values = Settings->CachedValues;

	if (!Settings->DisplacementWarnings.IsEmpty())
	{
		PCGLog::LogWarningOnGraph(FText::FromString(Settings->DisplacementWarnings), InContext);
	}

	InContext->OutputData.TaggedData.Emplace(OutData);

	return true;
}

#undef LOCTEXT_NAMESPACE
