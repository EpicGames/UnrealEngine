// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVMeshBuilderBaseSettings.h"
#include "DataTypes/PVMeshBuilderParams.h"
#include "PVMeshBuilderDisplacementSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderDisplacementSettings : public UPVMeshBuilderBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MeshBuilderDisplacementSettings")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif

protected:
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Displacement", meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPVMeshBuilderDisplacementParams Params;

	/** Cached pixel values extracted from Params.Texture; populated in PostEditChangeProperty / PostLoad. */
	UPROPERTY(Transient, NonTransactional)
	TArray<float> CachedValues;

	UPROPERTY()
	FString DisplacementWarnings;
};

class FPVMeshBuilderDisplacementSettingsElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
