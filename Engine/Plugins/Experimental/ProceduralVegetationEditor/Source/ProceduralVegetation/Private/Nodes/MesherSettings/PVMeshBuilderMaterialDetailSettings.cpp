// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderMaterialDetailSettings.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderMaterialDetailSettings"

#if WITH_EDITOR
FText UPVMeshBuilderMaterialDetailSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh Builder Material Detail Settings");
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderMaterialDetailSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderMaterialDetail::AsId() };
}

FPCGElementPtr UPVMeshBuilderMaterialDetailSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderMaterialDetailSettingsElement>();
}

bool FPVMeshBuilderMaterialDetailSettingsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderMaterialDetailSettingsElement::Execute);

	check(InContext);

	const UPVMeshBuilderMaterialDetailSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderMaterialDetailSettings>();
	check(Settings);

	UPVMeshBuilderMaterialDetailData* OutData = FPCGContext::NewObject_AnyThread<UPVMeshBuilderMaterialDetailData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);

	return true;
}

#undef LOCTEXT_NAMESPACE
