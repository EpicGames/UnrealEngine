// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVMeshBuilderBaseSettings.h"
#include "Implementations/PVMaterialSettings.h"
#include "PVMeshBuilderMaterialDetailSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderMaterialDetailSettings : public UPVMeshBuilderBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MeshBuilderMaterialDetailSettings")); }
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Material Details", meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPVMaterialSettings Params;
};

class FPVMeshBuilderMaterialDetailSettingsElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
