// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderMeshDetailSettings.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderMeshDetailSettings"

#if WITH_EDITOR
FText UPVMeshBuilderMeshDetailSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh Builder Mesh Detail Settings");
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderMeshDetailSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderMeshDetail::AsId() };
}

FPCGElementPtr UPVMeshBuilderMeshDetailSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderMeshDetailSettingsElement>();
}

bool FPVMeshBuilderMeshDetailSettingsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderMeshDetailSettingsElement::Execute);

	check(InContext);

	const UPVMeshBuilderMeshDetailSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderMeshDetailSettings>();
	check(Settings);

	UPVMeshBuilderMeshDetailData* OutData = FPCGContext::NewObject_AnyThread<UPVMeshBuilderMeshDetailData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);

	return true;
}

#undef LOCTEXT_NAMESPACE
