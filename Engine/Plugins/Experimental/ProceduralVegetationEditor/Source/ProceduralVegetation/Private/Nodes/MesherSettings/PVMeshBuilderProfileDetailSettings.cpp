// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderProfileDetailSettings.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderProfileDetailSettings"

#if WITH_EDITOR
FText UPVMeshBuilderProfileDetailSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh Builder Profile Detail Settings");
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderProfileDetailSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderProfileDetail::AsId() };
}

FPCGElementPtr UPVMeshBuilderProfileDetailSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderProfileDetailSettingsElement>();
}

bool FPVMeshBuilderProfileDetailSettingsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderProfileDetailSettingsElement::Execute);

	check(InContext);

	const UPVMeshBuilderProfileDetailSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderProfileDetailSettings>();
	check(Settings);

	UPVMeshBuilderProfileDetailData* OutData = FPCGContext::NewObject_AnyThread<UPVMeshBuilderProfileDetailData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);

	return true;
}

#undef LOCTEXT_NAMESPACE
