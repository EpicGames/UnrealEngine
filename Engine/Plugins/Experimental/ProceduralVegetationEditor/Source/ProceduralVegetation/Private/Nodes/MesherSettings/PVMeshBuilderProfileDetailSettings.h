// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PVMeshBuilderBaseSettings.h"
#include "DataTypes/PVMeshBuilderParams.h"
#include "PVMeshBuilderProfileDetailSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderProfileDetailSettings : public UPVMeshBuilderBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MeshBuilderProfileDetailSettings")); }
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Profile Details", meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPVMeshBuilderProfileDetailParams Params;
};

class FPVMeshBuilderProfileDetailSettingsElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
