// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderSkeletonShapingSettings.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderSkeletonShapingSettings"

#if WITH_EDITOR
FText UPVMeshBuilderSkeletonShapingSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh Builder Skeleton Shaping Settings");
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderSkeletonShapingSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderSkeletonShaping::AsId() };
}

FPCGElementPtr UPVMeshBuilderSkeletonShapingSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderSkeletonShapingSettingsElement>();
}

bool FPVMeshBuilderSkeletonShapingSettingsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderSkeletonShapingSettingsElement::Execute);

	check(InContext);

	const UPVMeshBuilderSkeletonShapingSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderSkeletonShapingSettings>();
	check(Settings);

	UPVMeshBuilderSkeletonShapingData* OutData = FPCGContext::NewObject_AnyThread<UPVMeshBuilderSkeletonShapingData>(InContext);
	OutData->Params = Settings->Params;

	InContext->OutputData.TaggedData.Emplace(OutData);

	return true;
}

#undef LOCTEXT_NAMESPACE
