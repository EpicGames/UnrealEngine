// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVBaseSettings.h"
#include "PCGContext.h"
#include "Helpers/PCGHelpers.h"
#include "DataTypes/PVData.h"
#include "PCGData.h"
#include "Helpers/PVAnalyticsHelper.h"

UPVBaseSettings::UPVBaseSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		if (const UClass* Class = GetClass())
		{
			PV::Analytics::SendNodeAddedEvent(Class->GetName());
		}
	}
}

#if WITH_EDITOR
void UPVBaseSettings::SetCurrentRenderType(TArray<EPVRenderType> InRenderTypes)
{
	Visualization.CurrentRenderType = MoveTemp(InRenderTypes);
}
#endif

TArray<FPCGPinProperties> UPVBaseSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, GetInputPinTypeIdentifier());
	Pin.SetRequiredPin();
	Pin.SetAllowMultipleConnections(false);
	Pin.bAllowMultipleData = false;

	return Properties;
}

TArray<FPCGPinProperties> UPVBaseSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, GetOutputPinTypeIdentifier());
	Pin.SetAllowMultipleConnections(true);
	Pin.bAllowMultipleData = false;

	return Properties;
}

FPCGDataTypeIdentifier UPVBaseSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ EPCGDataType::Any };
}

FPCGDataTypeIdentifier UPVBaseSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ EPCGDataType::Any };
}

#if WITH_EDITOR
void UPVBaseSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		if (const UClass* Class = GetClass())
		{
			PV::Analytics::SendNodeTweakedEvent(Class->GetName(), PropertyChangedEvent.Property);
		}
	}
}

#endif

FPCGElementPtr UPVBaseSettings::CreateElement() const
{
	return MakeShared<FPVBaseElement>();
}

void FPVBaseElement::PostExecuteInternal(FPCGContext* InContext) const
{
	IPCGElement::PostExecuteInternal(InContext);

#if WITH_EDITORONLY_DATA
	const UPVBaseSettings* Settings = InContext->GetInputSettings<UPVBaseSettings>();
	check(Settings);

	for (FPCGTaggedData& OutputData : InContext->OutputData.TaggedData)
	{
		if (OutputData.Data && OutputData.Data.IsA<UPVData>())
		{
			if (const UPVData* ProceduralVegetationData = Cast<UPVData>(OutputData.Data))
			{
				FPVDebugSettings DebugSettings = ProceduralVegetationData->GetDebugSettings();
				DebugSettings.VisualizationSettings = Settings->GetDebugVisualizationSettings().VisualizationSettings;
				ProceduralVegetationData->SetDebugSettings(MoveTemp(DebugSettings));
			}
		}
	}
#endif
}

bool FPVBaseElement::ExecuteInternal(FPCGContext* Context) const
{
	return true;
}