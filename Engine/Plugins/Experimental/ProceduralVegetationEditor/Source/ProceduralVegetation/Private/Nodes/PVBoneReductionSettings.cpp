// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVBoneReductionSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVMeshData.h"
#include "Data/PCGBasePointData.h"
#include "Facades/PVBoneFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVBoneReductionSettings"

#if WITH_EDITOR
FLinearColor UPVBoneReductionSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::BoneReduction;
}

FText UPVBoneReductionSettings::GetCategoryOverride() const
{
	return PV::Categories::Mesh;
}


FText UPVBoneReductionSettings::GetDefaultNodeTitle() const 
{ 
	return LOCTEXT("NodeTitle", "Bone Reduction"); 
}

FText UPVBoneReductionSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Reduce the bone count of the skeletal mesh (trades wind-animation accuracy for performance)."
		"\n\n"
		"Simplifies the skeletal hierarchy. Fewer bones = faster runtime cost but less accurate sway. "
	);
}
#endif

FPCGDataTypeIdentifier UPVBoneReductionSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMesh::AsId() };
}

FPCGDataTypeIdentifier UPVBoneReductionSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMesh::AsId() };
}

FPCGElementPtr UPVBoneReductionSettings::CreateElement() const
{
	return MakeShared<FPVBoneReductionElement>();
}

bool FPVBoneReductionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVBoneReductionElement::ExecuteInternal);

	check(InContext);

	const UPVBoneReductionSettings* InputSettings = InContext->GetInputSettings<UPVBoneReductionSettings>();
	check(InputSettings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVMeshData* InputData = Cast<UPVMeshData>(Input.Data))
		{
			FManagedArrayCollection SourceCollection = InputData->GetCollection();
			PV::Facades::FBoneFacade BoneFacadeConst = PV::Facades::FBoneFacade(SourceCollection);
			if (!BoneFacadeConst.IsValid())
			{
				PCGLog::LogErrorOnGraph(FText::FromString(TEXT("Can not reduce bones. Mesh data not available.")), InContext);
				return true;
			}

			FManagedArrayCollection OutCollection;
			SourceCollection.CopyTo(&OutCollection);

			PV::Facades::FBoneFacade BoneFacade = PV::Facades::FBoneFacade(OutCollection);
			UPVMeshData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVMeshData>(InContext);

			TArray<PV::Facades::FBoneNode> BoneNodes;
			float ReductionStrength = InputSettings->Strength;

			if (!BoneFacade.CreateBoneData(BoneNodes, ReductionStrength))
			{
				PCGLog::InputOutput::LogInvalidInputDataError(InContext);
				return true;
			}
			
			BoneFacade.SetBoneDataToCollection(BoneNodes);
			
			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
