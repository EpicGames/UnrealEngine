// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "DataTypes/PVData.h"
#include "PVBoneReductionSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVBoneReductionSettings : public UPVBaseSettings
{
	GENERATED_BODY()

	friend class FPVBoneReductionElement;

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationBoneReduction")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{EPVRenderType::Bones, EPVRenderType::Mesh}; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::Bones, EPVRenderType::Mesh, EPVRenderType::FoliageAttachments }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

private:
	UPROPERTY(EditAnywhere, Category = "Bone Reduction", meta = (PCG_Overridable, ShowOnlyInnerProperties, ClampMin ="0", ClampMax = "1", UIMin = "0", UIMax = "1", Tooltip="How aggressively bones are reduced.\n\n0 = no reduction (full bone count, max wind accuracy). 1 = maximum reduction (fewest bones, fastest, least accurate wind)."))
	float Strength = 0;
};

class FPVBoneReductionElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
