// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVCarveSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVCarveSettings"

#if WITH_EDITOR
FLinearColor UPVCarveSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVCarveSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVCarveSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Carve"); 
}

FText UPVCarveSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Trim branches off the plant based on a chosen rule (length, height, world Z, or thickness)."
		"\n\n"
		"Removes branch from the plant structure using one of four reference systems. Useful for cleaning up low-hanging branches, thinning out fine twigs, or shaping the silhouette. Acts as a post-process — does not re-run the growth simulation."
	);
}
#endif

FPCGDataTypeIdentifier UPVCarveSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVCarveSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVCarveSettings::CreateElement() const
{
	return MakeShared<FPVCarveElement>();
}

bool FPVCarveElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVCarveElement::ExecuteInternal);

	check(InContext);

	const UPVCarveSettings* InputSettings = InContext->GetInputSettings<UPVCarveSettings>();
	check(InputSettings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection SourceCollection = InputData->GetCollection();

			FManagedArrayCollection OutCollection;
			SourceCollection.CopyTo(&OutCollection);

			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

			auto [CarveBasis, Carve] = InputSettings->CarveSettings;
			FPVCarve::ApplyCarve(OutCollection, SourceCollection, CarveBasis, Carve);

			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
