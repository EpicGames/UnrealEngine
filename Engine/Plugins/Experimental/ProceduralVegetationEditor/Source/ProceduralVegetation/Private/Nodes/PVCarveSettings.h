// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Implementations/PVCarve.h"
#include "DataTypes/PVData.h"
#include "PVCarveSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVCarveSettings : public UPVBaseSettings
{
	GENERATED_BODY()

	friend class FPVCarveElement;

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationCarve")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{EPVRenderType::PointData}; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Carve", meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPVCarveParams CarveSettings;
};

class FPVCarveElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
