// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVExportSettings.h"
#include "PCGContext.h"
#include "DataTypes/PVMeshData.h"
#include "Helpers/PCGHelpers.h"
#include "Misc/PackageName.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Helpers/PVAnalyticsHelper.h"
#include "PVCommon.h"


#define LOCTEXT_NAMESPACE "PVExportSettings"

#if WITH_EDITOR
FLinearColor UPVExportSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Subgraph;
}

FText UPVExportSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}


FText UPVExportSettings::GetDefaultNodeTitle() const
{
	return ExportSettings.MeshName.IsNone() 
		? LOCTEXT("NodeTitle", "Export") 
		: FText::FromName(ExportSettings.MeshName);
}

FText UPVExportSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Export the built mesh as a Static Mesh or Skeletal Mesh asset on disk."
		"\n\n"
		"Writes the final mesh to a content folder. Supports Static Mesh (with optional Nanite foliage support) or Skeletal Mesh (with optional bones + wind animation). If foliage meshes don't already have skeletal data and Skeletal Mesh export is selected, the node builds bone data for them."
	);
}
#endif

UPVExportSettings::UPVExportSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		static FString DefaultMeshName = "Export";
		static FString DefaultAssetPath = "/Game/ProceduralVegetation/";
		static FString DefaultWindSettingsPath = "/ProceduralVegetationEditor/SampleAssets/WindSettings/DefaultTreeWindSettings.DefaultTreeWindSettings";
		ExportSettings.Initialize(DefaultAssetPath, DefaultMeshName, DefaultWindSettingsPath);
	}
}

FPCGDataTypeIdentifier UPVExportSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMesh::AsId() };
}

TArray<FPCGPinProperties> UPVExportSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Any);
	Pin.SetRequiredPin();
	Pin.SetAllowMultipleConnections(false);
	Pin.bInvisiblePin = true;

	return Properties;
}

FPCGElementPtr UPVExportSettings::CreateElement() const
{
	return MakeShared<FPVExportElement>();
}

#if	WITH_EDITOR
void UPVExportSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPVExportParams, WindSettings))
	{
		if (ExportSettings.WindSettings)
		{
			PV::Analytics::SendWindSettingsChangeEvent(ExportSettings.WindSettings->GetPathName());
		}
	}
}

EPCGChangeType UPVExportSettings::GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPVExportSettings, ExportSettings))
	{
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FDirectoryPath, Path)
			||PropertyName == GET_MEMBER_NAME_CHECKED(FPVExportParams, MeshName))
		{
			return EPCGChangeType::Cosmetic;
		}
	}

	return Super::GetChangeTypeForProperty(PropertyChangedEvent);
}

#endif

bool FPVExportElement::ExecuteInternal(FPCGContext* InContext) const
{
	const UPVExportSettings* Settings = InContext->GetInputSettings<UPVExportSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVData* InputData = Cast<UPVData>(Input.Data))
		{
			FString Error;
			if (!Settings->ExportSettings.Validate(Error))
			{
				PCGLog::LogErrorOnGraph(FText::FromString(Error), InContext);
				return true;
			}

			InContext->OutputData = InContext->InputData;
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
