// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVExtractFromImageSettings.h"
#include "PVImportCommon.h"
#include "ProceduralVegetationModule.h"
#include "DataTypes/PVGrowthData.h"
#include "Helpers/PVImportHelpers.h"
#include "Helpers/PVPlantTraversalHelper.h"
#include "ProceduralVegetationPreset.h"
#include "Params/PVImportTexture2DParams.h"
#include "Implementations/PVImporter_Texture2D.h"

#include "PCGContext.h"

#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/ConstructorHelpers.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVExtractFromImageSettings"

namespace PVExtractFromImageSettings
{
	const static FName CombinedOutputPinName = TEXT("CombinedOutput");

	FName GetPlantPinName(int32 PlantIndex)
	{
		return *FString::Printf(TEXT("Plant %d"), PlantIndex);
	}

#if WITH_EDITOR
	const TArray<FPVVisualizationSettings> GetVisualizationSettings()
	{
		TArray<FPVVisualizationSettings> OutVisualizationSettings;

		FPVVisualizationSettings& PerimeterCurveVisualization = OutVisualizationSettings.AddDefaulted_GetRef();
		PerimeterCurveVisualization.DebugType = EPVDebugType::Custom;
		PerimeterCurveVisualization.bShowAnchorPoints = false;
		PerimeterCurveVisualization.AttributeToFilter = PVImportNames::PerimeterCurveAttribute;
		PerimeterCurveVisualization.CustomGroupToFilter = PVImportNames::PerimeterCurveGroup;
		PerimeterCurveVisualization.VisualizationMode = EPVDebugValueVisualizationMode::Curve;

		FPVVisualizationSettings& TipVisualization = OutVisualizationSettings.AddDefaulted_GetRef();
		TipVisualization.DebugType = EPVDebugType::Custom;
		TipVisualization.bShowAnchorPoints = false;
		TipVisualization.bDrawPointAsMesh = false;
		TipVisualization.bUsePivotAsPosition = false;
		TipVisualization.AttributeToFilter = PVImportNames::TipVisualizationSizeAttribute;
		TipVisualization.CustomPivotPositionAttributeName = PVImportNames::TipVisualizationPositionAttribute;
		TipVisualization.CustomGroupToFilter = PVImportNames::TipVisualizationGroup;
		TipVisualization.VisualizationMode = EPVDebugValueVisualizationMode::Sphere;
		TipVisualization.Color = FColor::Red;
		TipVisualization.GizmoScale = 0.2f;

		FPVVisualizationSettings& LabelVisualization = OutVisualizationSettings.AddDefaulted_GetRef();
		LabelVisualization.DebugType = EPVDebugType::Custom;
		LabelVisualization.bShowAnchorPoints = false;
		LabelVisualization.CustomGroupToFilter = PVImportNames::LabelsGroup;
		LabelVisualization.AttributeToFilter = PVImportNames::LabelTextAttribute;
		LabelVisualization.CustomPivotPositionAttributeName = PVImportNames::LabelPositionAttribute;
		LabelVisualization.CustomPivotScaleAttributeName = PVImportNames::LabelScaleAttribute;
		LabelVisualization.VisualizationMode = EPVDebugValueVisualizationMode::Text;
		LabelVisualization.Color = FColor::Black;
		LabelVisualization.TextScale = 15.f;
		LabelVisualization.TextOffset = FVector3f(0, LabelVisualization.TextScale / 4.f, -LabelVisualization.TextScale / 2.f);

		return OutVisualizationSettings;
	}

	void AddTextureQuadToCollection(
		FManagedArrayCollection& InOutCollection,
		const UMaterialInterface* TextureVisualizationMaterial,
		const FMatrix44f& Transform
	)
	{
		const TArray<FVector> Points = {
			FVector(Transform.TransformPosition(FVector3f(0.f, 0.f, 0.f))),
			FVector(Transform.TransformPosition(FVector3f(1.f, 0.f, 0.f))),
			FVector(Transform.TransformPosition(FVector3f(1.f, 1.f, 0.f))),
			FVector(Transform.TransformPosition(FVector3f(0.f, 1.f, 0.f))),
		};
		const TArray<FVector3f> Normals = {
			Transform.ToQuat().RotateVector(FVector3f(0, 0, 1)),
			Transform.ToQuat().RotateVector(FVector3f(0, 0, 1)),
			Transform.ToQuat().RotateVector(FVector3f(0, 0, 1)),
			Transform.ToQuat().RotateVector(FVector3f(0, 0, 1))
		};
		const TArray<UE::Geometry::FVector3i> Triangles = {
			{ 0, 1, 2 },
			{ 2, 3, 0 }
		};
		const TArray<FVector2f> UVs = {
			FVector2f(0, 0),
			FVector2f(1, 0),
			FVector2f(1, 1),
			FVector2f(0, 1),
		};
		const TSharedPtr<FGeometryCollection> MeshGeomCollection = GeometryCollection::MakeMeshElement(
			Points,
			Normals,
			Triangles,
			UVs,
			FTransform::Identity,
			FTransform::Identity,
			1
		);

		const static FName MaterialPathAttributeName("MaterialPath");
		auto& MaterialArray = MeshGeomCollection->AddAttribute<FString>(MaterialPathAttributeName, FGeometryCollection::MaterialGroup);
		MaterialArray[0] = TextureVisualizationMaterial ? TextureVisualizationMaterial->GetPathName() : TEXT("None");

		MeshGeomCollection->CopyTo(&InOutCollection);
	}

	void ArrangePlantsIn2DGrid(
		FManagedArrayCollection& InCollection,
		const FVector3f& InOffset
	)
	{
		using namespace PV::PlantTraversalHelper;

		struct FPlantOffsetInfo
		{
			int32 PlantNumber;
			FBox3f Bounds;
		};
		TArray<FPlantOffsetInfo> PlantsOffsetInfo;

		PV::FPointPositionAttributeView PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(InCollection);
		if (!PointPositionAttribute.IsValid())
		{
			return;
		}

		ForEachPlant(InCollection, [&](int32 PlantNumber)
		{
			FPlantOffsetInfo& PlantOffsetInfo = PlantsOffsetInfo.AddDefaulted_GetRef();
			PlantOffsetInfo.PlantNumber = PlantNumber;

			ForEachPlantPoint(InCollection, PlantNumber, [&](int32 BranchIndex, int32 PointIndex)
			{
				PlantOffsetInfo.Bounds += PointPositionAttribute[PointIndex];
				return EForEachResult::Continue;
			});
			return EForEachResult::Continue;
		});

		PlantsOffsetInfo.Sort([](const auto& A, const auto& B) { return A.PlantNumber < B.PlantNumber; });

		FVector3f Offset = InOffset;
		for (int32 i = 0; i < PlantsOffsetInfo.Num(); ++i)
		{
			Offset.Y -= PlantsOffsetInfo[i].Bounds.Max.Y + 10.f;

			ForEachPlantPoint(InCollection, PlantsOffsetInfo[i].PlantNumber, [&](int32 BranchIndex, int32 PointIndex)
			{
				FVector3f& PointPosition = PointPositionAttribute[PointIndex];
				PointPosition += Offset;
				return EForEachResult::Continue;
			});

			Offset.Y += PlantsOffsetInfo[i].Bounds.Min.Y;
		}
	}

	void AddPerimeterCurvesToCollection(
		FManagedArrayCollection& InOutCollection,
		const TArray<UE::Geometry::FPolygon2f>& InPerimeterCurves,
		const FMatrix44f& Transform
	)
	{
		InOutCollection.AddGroup(PVImportNames::PerimeterCurveGroup);
		InOutCollection.AddElements(InPerimeterCurves.Num(), PVImportNames::PerimeterCurveGroup);

		auto& PerimeterCurveAttribute = InOutCollection.AddAttribute<TArray<FVector3f>>(PVImportNames::PerimeterCurveAttribute, PVImportNames::PerimeterCurveGroup);
		for (int32 i = 0; i < InPerimeterCurves.Num(); i++)
		{
			const TArray<FVector2f>& Curve = InPerimeterCurves[i].GetVertices();
			for (const FVector2f& CurvePos : Curve)
			{
				PerimeterCurveAttribute[i].Add(Transform.TransformPosition(FVector3f(CurvePos.X, CurvePos.Y, 0.f)));
			}
		}
	}

	void AddTipVisualizationToCollection(
		FManagedArrayCollection& InOutCollection,
		const TArray<UE::Geometry::FPolygon2f>& InPerimeterCurves,
		const TArray<TArray<bool>>& InTips,
		const FMatrix44f& Transform
	)
	{
		if (InTips.Num() != InPerimeterCurves.Num())
		{
			return;
		}

		TArray<FVector3f> TipPositions;
		for (int32 CurveIndex = 0; CurveIndex < InPerimeterCurves.Num(); ++CurveIndex)
		{
			const TArray<FVector2f>& Vertices = InPerimeterCurves[CurveIndex].GetVertices();
			const TArray<bool>& CurveTips = InTips[CurveIndex];
			for (int32 VertexIndex = 0; VertexIndex < Vertices.Num() && VertexIndex < CurveTips.Num(); ++VertexIndex)
			{
				if (CurveTips[VertexIndex])
				{
					TipPositions.Add(Transform.TransformPosition(FVector3f(Vertices[VertexIndex].X, Vertices[VertexIndex].Y, 0.f)));
				}
			}
		}

		if (TipPositions.Num() > 0)
		{
			InOutCollection.AddGroup(PVImportNames::TipVisualizationGroup);
			InOutCollection.AddElements(TipPositions.Num(), PVImportNames::TipVisualizationGroup);
			auto& TipPositionAttribute = InOutCollection.AddAttribute<FVector3f>(PVImportNames::TipVisualizationPositionAttribute, PVImportNames::TipVisualizationGroup);
			auto& TipSizeAttribute = InOutCollection.AddAttribute<float>(PVImportNames::TipVisualizationSizeAttribute, PVImportNames::TipVisualizationGroup);
			for (int32 i = 0; i < TipPositions.Num(); ++i)
			{
				TipPositionAttribute[i] = TipPositions[i];
				TipSizeAttribute[i] = 1.f;
			}
		}
	}

	void AddPlantLabelsToCollection(FManagedArrayCollection& InOutCollection, const TMap<int32, int32>& PlantNumberToPlantSettingsIndex)
	{
		const PV::FBranchPlantNumberAttributeConstView BranchPlantNumberAttribute = PV::FBranchPlantNumberAttribute::FindAttribute(InOutCollection);
		const PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(InOutCollection);
		const PV::FBranchPointsAttributeConstView BranchPointsAttribute = PV::FBranchPointsAttribute::FindAttribute(InOutCollection);
		const PV::FPointPositionAttributeConstView PointPositionAttribute = PV::FPointPositionAttribute::FindAttribute(InOutCollection);

		TArray<PV::ImportHelper::FLabel> Labels;
		PV::PlantTraversalHelper::ForEachPlant(BranchPlantNumberAttribute, [&](int32 PlantNumber)
		{
			const int32* PlantSettingsIndex = PlantNumberToPlantSettingsIndex.Find(PlantNumber);
			if (!PlantSettingsIndex)
			{
				return PV::PlantTraversalHelper::EForEachResult::Continue;
			}

			const FVector3f RootPoint = PV::PlantTraversalHelper::GetPlantRootPoint(
				BranchPlantNumberAttribute,
				BranchParentNumberAttribute,
				BranchPointsAttribute,
				PointPositionAttribute,
				PlantNumber
			);
			Labels.Emplace(RootPoint + FVector3f(0.f, 0.f, -10.f), FString::Printf(TEXT("%d"), *PlantSettingsIndex), 1.0f);
			return PV::PlantTraversalHelper::EForEachResult::Continue;
		});

		PV::ImportHelper::AddLabelsToCollection(Labels, InOutCollection);
	}

	UMaterialInstanceDynamic* CreateTextureVisualizationMaterialInstance(
		const TObjectPtr<UMaterialInterface> TextureVisualizationMaterial,
		const TObjectPtr<UTexture2D> Texture2DAsset,
		bool bInvertTexture,
		float WhiteLevel,
		EPCGTextureColorChannel ColorChannel
	)
	{
		UMaterialInstanceDynamic* OutMaterialInstance = nullptr;

		if (TextureVisualizationMaterial)
		{
			OutMaterialInstance = UMaterialInstanceDynamic::Create(TextureVisualizationMaterial, nullptr);
			OutMaterialInstance->SetFlags(RF_Transient);

			const FTaskTagScope Scope(ETaskTag::EParallelGameThread); // Needed for SetTextureParameterValue
			OutMaterialInstance->SetTextureParameterValue(TEXT("Texture"), Texture2DAsset);
			OutMaterialInstance->SetScalarParameterValue(TEXT("InvertTextureSwitch"), bInvertTexture ? 1.f : 0.f);
			OutMaterialInstance->SetScalarParameterValue(TEXT("WhiteLevel"), WhiteLevel);

			switch (ColorChannel)
			{
			case EPCGTextureColorChannel::Red:
				OutMaterialInstance->SetVectorParameterValue(TEXT("Mask"), FLinearColor(1, 0, 0, 0));
				break;
			case EPCGTextureColorChannel::Green:
				OutMaterialInstance->SetVectorParameterValue(TEXT("Mask"), FLinearColor(0, 1, 0, 0));
				break;
			case EPCGTextureColorChannel::Blue:
				OutMaterialInstance->SetVectorParameterValue(TEXT("Mask"), FLinearColor(0, 0, 1, 0));
				break;
			case EPCGTextureColorChannel::Alpha:
				OutMaterialInstance->SetVectorParameterValue(TEXT("Mask"), FLinearColor(0, 0, 0, 1));
				break;
			default:
				break;
			}
		}

		return OutMaterialInstance;
	}

	void FillVisualizationCollection(
		FManagedArrayCollection& OutVisualizationCollection,
		const FManagedArrayCollection& CombinedPlantCollection,
		const TArray<UE::Geometry::FPolygon2f>& PerimeterCurves,
		const TArray<TArray<bool>>& Tips,
		const TMap<int32, int32>& PlantNumberToPlantSettingsIndex,
		UMaterialInstanceDynamic* TextureVisualizationMaterialInstance
	)
	{
		const FMatrix44f UVToWorldMatrix = PVTexture2DImporterVisualization::GetUVToWorldMatrix();

		AddTextureQuadToCollection(
			OutVisualizationCollection,
			TextureVisualizationMaterialInstance,
			UVToWorldMatrix
		);

		AddTipVisualizationToCollection(
			OutVisualizationCollection,
			PerimeterCurves,
			Tips,
			UVToWorldMatrix
		);

		AddPerimeterCurvesToCollection(
			OutVisualizationCollection,
			PerimeterCurves,
			UVToWorldMatrix
		);

		CombinedPlantCollection.CopyTo(&OutVisualizationCollection);
		ArrangePlantsIn2DGrid(OutVisualizationCollection, PVTexture2DImporterVisualization::Offset);

		AddPlantLabelsToCollection(OutVisualizationCollection, PlantNumberToPlantSettingsIndex);
	}
#endif
}

UPVExtractFromImageSettings::UPVExtractFromImageSettings()
{
#if WITH_EDITORONLY_DATA
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> TextureVisualizationMaterialRef(TEXT("/Script/Engine.Material'/ProceduralVegetationEditor/DefaultAssets/Visualization/Materials/M_TextureVisualization.M_TextureVisualization'"));
	TextureVisualizationMaterial = TextureVisualizationMaterialRef.Object;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
FLinearColor UPVExtractFromImageSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::InputOutput;
}

FText UPVExtractFromImageSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}

FText UPVExtractFromImageSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Extract from Image");
}

FText UPVExtractFromImageSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Derive a plant skeleton from a 2D image. Works best with a black and white mask"
		"\n\n"
		"Traces the outline of a 2D texture (e.g. a hand-drawn plant skeleton) and extracts a branch skeleton from it. Useful for quick sketch-to-3D plant authoring, or for creating stylized plants matching a concept image."
	);
}
#endif

TArray<FPCGPinProperties> UPVExtractFromImageSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPVExtractFromImageSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	
	FPCGPinProperties& CombinedOutputPin = Properties.Emplace_GetRef(PVExtractFromImageSettings::CombinedOutputPinName, GetOutputPinTypeIdentifier());
		CombinedOutputPin.bAllowMultipleData = false;

	for (int32 i = 0; i < Params.PlantSettings.Num(); i++)
	{
		FPCGPinProperties& Pin = Properties.Emplace_GetRef(PVExtractFromImageSettings::GetPlantPinName(i), GetOutputPinTypeIdentifier());
		Pin.bAllowMultipleData = false;
	}
	return Properties;
}

FPCGDataTypeIdentifier UPVExtractFromImageSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVExtractFromImageSettings::CreateElement() const
{
	return MakeShared<FPVExtractFromImageElement>();
}

bool FPVExtractFromImageElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVExtractFromImageElement::ExecuteInternal);

	check(InContext);

	using namespace PV::Texture2DImport;

	const UPVExtractFromImageSettings* Settings = InContext->GetInputSettings<UPVExtractFromImageSettings>();
	check(Settings);

	const FPVImportTexture2DParams& Params = Settings->Params;

	if (!Params.Texture2DAsset)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("MissingTexture2DAsset", "Texture2D asset not set"), InContext);
		return true;
	}
	
	FPVImportTexture2DOutput ImportTexture2DOutput;
	FManagedArrayCollection CombinedPlantCollection;
	TArray<UE::Geometry::FPolygon2f> PerimeterCurves;
	TArray<TArray<bool>> Tips;
	TMap<int32, int32> PlantNumberToPlantSettingsIndex;

	// Wrap output in ON_SCOPE_EXIT to ensure we always output some data, as it's needed for visualization (we want to visualize
	// the texture, image trace, and find tips, even if subsequent steps fail.
	ON_SCOPE_EXIT
	{
#if WITH_EDITOR
		UMaterialInstanceDynamic* TextureVisualizationMaterialInstance = PVExtractFromImageSettings::CreateTextureVisualizationMaterialInstance(
			Settings->TextureVisualizationMaterial, 
			Params.Texture2DAsset,
			Params.bInvertImage,
			Params.WhiteLevel,
			Params.ColorChannel
		);
		FManagedArrayCollection VisualizationCollection;
		const TArray<FPVVisualizationSettings> VisualizationSettings = PVExtractFromImageSettings::GetVisualizationSettings();
		PVExtractFromImageSettings::FillVisualizationCollection(
			VisualizationCollection,
			CombinedPlantCollection,
			PerimeterCurves,
			Tips,
			PlantNumberToPlantSettingsIndex,
			TextureVisualizationMaterialInstance
		);

		const auto AddVisualizationToOutput = [&](UPVGrowthData* OutVariantData)
		{
			const bool bVisualizeDebugCollection = ImportTexture2DOutput.DebugCollection.IsSet() && PV::Utilities::DebugModeEnabled();
			OutVariantData->AddManagedResource(TextureVisualizationMaterialInstance);
			if (bVisualizeDebugCollection)
			{
				OutVariantData->AddVisualizationCollection(ImportTexture2DOutput.DebugCollection.GetValue(), Settings->GetDebugVisualizationSettings().VisualizationSettings);
			}
			else
			{
				OutVariantData->AddVisualizationCollection(VisualizationCollection, VisualizationSettings);
			}
		};
#endif

		// Add Combined output
		{
			UPVGrowthData* OutVariantData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			OutVariantData->Initialize(MoveTemp(CombinedPlantCollection));

#if WITH_EDITOR
			AddVisualizationToOutput(OutVariantData);
#endif

			FPCGTaggedData& CollectionOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
			CollectionOutput.Data = OutVariantData;
			CollectionOutput.Pin = PVExtractFromImageSettings::CombinedOutputPinName;
		}

		// Add per-plant pin output
		for (int32 i = 0; i < ImportTexture2DOutput.PlantCollections.Num(); ++i)
		{
			UPVGrowthData* OutVariantData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			OutVariantData->Initialize(MoveTemp(ImportTexture2DOutput.PlantCollections[i]));

#if WITH_EDITOR
			// TODO: For now we add the visualization to all outputs as we cannot control which pin gets inspected
			AddVisualizationToOutput(OutVariantData);
#endif

			FPCGTaggedData& CollectionOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
			CollectionOutput.Data = OutVariantData;
			CollectionOutput.Pin = PVExtractFromImageSettings::GetPlantPinName(i);
		}
	};

	const ETracePerimeterCurvesResult TracePerimeterCurvesResult = TracePerimeterCurves(
		Params.Texture2DAsset,
		Params.SampleResolution,
		Params.bInvertImage,
		Params.WhiteLevel,
		Params.ColorChannel,
		Params.MinBoundsArea,
		Params.SmoothingIterations,
		Params.SimplificationAmount,
		PerimeterCurves
	);

	if (TracePerimeterCurvesResult != ETracePerimeterCurvesResult::Success)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("FailedToTraceImage", "No plants found in image"), InContext);
		return true;
	}

	const EFindTipsResult FindTipsResult = FindTips(PerimeterCurves, Params.TipAngleThresholdInDegrees, Params.MaxTipAngleSearchDist, Tips);
	if (FindTipsResult != EFindTipsResult::Success)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("FailedToFindTips", "No tips found on plants"), InContext);
		return true;
	}

	if (Params.PlantSettings.Num() == 0)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("NoPlantSettings", "No plant settings added"), InContext);
		return true;
	}

	const auto ImportResult = PV::Texture2DImport::ImportGrowthDataFromPerimeterCurves(
		PerimeterCurves,
		Tips,
		Params.PlantSettings,
		Settings->DebugParams.DebugState,
		ImportTexture2DOutput
	);
	if (ImportResult != EImportResult::Success)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("FailedToExtractGrowthData", "Failed to extract plant from image"), InContext);
		return true;
	}

	PV::ImportHelper::CreateEmptyGrowthData(CombinedPlantCollection, 0, 0);

	int32 CurrentPlantNumber = 0;
	for (int32 i = 0; i < ImportTexture2DOutput.PlantCollections.Num(); ++i)
	{
		const FManagedArrayCollection& Collection = ImportTexture2DOutput.PlantCollections[i];
		const auto Result = PV::ImportHelper::AppendGrowthData(CombinedPlantCollection, Collection);
		if (Result == PV::ImportHelper::EAppendGrowthDataResult::Success)
		{
			PlantNumberToPlantSettingsIndex.Add(CurrentPlantNumber, i);
			CurrentPlantNumber += 1;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
