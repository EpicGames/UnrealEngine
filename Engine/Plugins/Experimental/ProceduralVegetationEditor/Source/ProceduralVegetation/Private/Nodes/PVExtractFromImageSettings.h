// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Params/PVImportTexture2DParams.h"

#include "PVExtractFromImageSettings.generated.h"

class UMaterialInterface;

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVExtractFromImageSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	
	UPVExtractFromImageSettings();
	
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationExtractFromImage")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData, EPVRenderType::Mesh }; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::PointData, EPVRenderType::Mesh }; }
	virtual bool IsCollectionRenderingEnabled() const override { return false; }
	virtual ELevelViewportType GetPreferredViewportType() const { return ELevelViewportType::LVT_OrthoNegativeYZ; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category="Importer Settings", meta = (ShowOnlyInnerProperties))
	FPVImportTexture2DParams Params;

	UPROPERTY(EditAnywhere, Transient, Category = DebugSettings, meta = (EditCondition = "bDebug", EditConditionHides))
	FPVImportTexture2DDebugParams DebugParams;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "bDebug", EditConditionHides))
	TObjectPtr<UMaterialInterface> TextureVisualizationMaterial;
#endif // WITH_EDITOR_ONLY_DATA
};

class FPVExtractFromImageElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
