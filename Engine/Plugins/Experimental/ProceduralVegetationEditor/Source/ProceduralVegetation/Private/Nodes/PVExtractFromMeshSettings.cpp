// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVExtractFromMeshSettings.h"
#include "PVImportCommon.h"
#include "Implementations/PVImporter_StaticMesh.h"
#include "PCGContext.h"
#include "ProceduralVegetationModule.h"
#include "DataTypes/PVGrowthData.h"
#include "ProceduralVegetationPreset.h"
#include "Engine/StaticMesh.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVExtractFromMeshSettings"

#if WITH_EDITOR
FLinearColor UPVExtractFromMeshSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::InputOutput;
}

FText UPVExtractFromMeshSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}


FText UPVExtractFromMeshSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Extract from Mesh"); 
}

FText UPVExtractFromMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Derive a plant skeleton from a static mesh."
		"\n\n"
		"Traces the geometry of a Static Mesh asset and extracts a branch skeleton from it. Material filters let you keep only the bark/structural portions while ignoring leaves or other elements. Useful for converting hand-modeled trees into procedural skeletons that downstream nodes can grow and mesh."
	);
}
#endif

TArray<FPCGPinProperties> UPVExtractFromMeshSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGDataTypeIdentifier UPVExtractFromMeshSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVExtractFromMeshSettings::CreateElement() const
{
	return MakeShared<FPVExtractFromMeshElement>();
}

bool FPVExtractFromMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVExtractFromMeshElement::ExecuteInternal);

	check(InContext);

	const UPVExtractFromMeshSettings* Settings = InContext->GetInputSettings<UPVExtractFromMeshSettings>();
	check(Settings);

	if (!Settings->Params.StaticMeshAsset)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("MissingStaticMeshAsset", "Static Mesh asset not set"), InContext);
		return true;
	}

	if (Settings->Params.StaticMeshAsset->IsCompiling())
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("StaticMeshAssetNotReady", "Static Mesh asset not ready"), InContext);
		return false;
	}

	if (Settings->Params.MaterialsToKeep.Num() == 0)
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("NoMaterialsSelected", "No materials to keep set. Output will be empty"), InContext);
		return true;
	}
	
	FPVImportStaticMeshOutput Output;
	const auto Result = PV::StaticMeshImport::ImportGrowthDataFromStaticMesh(Settings->Params, Settings->DebugParams, Output);
	if (Result != PV::StaticMeshImport::EImportResult::Success)
	{
		return Result == PV::StaticMeshImport::EImportResult::MeshNotReady ? false : true;
	}

	UPVGrowthData* OutVariantData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

	if (Output.DebugCollection.IsSet() && PV::Utilities::DebugModeEnabled())
	{
		OutVariantData->Initialize(MoveTemp(Output.DebugCollection.GetValue()));
	}
	else
	{
		OutVariantData->Initialize(MoveTemp(Output.GrowthDataCollection));
	}

#if WITH_EDITOR
	const TArray<FPVVisualizationSettings> VisualizationSettings = Invoke([&]()
	{
		TArray<FPVVisualizationSettings> OutVisualizationSettings;

		if (Settings->DebugParams.bShowFoundTips)
		{
			FPVVisualizationSettings& TipVisualization = OutVisualizationSettings.AddDefaulted_GetRef();
			TipVisualization.DebugType = EPVDebugType::Custom;
			TipVisualization.bShowAnchorPoints = false;
			TipVisualization.bDrawPointAsMesh = false;
			TipVisualization.bUsePivotAsPosition = false;
			TipVisualization.AttributeToFilter = PVImportNames::TipVisualizationPositionAttribute;
			TipVisualization.CustomGroupToFilter = PVImportNames::TipVisualizationGroup;
			TipVisualization.VisualizationMode = EPVDebugValueVisualizationMode::Point;
			TipVisualization.Color = FColor::Red;
			TipVisualization.GizmoScale = 6.5f;
		}

		if (Settings->DebugParams.bShowFoundBranchCurves)
		{
			FPVVisualizationSettings& BranchHierarchyVisualization = OutVisualizationSettings.AddDefaulted_GetRef();
			BranchHierarchyVisualization.DebugType = EPVDebugType::Custom;
			BranchHierarchyVisualization.bShowAnchorPoints = false;
			BranchHierarchyVisualization.bDrawPointAsMesh = false;
			BranchHierarchyVisualization.AttributeToFilter = PVImportNames::BranchHierarchyAttribute;
			BranchHierarchyVisualization.CustomGroupToFilter = PVImportNames::BranchHierarchyGroup;
			BranchHierarchyVisualization.VisualizationMode = EPVDebugValueVisualizationMode::Curve;
			BranchHierarchyVisualization.Color = FColor::White;
			BranchHierarchyVisualization.DepthPriorityGroup = ESceneDepthPriorityGroup::SDPG_Foreground;
		}

		return OutVisualizationSettings;
	});

	OutVariantData->AddVisualizationCollection(Output.VisualizationCollection, VisualizationSettings);
#endif

	FPCGTaggedData& CollectionOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
	CollectionOutput.Data = OutVariantData;
	CollectionOutput.Pin = TEXT("Output");

	return true;
}

#undef LOCTEXT_NAMESPACE
