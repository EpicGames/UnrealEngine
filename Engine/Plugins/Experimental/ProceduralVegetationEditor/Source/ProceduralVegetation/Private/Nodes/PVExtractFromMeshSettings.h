// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Params/PVImportStaticMeshParams.h"

#include "PVExtractFromMeshSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVExtractFromMeshSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationExtractFromMesh")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData, EPVRenderType::Mesh }; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::PointData, EPVRenderType::Mesh }; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category="Settings", meta = (ShowOnlyInnerProperties))
	FPVImportStaticMeshParams Params;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "bDebug", EditConditionHides))
	FPVImportStaticMeshDebugParams DebugParams;
};

class FPVExtractFromMeshElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
