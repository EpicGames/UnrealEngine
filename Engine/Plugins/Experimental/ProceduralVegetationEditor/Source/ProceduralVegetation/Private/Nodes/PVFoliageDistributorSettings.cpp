// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVFoliageDistributorSettings.h"
#include "PVCommon.h"
#include "PCGNode.h"
#include "DataTypes/PVDistributionSettingsData.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVMeshData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVFoliageData.h"
#include "Facades/PVFoliageConditionFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Helpers/PVHormoneDistributionHelper.h"
 
#define LOCTEXT_NAMESPACE "PVFoliageDistributorSettings"
 
#if WITH_EDITOR
FLinearColor UPVFoliageDistributorSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Foliage;
}

FText UPVFoliageDistributorSettings::GetCategoryOverride() const
{
	return PV::Categories::Foliage;
}


FText UPVFoliageDistributorSettings::GetDefaultNodeTitle() const 
{ 
	return LOCTEXT("NodeTitle", "Foliage Distributor"); 
}
 
FText UPVFoliageDistributorSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Place foliage meshes onto a meshed plant using hormone-based or parametric rules."
		"\n\n"
		"Takes growth data from a Grower plus a Foliage Palette and produces foliage instances (each instance = one foliage-mesh transform). Choose between two distribution modes: Hormone Based (uses the simulation's growth data — natural feel) or Parametric (gradient and ramp based settings for more artistic control)."
	);
}

bool UPVFoliageDistributorSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	if (!Node)
	{
		return true;
	}

	const FName PropertyName = InProperty->GetFName();
	const UStruct* OwnerStruct = InProperty->GetOwnerStruct();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVFoliageDistributorSettings, ParametricSettings))
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionParametricSettingsInputLabel);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVFoliageDistributorSettings, HormoneBasedSettings))
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionHormoneBasedSettingsInputLabel);
	}

	// VectorSettings uses ShowOnlyInnerProperties, so CanEditChange is called for each inner field.
	// Check both the struct property itself and any property owned by the struct.
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVFoliageDistributorSettings, VectorSettings) ||
		OwnerStruct == FPVDistributionVectorParams::StaticStruct())
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionVectorSettingsInputLabel);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVFoliageDistributorSettings, ConditionSettings) ||
		OwnerStruct == FPVDistributionConditionParams::StaticStruct())
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionConditionSettingsInputLabel);
	}

	return true;
}
#endif
 
TArray<FPCGPinProperties> UPVFoliageDistributorSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties = Super::InputPinProperties();
	
	FPCGPinProperties& PhyllotaxyPin = Properties.Emplace_GetRef(PV::Pins::FoliageDistributorFoliageInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoFoliage::AsId()});
	PhyllotaxyPin.SetAllowMultipleConnections(false);
	PhyllotaxyPin.SetRequiredPin();

	FPCGPinProperties& ParametricPin = Properties.Emplace_GetRef(PV::Pins::DistributionParametricSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionParametricSettings::AsId() });
	ParametricPin.SetAllowMultipleConnections(false);
	ParametricPin.SetAdvancedPin();
	ParametricPin.bAllowMultipleData = false;

	FPCGPinProperties& HormoneBasedPin = Properties.Emplace_GetRef(PV::Pins::DistributionHormoneBasedSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionHormoneBasedSettings::AsId() });
	HormoneBasedPin.SetAllowMultipleConnections(false);
	HormoneBasedPin.SetAdvancedPin();
	HormoneBasedPin.bAllowMultipleData = false;

	FPCGPinProperties& VectorPin = Properties.Emplace_GetRef(PV::Pins::DistributionVectorSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionVectorSettings::AsId() });
	VectorPin.SetAllowMultipleConnections(false);
	VectorPin.SetAdvancedPin();
	VectorPin.bAllowMultipleData = false;

	FPCGPinProperties& ConditionsPin = Properties.Emplace_GetRef(PV::Pins::DistributionConditionSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionConditionSettings::AsId() });
	ConditionsPin.SetAllowMultipleConnections(false);
	ConditionsPin.SetAdvancedPin();
	ConditionsPin.bAllowMultipleData = false;

	return Properties;
}
 
FPCGDataTypeIdentifier UPVFoliageDistributorSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMesh::AsId() };
}
 
FPCGDataTypeIdentifier UPVFoliageDistributorSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMesh::AsId() };
}
 
FPCGElementPtr UPVFoliageDistributorSettings::CreateElement() const
{
	return MakeShared<FPVFoliageDistributorElement>();
}
 
bool FPVFoliageDistributorElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVFoliageDistributorElement::Execute);
 
	check(InContext);
 
	const UPVFoliageDistributorSettings* Settings = InContext->GetInputSettings<UPVFoliageDistributorSettings>();
	check(Settings);

	FPVDistributionParametricParams   ParametricSettings   = Settings->ParametricSettings;
	FPVDistributionHormoneBasedParams HormoneBasedSettings = Settings->HormoneBasedSettings;
	FPVDistributionVectorParams       VectorSettings       = Settings->VectorSettings;
	FPVDistributionConditionParams    ConditionSettings    = Settings->ConditionSettings;

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionParametricSettingsInputLabel))
	{
		if (const UPVDistributionParametricSettingsData* Data = Cast<UPVDistributionParametricSettingsData>(TaggedData.Data))
		{
			ParametricSettings = Data->Params;
		}
	}

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionHormoneBasedSettingsInputLabel))
	{
		if (const UPVDistributionHormoneBasedSettingsData* Data = Cast<UPVDistributionHormoneBasedSettingsData>(TaggedData.Data))
		{
			HormoneBasedSettings = Data->Params;
		}
	}

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionVectorSettingsInputLabel))
	{
		if (const UPVDistributionVectorSettingsData* Data = Cast<UPVDistributionVectorSettingsData>(TaggedData.Data))
		{
			VectorSettings = Data->Params;
		}
	}

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionConditionSettingsInputLabel))
	{
		if (const UPVDistributionConditionSettingsData* Data = Cast<UPVDistributionConditionSettingsData>(TaggedData.Data))
		{
			ConditionSettings = Data->Params;
		}
	}

	FManagedArrayCollection FoliageCollection;
	
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PV::Pins::FoliageDistributorFoliageInputLabel))
	{
		if(const UPVFoliageData* FoliageData = Cast<UPVFoliageData>(Input.Data))
		{
			FoliageCollection = FoliageData->GetCollection();
		}		
	}
	
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if(const UPVMeshData* InputData = Cast<UPVMeshData>(Input.Data))
		{
			FManagedArrayCollection Collection = InputData->GetCollection();
			
			UPVMeshData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVMeshData>(InContext);
			
			if (Settings->Mode == EPVDistributionSettingsMode::HormoneBasedSettings)
			{
				const FHormoneSettings::FDistributionSettings DistributionSettings = {
					.EthyleneThreshold = HormoneBasedSettings.DistributionSettings.EthyleneThreshold,
					.InstanceSpacing = HormoneBasedSettings.DistributionSettings.InstanceSpacing,
					.InstanceSpacingRamp = &HormoneBasedSettings.DistributionSettings.InstanceSpacingRamp,
					.InstanceSpacingRampEffect = HormoneBasedSettings.DistributionSettings.InstanceSpacingRampEffect,
					.MaxPerBranch = HormoneBasedSettings.DistributionSettings.MaxPerBranch
				};

				const FHormoneSettings::FScaleSettings ScaleSettings = {
					.BaseScale = HormoneBasedSettings.ScaleSettings.BaseScale,
					.BranchScaleImpact = HormoneBasedSettings.ScaleSettings.BranchScaleImpact,
					.MinScale = HormoneBasedSettings.ScaleSettings.MinScale,
					.MaxScale = HormoneBasedSettings.ScaleSettings.MaxScale,
					.RandomScaleMin = HormoneBasedSettings.ScaleSettings.RandomScaleMin,
					.RandomScaleMax = HormoneBasedSettings.ScaleSettings.RandomScaleMax,
					.ScaleRamp = &HormoneBasedSettings.ScaleSettings.ScaleRamp
				};

				const FHormoneSettings::FAxilSettings AxilSettings = {
					.OverrideAxilAngle = HormoneBasedSettings.AngleSettings.OverrideAxilAngle,
					.AxilAngle = HormoneBasedSettings.AngleSettings.AxilAngle,
					.AxilAngleRamp = &HormoneBasedSettings.AngleSettings.AxilAngleRamp,
					.AxilAngleRampUpperValue = HormoneBasedSettings.AngleSettings.AxilAngleRampUpperValue,
					.AxilAngleRampEffect = HormoneBasedSettings.AngleSettings.AxilAngleRampEffect
				};

				const FHormoneSettings::FPhyllotaxySettings PhyllotaxySettings = {
					.PhyllotaxyType = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyType,
					.PhyllotaxyFormation = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyFormation,
					.MinimumNodeBuds = HormoneBasedSettings.PhyllotaxySettings.MinimumNodeBuds,
					.MaximumNodeBuds = HormoneBasedSettings.PhyllotaxySettings.MaximumNodeBuds,
					.bSingleBudTip = HormoneBasedSettings.PhyllotaxySettings.bSingleBudTip,
					.PhyllotaxyAdditionalAngle = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyAdditionalAngle,
					.ResetPhyllotaxy = HormoneBasedSettings.PhyllotaxySettings.ResetPhyllotaxy,
					.PhyllotaxyOffset = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyOffset
				};

				PVE_PARAM_DEBUG_INIT(Settings, OutManagedArrayCollectionData);
	
				FPVFoliage::DistributeFoliageWithHormoneBasedSettings(
					Collection, 
					FoliageCollection, 
					DistributionSettings, 
					ScaleSettings, 
					AxilSettings,
					PhyllotaxySettings,
					VectorSettings,
					ConditionSettings,
					Settings->RandomSeed,
					Settings->ChainMaskDistance,
					Settings->TrunkOffset
				);
				
				PVE_PARAM_DEBUG_END()
			}
			else
			{
				const FParametricSettings::FSpacingSettings SpacingSettings = {
					.BranchDensity = ParametricSettings.SpacingSettings.BranchDensity,
					.RelativeStart = ParametricSettings.SpacingSettings.RelativeStart,
					.RelativeEnd = ParametricSettings.SpacingSettings.RelativeEnd,
					.LimitStartGeneration = ParametricSettings.SpacingSettings.LimitStartGeneration,
					.StartGeneration = ParametricSettings.SpacingSettings.StartGeneration,
					.LimitEndGeneration = ParametricSettings.SpacingSettings.LimitEndGeneration,
					.EndGeneration = ParametricSettings.SpacingSettings.EndGeneration,
					.SpacingBasis = ParametricSettings.SpacingSettings.SpacingBasis == EPVDistributionBasis::Plant
					? FParametricSettings::EDistributionBasis::Plant
					: FParametricSettings::EDistributionBasis::Branch,
					.SpacingRamp = &ParametricSettings.SpacingSettings.SpacingRamp
				};

				const FParametricSettings::FPhyllotaxySettings PhyllotaxySettings = {
					.PhyllotaxyType = ParametricSettings.PhyllotaxySettings.PhyllotaxyType,
					.PhyllotaxyFormation = ParametricSettings.PhyllotaxySettings.PhyllotaxyFormation,
					.MinimumNodeBuds = ParametricSettings.PhyllotaxySettings.MinimumNodeBuds,
					.MaximumNodeBuds = ParametricSettings.PhyllotaxySettings.MaximumNodeBuds,
					.bSingleBudTip = ParametricSettings.PhyllotaxySettings.bSingleBudTip,
					.PhyllotaxyAdditionalAngle = ParametricSettings.PhyllotaxySettings.PhyllotaxyAdditionalAngle,
					.ResetPhyllotaxy = ParametricSettings.PhyllotaxySettings.ResetPhyllotaxy,
					.PhyllotaxyOffset = ParametricSettings.PhyllotaxySettings.PhyllotaxyOffset
				};

				const FParametricSettings::FAngleSettings AngleSettings = {
					.Rotation = ParametricSettings.AngleSettings.Rotation,
					.AxilAngle = ParametricSettings.AngleSettings.AxilAngle,
					.RandomizeAxilAngleMinimum = ParametricSettings.AngleSettings.RandomizeAxilAngleMinimum,
					.RandomizeAxilAngleMaximum = ParametricSettings.AngleSettings.RandomizeAxilAngleMaximum,
					.AxilAngleRampBasis = ParametricSettings.AngleSettings.AxilAngleRampBasis == EPVDistributionBasis::Plant
					? FParametricSettings::EDistributionBasis::Plant
					: FParametricSettings::EDistributionBasis::Branch,
					.AxilAngleRamp = &ParametricSettings.AngleSettings.AxilAngleRamp
				};

				const FParametricSettings::FScaleSettings ScaleSettings = {
					.ScaleRampBasis = ParametricSettings.ScaleSettings.ScaleRampBasis == EPVDistributionBasis::Plant
					? FParametricSettings::EDistributionBasis::Plant
					: FParametricSettings::EDistributionBasis::Branch,
					.BaseScale = ParametricSettings.ScaleSettings.BaseScale,
					.RandomizeScaleMinimum = ParametricSettings.ScaleSettings.RandomizeScaleMinimum,
					.RandomizeScaleMaximum = ParametricSettings.ScaleSettings.RandomizeScaleMaximum,
					.ScaleRamp = &ParametricSettings.ScaleSettings.ScaleRamp,
					.BranchScaleImpact = ParametricSettings.ScaleSettings.BranchScaleImpact
				};
				
				FPVFoliage::DistributeFoliageWithParametricSettings(
					Collection,
					FoliageCollection,
					SpacingSettings,
					PhyllotaxySettings,
					AngleSettings,
					ScaleSettings,
					VectorSettings,
					ConditionSettings,
					Settings->RandomSeed,
					Settings->ChainMaskDistance,
					Settings->TrunkOffset
				);
			}
 
			OutManagedArrayCollectionData->Initialize(MoveTemp(Collection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}
	
	return true;
}
 
#undef LOCTEXT_NAMESPACE
