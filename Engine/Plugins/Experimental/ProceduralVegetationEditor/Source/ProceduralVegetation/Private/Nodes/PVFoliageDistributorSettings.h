// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVDistributionParams.h"
#include "Nodes/PVBaseSettings.h"

#include "PVFoliageDistributorSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVFoliageDistributorSettings : public UPVBaseSettings
{
	GENERATED_BODY()
 
	friend class FPVFoliageDistributorElement;
	
public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationFoliageDistributor")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::Mesh, EPVRenderType::Foliage }; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::Mesh, EPVRenderType::Foliage, EPVRenderType::FoliageAttachments }; }
#endif
 
protected:

#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
 
	virtual FPCGElementPtr CreateElement() const override;
 
private:
 
	UPROPERTY(EditAnywhere, Category="Distribution Settings Mode", meta=(PCG_Overridable, Tooltip="Choose between hormone-based or parametric placement.\n\nHormone Based: leverages the Grower's growth data in order to produce more botanically correct and realistic foliage that mirrors the growth simulation. Parametric: gradient and ramp based control for more artistic control with less reliance on botanical properties."))
	EPVDistributionSettingsMode Mode = EPVDistributionSettingsMode::HormoneBasedSettings;

	UPROPERTY(EditAnywhere, Category="Parametric Settings",
		meta=(PCG_NotOverridable, EditCondition="Mode == EPVDistributionSettingsMode::ParametricSettings", EditConditionHides, Tooltip="Parametric distribution rules using ramps and gradients."))
	FPVDistributionParametricParams ParametricSettings;

	UPROPERTY(EditAnywhere, Category="Hormone Based Settings",
		meta=(PCG_NotOverridable, EditCondition="Mode == EPVDistributionSettingsMode::HormoneBasedSettings", EditConditionHides, Tooltip="Hormone-driven distribution rules using the Grower's growth data."))
	FPVDistributionHormoneBasedParams HormoneBasedSettings;

	UPROPERTY(EditAnywhere, Category="Vector Settings",
		meta=(PCG_NotOverridable, ShowOnlyInnerProperties, Tooltip="Orientation rules: aim direction, face direction, roll/pitch/yaw."))
	FPVDistributionVectorParams VectorSettings;

	UPROPERTY(EditAnywhere, Category="ConditionSettings",
		meta=(PCG_NotOverridable, ShowOnlyInnerProperties, Tooltip="Per-condition filters (Light, Scale, UpAlignment, Tip, Health, Height, Generation)."))
	FPVDistributionConditionParams ConditionSettings;

	UPROPERTY(EditAnywhere, Category = "Misc Settings", meta = (PCG_Overridable, Tooltip="Random seed for the distributor.\n\nIndependent of the Grower's RandomSeed. Same seed = identical foliage layout."))
	int32 RandomSeed = 123456;

	UPROPERTY(EditAnywhere, Category = "Misc Settings", meta = (PCG_Overridable, ClampMin = 0, UIMin = 0, UIMax = 100, Tooltip="Distance from previous foliage to mask out new placement.\n\nWhen chaining multiple Foliage Distributors, this prevents the new one from placing too close to existing foliage. 0 = no masking. Higher values create larger clear zones."))
	float ChainMaskDistance = 0.f;
	
	UPROPERTY(EditAnywhere, Category="Misc Settings", meta=(Tooltip="Offset spawned foliage along the trunk thickness", PCG_Overridable, UIMin=0.0f, UIMax=1.0f))
	float TrunkOffset = 0.0f;
};
 
class FPVFoliageDistributorElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};