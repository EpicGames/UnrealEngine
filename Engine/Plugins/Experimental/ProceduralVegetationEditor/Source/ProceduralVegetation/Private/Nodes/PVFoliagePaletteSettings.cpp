// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVFoliagePaletteSettings.h"

#include "PCGEdge.h"
#include "ProceduralVegetationModule.h"
#include "PVCommon.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVMeshData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGStaticMeshResourceData.h"
#include "DataTypes/PVFoliageData.h"

#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"

#include "Facades/PVFoliageFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Helpers/PVAnalyticsHelper.h"
#include "Helpers/PVDistributionHelper.h"
#include "Implementations/PVFoliage.h"
 
#define LOCTEXT_NAMESPACE "PVFoliagePaletteSettings"
 
#if WITH_EDITOR
FLinearColor UPVFoliagePaletteSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Foliage;
}

FText UPVFoliagePaletteSettings::GetCategoryOverride() const
{
	return PV::Categories::Foliage;
}


FText UPVFoliagePaletteSettings::GetDefaultNodeTitle() const 
{ 
	return LOCTEXT("NodeTitle", "Foliage Palette");
}
 
FText UPVFoliagePaletteSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Define which mesh assets are used as leaves/needles/etc., feeding the Foliage Distributor."
		"\n\n"
		"Holds the list of leaf, needle, flower, or fruit meshes for this plant. The Foliage Distributor consumes this palette to scatter instances on the grown plant."
	);
}
 
void UPVFoliagePaletteSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
 
	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
 
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}
 
	if (MemberProperty && Property)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPVFoliagePaletteSettings, FoliageInfos))
		{
			FName Name = Property->GetFName();
			
			int Index = PropertyChangedEvent.GetArrayIndex(Name.ToString());
 
			if (Index >= 0 && FoliageInfos.Num() > Index)
			{
				if (!FoliageInfos[Index].Mesh.IsNull())
				{
					FString MeshPath = FoliageInfos[Index].Mesh.ToString();
 
					PV::Analytics::SendFoliageMeshChangeEvent(MeshPath);	
				}
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// For backwards compatibility 
			FoliageMeshes.Empty();
 
			for (int i = 0; i < FoliageInfos.Num(); i++)
			{
				FoliageMeshes.Add(FoliageInfos[i].Mesh);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			
		}
	}
}

void UPVFoliagePaletteSettings::ApplyDeprecation(UPCGNode* InOutNode)
{
	Super::ApplyDeprecation(InOutNode);
	
	for (const auto& Connection : PinConnectionsToUpdate)
	{
		if (Connection.Node.IsValid() && Connection.TargetNode.IsValid() && Connection.TargetNode->GetInputPin(Connection.ToPinLabel))
		{
			Connection.Node->AddEdgeTo(Connection.FromPinLabel, Connection.TargetNode.Get(), Connection.ToPinLabel);
		}
	}
}

void UPVFoliagePaletteSettings::ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InputPins,
                                                                 TArray<TObjectPtr<UPCGPin>>& OutputPins)
{
	Super::ApplyDeprecationBeforeUpdatePins(InOutNode, InputPins, OutputPins);

	if (InputPins.Num() > 0 && OutputPins.Num() > 0)
	{
		auto InputPin = InputPins[0];
		auto OutputPin = OutputPins[0];

		if (InputPin->Edges.Num() > 0 && OutputPin->Edges.Num() > 0)
		{
			auto InputEdge = InputPin->Edges[0];
			auto OutputEdge = OutputPin->Edges[0];
			
			{
				// Foliage palette connection to Foliage distributor

				FPinConnectionInfo ConnectionInfo;
				ConnectionInfo.Node = MakeWeakObjectPtr(InOutNode);
				ConnectionInfo.TargetNode = MakeWeakObjectPtr(OutputEdge->OutputPin->Node);
				ConnectionInfo.FromPinLabel = OutputEdge->GetInputPinLabel();
				ConnectionInfo.ToPinLabel = PV::Pins::FoliageDistributorFoliageInputLabel;

				PinConnectionsToUpdate.Add(ConnectionInfo);
			}

			{
				FPinConnectionInfo ConnectionInfo;
				
				// Foliage palette Input pin gets connection to distributor
				ConnectionInfo.TargetNode = MakeWeakObjectPtr(OutputEdge->OutputPin->Node);
				ConnectionInfo.Node = MakeWeakObjectPtr(InputEdge->InputPin->Node);
				ConnectionInfo.FromPinLabel = InputEdge->GetInputPinLabel();
				ConnectionInfo.ToPinLabel = OutputEdge->GetOutputPinLabel();
				
				PinConnectionsToUpdate.Add(ConnectionInfo);
			}
		}
	}
}
#endif

TArray<FPCGPinProperties> UPVFoliagePaletteSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGDataTypeIdentifier UPVFoliagePaletteSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoFoliage::AsId() };
}
 
FPCGElementPtr UPVFoliagePaletteSettings::CreateElement() const
{
	return MakeShared<FPVFoliageElement>();
}
 
void UPVFoliagePaletteSettings::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
 
	if (bOverrideFoliage)
	{
		if (FoliageInfos.Num() == 0)
		{
			for (int32 i = 0; i < FoliageMeshes.Num(); i++)
			{
				FoliageInfos.Emplace(!FoliageMeshes[i].IsValid(), FoliageMeshes[i]);
			}
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
 
bool FPVFoliageElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVFoliageElement::Execute);
 
	check(InContext);
 
	const UPVFoliagePaletteSettings* Settings = InContext->GetInputSettings<UPVFoliagePaletteSettings>();
	check(Settings);

	FManagedArrayCollection FoliageCollection;
 
	UPVFoliageData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVFoliageData>(InContext);
 
	PV::Facades::FFoliageFacade Facade(FoliageCollection);
 
	for (const FPVFoliageInfo& FoliageInfo : Settings->FoliageInfos)
	{
		if (!FoliageInfo.bUseAsMask && !PV::Utilities::DoesAssetExist(FoliageInfo.Mesh.ToSoftObjectPath()))
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("NullFoliageMesh", "A foliage mesh is [None] or doesn't exist"), InContext);
			return true;
		}
		if (const UObject* MeshObject = FoliageInfo.Mesh.ToSoftObjectPath().TryLoad(nullptr, StaticCast<ELoadFlags>(LOAD_Quiet | LOAD_NoWarn)))
		{
			bool bHasZeroBounds = false;
			bool bHasNaniteAssemblies = false;
			if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(MeshObject))
			{
				const FBoxSphereBounds Bounds = StaticMesh->GetBounds();
				bHasZeroBounds = Bounds.BoxExtent.IsZero() && Bounds.SphereRadius == 0;
#if WITH_EDITORONLY_DATA
				bHasNaniteAssemblies = !StaticMesh->GetNaniteSettings().NaniteAssemblyData.Nodes.IsEmpty();
#endif
			}
			if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(MeshObject))
			{
				const FBoxSphereBounds Bounds = SkeletalMesh->GetBounds();
				bHasZeroBounds = Bounds.BoxExtent.IsZero() && Bounds.SphereRadius == 0;
#if WITH_EDITORONLY_DATA
				bHasNaniteAssemblies = !SkeletalMesh->GetNaniteSettings().NaniteAssemblyData.Nodes.IsEmpty();
#endif
			}
			if (bHasZeroBounds)
			{
				PCGLog::LogErrorOnGraph(
					FText::Format(
						LOCTEXT("EmptyFoliageMesh", "A foliage mesh [{0}] has zero bounds. This is not supported."),
						FText::FromString(FoliageInfo.Mesh.GetAssetName())
					),
					InContext
				);
				return true;
			}
			if (bHasNaniteAssemblies)
			{
				PCGLog::LogErrorOnGraph(
					FText::Format(
						LOCTEXT("NaniteAssembliesFoliageMesh",
							"A foliage mesh [{0}] has Nanite Assemblies. Nanite Assembly foliage is not supported yet."),
						FText::FromString(FoliageInfo.Mesh.GetAssetName())
					),
					InContext
				);
				return true;
			}
		}
	}
	
	Facade.SetFoliageInfos(Settings->FoliageInfos);
 
	OutManagedArrayCollectionData->Initialize(MoveTemp(FoliageCollection));
	InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
 
	return true;
}
 
#undef LOCTEXT_NAMESPACE
