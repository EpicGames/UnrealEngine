// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
 
#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "DataTypes/PVData.h"
#include "PVFoliagePaletteSettings.generated.h"

struct FPinConnectionInfo
{
	TWeakObjectPtr<UPCGNode> Node = nullptr;
	TWeakObjectPtr<UPCGNode> TargetNode = nullptr;
	FName FromPinLabel = NAME_None;
	FName ToPinLabel = NAME_None;
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVFoliagePaletteSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationFoliage")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{EPVRenderType::FoliageGrid}; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{EPVRenderType::FoliageGrid, EPVRenderType::Foliage, EPVRenderType::FoliageAttachments, EPVRenderType::Mesh}; }
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	virtual void ApplyDeprecation(UPCGNode* InOutNode) override;
	virtual void ApplyDeprecationBeforeUpdatePins(UPCGNode* InOutNode, TArray<TObjectPtr<UPCGPin>>& InInputPins, TArray<TObjectPtr<UPCGPin>>& InOutputPins) override;
#endif
 
protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;
 
public:
	virtual void PostLoad() override;

	UE_DEPRECATED(5.8, "FoliageInfos can now be updated without Override foliage")
	UPROPERTY(EditAnywhere, Category="Meshes", meta=(PinHiddenByDefault, InlineEditConditionToggle, DeprecatedProperty, DeprecationMessage = "FoliageInfos can now be updated without Override foliage", Tooltip="(Deprecated) Override flag for foliage assignments.\n\nDeprecated. FoliageInfos can now be updated without an override toggle. Will be removed in a future release."))
	bool bOverrideFoliage = true;

	UE_DEPRECATED(5.8, "Use FoliageInfos instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FoliageInfos instead"))
	TArray<TSoftObjectPtr<UObject>> FoliageMeshes;

	UPROPERTY(EditAnywhere, Category = "Meshes",
		Meta = (AllowedClasses = "/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh", ShowOnlyInnerProperties, FullyExpand="true", Tooltip=
			"List of foliage meshes used for this plant.\n\nEach entry references a Static Mesh or Skeletal Mesh and sets condition values that decide where it spawns. The Foliage Distributor compares these condition values against each spawning point's attributes (light, scale, height, etc.) and places the entry whose conditions best match."))
	TArray<struct FPVFoliageInfo> FoliageInfos;

private:
	TArray<FPinConnectionInfo> PinConnectionsToUpdate;
};
 
class FPVFoliageElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
 
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
};