// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGraftDistributorSettings.h"
#include "PCGContext.h"
#include "ProceduralVegetationModule.h"
#include "DataTypes/PVGrowthData.h"
#include "DataTypes/PVDistributionSettingsData.h"
#include "ProceduralVegetationPreset.h"
#include "DataTypes/PVGrafterPaletteData.h"
#include "Facades/PVPointFacade.h"
#include "Implementations/PVGrafter.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVGraftDistributorSettings"

#if WITH_EDITOR
FLinearColor UPVGraftDistributorSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Growth;
}

FText UPVGraftDistributorSettings::GetCategoryOverride() const
{
	return PV::Categories::Growth;
}



FText UPVGraftDistributorSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Graft Distributor");
}

FText UPVGraftDistributorSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Distribute pre-grown sub-plants (grafts) onto a main plant's attachment points."
		"\n\n"
		"Takes the host plant's growth data plus a Grafter Palette and attaches sub-plant skeletons at chosen attachment positions. Distribution rules mirror the Foliage Distributor (parametric or hormone-based)."
	);
}

bool UPVGraftDistributorSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	if (!Node)
	{
		return true;
	}
	
	const FName PropertyName = InProperty->GetFName();
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FPVHormonePhyllotaxySettings, bSingleBudTip)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FPVParametricPhyllotaxySettings, bSingleBudTip))
	{
		return false;
	}
	
	const UStruct* OwnerStruct = InProperty->GetOwnerStruct();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVGraftDistributorSettings, ParametricSettings))
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionParametricSettingsInputLabel);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVGraftDistributorSettings, HormoneBasedSettings))
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionHormoneBasedSettingsInputLabel);
	}

	// VectorSettings uses ShowOnlyInnerProperties, so CanEditChange is called for each inner field.
	// Check both the struct property itself and any property owned by the struct.
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVGraftDistributorSettings, VectorSettings) ||
		OwnerStruct == FPVDistributionVectorParams::StaticStruct())
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionVectorSettingsInputLabel);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPVGraftDistributorSettings, ConditionSettings) ||
		OwnerStruct == FPVDistributionConditionParams::StaticStruct())
	{
		return !Node->IsInputPinConnected(PV::Pins::DistributionConditionSettingsInputLabel);
	}

	return true;
}
#endif

TArray<FPCGPinProperties> UPVGraftDistributorSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& Pin1 = Properties.Emplace_GetRef(PVGraftDistributorInputPins::Skeleton, GetSkeletonPinTypeIdentifier());
	Pin1.SetRequiredPin();
	Pin1.SetAllowMultipleConnections(false);
	Pin1.bAllowMultipleData = false;

	FPCGPinProperties& Pin2 = Properties.Emplace_GetRef(PVGraftDistributorInputPins::Graft, GetGraftPinTypeIdentifier());
	Pin2.SetRequiredPin();
	Pin2.SetAllowMultipleConnections(false);
	Pin2.bAllowMultipleData = false;

	FPCGPinProperties& ParametricPin = Properties.Emplace_GetRef(PV::Pins::DistributionParametricSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionParametricSettings::AsId() });
	ParametricPin.SetAllowMultipleConnections(false);
	ParametricPin.SetAdvancedPin();
	ParametricPin.bAllowMultipleData = false;

	FPCGPinProperties& HormoneBasedPin = Properties.Emplace_GetRef(PV::Pins::DistributionHormoneBasedSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionHormoneBasedSettings::AsId() });
	HormoneBasedPin.SetAllowMultipleConnections(false);
	HormoneBasedPin.SetAdvancedPin();
	HormoneBasedPin.bAllowMultipleData = false;

	FPCGPinProperties& VectorPin = Properties.Emplace_GetRef(PV::Pins::DistributionVectorSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionVectorSettings::AsId() });
	VectorPin.SetAllowMultipleConnections(false);
	VectorPin.SetAdvancedPin();
	VectorPin.bAllowMultipleData = false;

	FPCGPinProperties& ConditionsPin = Properties.Emplace_GetRef(PV::Pins::DistributionConditionSettingsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoDistributionConditionSettings::AsId() });
	ConditionsPin.SetAllowMultipleConnections(false);
	ConditionsPin.SetAdvancedPin();
	ConditionsPin.bAllowMultipleData = false;

	return Properties;
}

FPCGDataTypeIdentifier UPVGraftDistributorSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()};
}

FPCGDataTypeIdentifier UPVGraftDistributorSettings::GetSkeletonPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVGraftDistributorSettings::GetGraftPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrafterPalette::AsId() };
}

FPCGElementPtr UPVGraftDistributorSettings::CreateElement() const
{
	return MakeShared<FPVGraftDistributorElement>();
}

bool FPVGraftDistributorElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGraftDistributorElement::ExecuteInternal);

	check(InContext);

	const UPVGraftDistributorSettings* Settings = InContext->GetInputSettings<UPVGraftDistributorSettings>();
	check(Settings);

	FPVDistributionParametricParams   ParametricSettings   = Settings->ParametricSettings;
	FPVDistributionHormoneBasedParams HormoneBasedSettings = Settings->HormoneBasedSettings;
	FPVDistributionVectorParams       VectorSettings       = Settings->VectorSettings;
	FPVDistributionConditionParams    ConditionSettings    = Settings->ConditionSettings;

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionParametricSettingsInputLabel))
	{
		if (const UPVDistributionParametricSettingsData* Data = Cast<UPVDistributionParametricSettingsData>(TaggedData.Data))
		{
			ParametricSettings = Data->Params;
		}
	}

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionHormoneBasedSettingsInputLabel))
	{
		if (const UPVDistributionHormoneBasedSettingsData* Data = Cast<UPVDistributionHormoneBasedSettingsData>(TaggedData.Data))
		{
			HormoneBasedSettings = Data->Params;
		}
	}

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionVectorSettingsInputLabel))
	{
		if (const UPVDistributionVectorSettingsData* Data = Cast<UPVDistributionVectorSettingsData>(TaggedData.Data))
		{
			VectorSettings = Data->Params;
		}
	}

	for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::DistributionConditionSettingsInputLabel))
	{
		if (const UPVDistributionConditionSettingsData* Data = Cast<UPVDistributionConditionSettingsData>(TaggedData.Data))
		{
			ConditionSettings = Data->Params;
		}
	}

	auto SkeletonPinInputs = InContext->InputData.GetInputsByPin(PVGraftDistributorInputPins::Skeleton);
	auto GraftPinInputs = InContext->InputData.GetInputsByPin(PVGraftDistributorInputPins::Graft);

	if (SkeletonPinInputs.IsEmpty() || GraftPinInputs.IsEmpty())
	{
		return true;
	}

	const UPVGrowthData* SkeletonInputData = Cast<UPVGrowthData>(SkeletonPinInputs[0].Data);
	const UPVGrafterPaletteData* GraftPaletteInputData = Cast<UPVGrafterPaletteData>(GraftPinInputs[0].Data);
	if (!SkeletonInputData || !GraftPaletteInputData)
	{
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}

	FManagedArrayCollection SourceSkeletonCollection = SkeletonInputData->GetCollection();
	if (!PV::Utilities::IsValidGrowthData(SourceSkeletonCollection))
	{
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}

	const TArray<TObjectPtr<UPVGrowthData>>& GraftPaletteElements = GraftPaletteInputData->GetGrowthDataElements();
	if (GraftPaletteElements.IsEmpty())
	{
		return true;
	}
	
	TArray<FManagedArrayCollection> SourceGraftCollections;
	for (int32 i = 0; i < GraftPaletteElements.Num(); ++i)
	{
		const UPVGrowthData* GraftElement = GraftPaletteElements[i];
		if (!GraftElement)
		{
			continue;
		}

		FManagedArrayCollection SourceGraft = GraftElement->GetCollection();
		SourceGraftCollections.Add(SourceGraft);
	}
	
	if (SourceGraftCollections.Num() == 0)
	{
		return true;
	}

	FManagedArrayCollection OutCollection;
	SourceSkeletonCollection.CopyTo(&OutCollection);

	UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

	if (FPVGrafter::RemoveFoliageDataIfPresent(OutCollection))
	{
		PCGLog::LogWarningOnGraph(LOCTEXT("FoliageDataRemovedFromSourceSkeleton", "Foliage data removed from source skeleton"), InContext);
	}

	if (Settings->Mode == EPVDistributionSettingsMode::HormoneBasedSettings)
	{
		const FHormoneSettings::FDistributionSettings DistributionSettings = {
			.EthyleneThreshold = HormoneBasedSettings.DistributionSettings.EthyleneThreshold,
			.InstanceSpacing = HormoneBasedSettings.DistributionSettings.InstanceSpacing,
			.InstanceSpacingRamp = &HormoneBasedSettings.DistributionSettings.InstanceSpacingRamp,
			.InstanceSpacingRampEffect = HormoneBasedSettings.DistributionSettings.InstanceSpacingRampEffect,
			.MaxPerBranch = HormoneBasedSettings.DistributionSettings.MaxPerBranch
		};

		const FHormoneSettings::FPhyllotaxySettings PhyllotaxySettings = {
			.PhyllotaxyType = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyType,
			.PhyllotaxyFormation = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyFormation,
			.MinimumNodeBuds = HormoneBasedSettings.PhyllotaxySettings.MinimumNodeBuds,
			.MaximumNodeBuds = HormoneBasedSettings.PhyllotaxySettings.MaximumNodeBuds,
			.PhyllotaxyAdditionalAngle = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyAdditionalAngle,
			.ResetPhyllotaxy = HormoneBasedSettings.PhyllotaxySettings.ResetPhyllotaxy,
			.PhyllotaxyOffset = HormoneBasedSettings.PhyllotaxySettings.PhyllotaxyOffset
		};

		const FHormoneSettings::FScaleSettings ScaleSettings = {
			.BaseScale = HormoneBasedSettings.ScaleSettings.BaseScale,
			.BranchScaleImpact = HormoneBasedSettings.ScaleSettings.BranchScaleImpact,
			.MinScale = HormoneBasedSettings.ScaleSettings.MinScale,
			.MaxScale = HormoneBasedSettings.ScaleSettings.MaxScale,
			.RandomScaleMin = HormoneBasedSettings.ScaleSettings.RandomScaleMin,
			.RandomScaleMax = HormoneBasedSettings.ScaleSettings.RandomScaleMax,
			.ScaleRamp = &HormoneBasedSettings.ScaleSettings.ScaleRamp
		};

		const FHormoneSettings::FAxilSettings AxilSettings = {
			.OverrideAxilAngle = HormoneBasedSettings.AngleSettings.OverrideAxilAngle,
			.AxilAngle = HormoneBasedSettings.AngleSettings.AxilAngle,
			.AxilAngleRamp = &HormoneBasedSettings.AngleSettings.AxilAngleRamp,
			.AxilAngleRampUpperValue = HormoneBasedSettings.AngleSettings.AxilAngleRampUpperValue,
			.AxilAngleRampEffect = HormoneBasedSettings.AngleSettings.AxilAngleRampEffect
		};

		FPVGrafter::DistributeGraftWithHormoneBasedSettings(
			OutCollection,
			SourceGraftCollections,
			DistributionSettings,
			PhyllotaxySettings,
			ScaleSettings,
			AxilSettings,
			VectorSettings,
			ConditionSettings,
			Settings->RandomSeed,
			Settings->bRecomputeLight);
	}
	else
	{
		const FParametricSettings::FSpacingSettings SpacingSettings = {
			.BranchDensity = ParametricSettings.SpacingSettings.BranchDensity,
			.RelativeStart = ParametricSettings.SpacingSettings.RelativeStart,
			.RelativeEnd = ParametricSettings.SpacingSettings.RelativeEnd,
			.LimitStartGeneration = ParametricSettings.SpacingSettings.LimitStartGeneration,
			.StartGeneration = ParametricSettings.SpacingSettings.StartGeneration,
			.LimitEndGeneration = ParametricSettings.SpacingSettings.LimitEndGeneration,
			.EndGeneration = ParametricSettings.SpacingSettings.EndGeneration,
			.SpacingBasis = ParametricSettings.SpacingSettings.SpacingBasis == EPVDistributionBasis::Plant
			? FParametricSettings::EDistributionBasis::Plant
			: FParametricSettings::EDistributionBasis::Branch,
			.SpacingRamp = &ParametricSettings.SpacingSettings.SpacingRamp
		};

		const FParametricSettings::FPhyllotaxySettings PhyllotaxySettings = {
			.PhyllotaxyType = ParametricSettings.PhyllotaxySettings.PhyllotaxyType,
			.PhyllotaxyFormation = ParametricSettings.PhyllotaxySettings.PhyllotaxyFormation,
			.MinimumNodeBuds = ParametricSettings.PhyllotaxySettings.MinimumNodeBuds,
			.MaximumNodeBuds = ParametricSettings.PhyllotaxySettings.MaximumNodeBuds,
			.PhyllotaxyAdditionalAngle = ParametricSettings.PhyllotaxySettings.PhyllotaxyAdditionalAngle,
			.ResetPhyllotaxy = ParametricSettings.PhyllotaxySettings.ResetPhyllotaxy,
			.PhyllotaxyOffset = ParametricSettings.PhyllotaxySettings.PhyllotaxyOffset
		};

		const FParametricSettings::FAngleSettings AngleSettings = {
			.Rotation = ParametricSettings.AngleSettings.Rotation,
			.AxilAngle = ParametricSettings.AngleSettings.AxilAngle,
			.RandomizeAxilAngleMinimum = ParametricSettings.AngleSettings.RandomizeAxilAngleMinimum,
			.RandomizeAxilAngleMaximum = ParametricSettings.AngleSettings.RandomizeAxilAngleMaximum,
			.AxilAngleRampBasis = ParametricSettings.AngleSettings.AxilAngleRampBasis == EPVDistributionBasis::Plant
			? FParametricSettings::EDistributionBasis::Plant
			: FParametricSettings::EDistributionBasis::Branch,
			.AxilAngleRamp = &ParametricSettings.AngleSettings.AxilAngleRamp
		};

		const FParametricSettings::FScaleSettings ScaleSettings = {
			.ScaleRampBasis = ParametricSettings.ScaleSettings.ScaleRampBasis == EPVDistributionBasis::Plant
			? FParametricSettings::EDistributionBasis::Plant
			: FParametricSettings::EDistributionBasis::Branch,
			.BaseScale = ParametricSettings.ScaleSettings.BaseScale,
			.RandomizeScaleMinimum = ParametricSettings.ScaleSettings.RandomizeScaleMinimum,
			.RandomizeScaleMaximum = ParametricSettings.ScaleSettings.RandomizeScaleMaximum,
			.ScaleRamp = &ParametricSettings.ScaleSettings.ScaleRamp,
			.BranchScaleImpact = ParametricSettings.ScaleSettings.BranchScaleImpact
		};

		FPVGrafter::DistributeGraftWithParametricSettings(
			OutCollection,
			SourceGraftCollections,
			SpacingSettings,
			PhyllotaxySettings,
			AngleSettings,
			ScaleSettings,
			VectorSettings,
			ConditionSettings,
			Settings->RandomSeed,
			Settings->bRecomputeLight);
	}

	OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
	InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);

	return true;
}

#undef LOCTEXT_NAMESPACE
