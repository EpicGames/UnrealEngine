// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DataTypes/PVDistributionParams.h"
#include "Nodes/PVBaseSettings.h"

#include "PVGraftDistributorSettings.generated.h"

namespace PVGraftDistributorInputPins
{
	static const FName Skeleton("Skeleton");
	static const FName Graft("Graft");
}

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGraftDistributorSettings : public UPVBaseSettings
{
	GENERATED_BODY()

	friend class FPVGraftDistributorElement;

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GraftDistributor")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{EPVRenderType::PointData}; }
#endif

protected:

#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetSkeletonPinTypeIdentifier() const;
	virtual FPCGDataTypeIdentifier GetGraftPinTypeIdentifier() const;
	virtual FPCGElementPtr CreateElement() const override;

private:
	UPROPERTY(EditAnywhere, Category="Distribution Settings Mode", meta=(PCG_Overridable, Tooltip="Choose parametric or hormone-based distribution.\n\nParametric: parametric distribution leveraging ramps and gradients to control distribution. HormoneBased: uses the Grower's growth data for natural-looking placement."))
	EPVDistributionSettingsMode Mode = EPVDistributionSettingsMode::ParametricSettings;

	UPROPERTY(EditAnywhere, Category="Parametric Settings",
		meta=(PCG_NotOverridable, EditCondition="Mode == EPVDistributionSettingsMode::ParametricSettings", EditConditionHides, Tooltip="Parametric distribution rules (density, spacing, axil angle)."))
	FPVDistributionParametricParams ParametricSettings;

	UPROPERTY(EditAnywhere, Category="Hormone Based Settings",
		meta=(PCG_NotOverridable, EditCondition="Mode == EPVDistributionSettingsMode::HormoneBasedSettings", EditConditionHides, Tooltip="Hormone-driven distribution rules (ethylene threshold, scale, angle)."))
	FPVDistributionHormoneBasedParams HormoneBasedSettings;

	UPROPERTY(EditAnywhere, Category="Vector Settings",
		meta=(PCG_NotOverridable, ShowOnlyInnerProperties, Tooltip="Orientation rules: aim, face, roll/pitch/yaw."))
	FPVDistributionVectorParams VectorSettings;

	UPROPERTY(EditAnywhere, Category="ConditionSettings",
		meta=(PCG_NotOverridable, ShowOnlyInnerProperties, Tooltip="Filtering conditions (light, scale, tip, health, etc.)."))
	FPVDistributionConditionParams ConditionSettings;

	UPROPERTY(EditAnywhere, Category = "Misc Settings", meta = (PCG_Overridable, Tooltip="Random seed for graft placement.\n\nIndependent of the Grower's RandomSeed. Same seed = identical graft layout."))
	int32 RandomSeed = 123456;

	UPROPERTY(EditAnywhere, Category = "Misc Settings", meta = (PCG_Overridable, Tooltip="Recompute per-bud light after grafts are placed.\n\nWhen enabled, grafts contribute to light occlusion calculations affecting downstream nodes. Useful if you have additional Foliage Distributors or Senescence-aware nodes after the Grafter. Off = grafts will retain their incoming light data and will not consider its context after distribution."))
	bool bRecomputeLight = true;
};

class FPVGraftDistributorElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
