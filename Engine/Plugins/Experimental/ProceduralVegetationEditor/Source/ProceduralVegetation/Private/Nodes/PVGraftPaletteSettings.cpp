// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGraftPaletteSettings.h"

#include "ProceduralVegetationModule.h"
#include "DataTypes/PVGrafterPaletteData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVGrowthData.h"
#include "Facades/PVMetaInfoFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Helpers/PVDistributionHelper.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVGraftPaletteSettings"

UPVGraftPaletteSettings::UPVGraftPaletteSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		GraftInfos = {{false, FPVDistributionConditions{}}};
	}
}

#if WITH_EDITOR
FLinearColor UPVGraftPaletteSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Growth;
}

FText UPVGraftPaletteSettings::GetCategoryOverride() const
{
	return PV::Categories::Growth;
}


FText UPVGraftPaletteSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Graft Palette");
}

FText UPVGraftPaletteSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Collects pre-grown plant skeletons (grafts) into a single palette for the Graft Distributor. "
		"Each entry on this node exposes an input pin where a grown skeleton can be wired in. "
		"The Graft Distributor picks one graft per attachment point on the main plant, using each entry's Attributes to match against the point's sampled conditions."
		"\n\nPress Ctrl + L to lock/unlock node output"
	);
}
#endif

TArray<FPCGPinProperties> UPVGraftPaletteSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	if (GraftInfos.Num() > 0)
	{
		for (int32 i = 0; i < GraftInfos.Num(); i++)
		{
			FName PinName = FName(*FString::Printf(TEXT("Graft %d"), i + 1));
			FPCGPinProperties& Pin = Properties.Emplace_GetRef(PinName, GetInputPinTypeIdentifier());
			if (GraftInfos[i].bUseAsMask)
			{
				Pin.bInvisiblePin = true;
			}
			Pin.bAllowMultipleData = false;
		}
	}

	return Properties;
}

FPCGDataTypeIdentifier UPVGraftPaletteSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrafterPalette::AsId()};
}

FPCGDataTypeIdentifier UPVGraftPaletteSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()};;
}

FPCGElementPtr UPVGraftPaletteSettings::CreateElement() const
{
	return MakeShared<FPVGraftPaletteElement>();
}

void UPVGraftPaletteSettings::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
 
	if (GrafterInfos.Num())
	{
		GraftInfos.Empty();
		for (int32 i = 0; i < GrafterInfos.Num(); i++)
		{
			GraftInfos.Emplace(false, GrafterInfos[i]);
		}
	}
	
	GrafterInfos.Empty();

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FPVGraftPaletteElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGraftPaletteElement::Execute);

	check(InContext);

	const UPVGraftPaletteSettings* Settings = InContext->GetInputSettings<UPVGraftPaletteSettings>();
	check(Settings);

	TArray<TObjectPtr<UPVGrowthData>> GrowthDataElements;

	const TArray<FPCGPinProperties> InputPins = Settings->AllInputPinProperties();
	const int32 NumOfInputPins = InputPins.Num();
	for (int32 PinIndex = 0; PinIndex < NumOfInputPins; PinIndex++)
	{
		const FName PinName = InputPins[PinIndex].Label;
		const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PinName);

		if (Settings->GraftInfos.IsValidIndex(PinIndex))
		{
			UPVGrowthData* NewGrowthDataElement = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			
			FManagedArrayCollection OutCopy;
			PV::Facades::FMetaInfoFacade Facade(OutCopy);

			const FPVGraftInfo& GrafterInfo = Settings->GraftInfos[PinIndex];
			
			const UPVGrowthData* InGrowthData = Inputs.IsValidIndex(0) ? Cast<UPVGrowthData>(Inputs[0].Data) : nullptr;
			if (!GrafterInfo.bUseAsMask && InGrowthData)
			{
				if (PV::Utilities::IsValidGrowthData(InGrowthData->GetCollection()))
				{
					InGrowthData->GetCollection().CopyTo(&OutCopy);
				}
				else
				{
					PCGLog::InputOutput::LogInvalidInputDataError(InContext);
					return true;
				}
			}
			else if (!GrafterInfo.bUseAsMask && !InGrowthData)
			{
				PCGLog::InputOutput::LogInvalidInputDataError(InContext);
				return true;
			}
			else if (GrafterInfo.bUseAsMask && InGrowthData)
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("MaskPinWithGrowthData", "A mask pin is connected to growth data. It will be ignored"), InContext);
			}
			
			Facade.SetGraftAttributes(GrafterInfo);
			
			NewGrowthDataElement->Initialize(MoveTemp(OutCopy));
			GrowthDataElements.Add(NewGrowthDataElement);
		}
	}

	UPVGrafterPaletteData* OutPalette = FPCGContext::NewObject_AnyThread<UPVGrafterPaletteData>(InContext);
	OutPalette->Initialize(MoveTemp(GrowthDataElements));

	InContext->OutputData.TaggedData.Emplace(OutPalette);

	return true;
}

#undef LOCTEXT_NAMESPACE
