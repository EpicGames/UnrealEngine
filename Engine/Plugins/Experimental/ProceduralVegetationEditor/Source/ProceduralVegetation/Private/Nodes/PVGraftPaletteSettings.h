// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
 
#include "CoreMinimal.h"

#include "DataTypes/PVData.h"
#include "DataTypes/PVGraftInfo.h"

#include "Nodes/PVBaseSettings.h"

#include "PVGraftPaletteSettings.generated.h"


UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGraftPaletteSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVGraftPaletteSettings();

	#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("Graft Palette")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::GrafterGrid }; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::GrafterGrid }; }
#endif
 
protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;
 
public:
	virtual void PostLoad() override;
	
	UE_DEPRECATED(5.8, "Use GraftInfos instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use GraftInfos instead"))
	TArray<FPVDistributionConditions> GrafterInfos;
	
	UPROPERTY(EditAnywhere, Category = "Graft Skeletons", Meta = (ShowOnlyInnerProperties, FullyExpand="true", Tooltip="List of graft slots — one input pin per entry.\n\nEach entry creates an input pin on this node where a pre-grown plant skeleton (graft) can be connected. The entry's Attributes set the placement conditions the Graft Distributor uses to decide which graft goes onto which attachment point."))
	TArray<FPVGraftInfo> GraftInfos;
};
 
class FPVGraftPaletteElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool ShouldComputeFullOutputDataCrc(FPCGContext* Context) const override { return true; }
 
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
};