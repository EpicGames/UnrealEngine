// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGravitySettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVGravity.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVGravitySettings"

#if WITH_EDITOR
FLinearColor UPVGravitySettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVGravitySettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVGravitySettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Gravity"); 
}

FText UPVGravitySettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"This node controls how generated vegetation responds to directional forces, mimicking natural phenomena such as the downward pull of gravity and the growth tendency toward the direction of the optimal light conditions.(phototropism)"
		"\n\n"
		"Adds bending or uprighting effects after growth has completed. Distinct from the Grower's built-in gravity (which runs during simulation). Useful for tweaking the result without re-running the simulation. Mode selects gravity (downward pull) or phototropic (upward toward light)."
	);
}
#endif

FPCGDataTypeIdentifier UPVGravitySettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVGravitySettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVGravitySettings::CreateElement() const
{
	return MakeShared<FPVGravityElement>();
}

bool FPVGravityElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGravityElement::Execute);

	check(InContext);

	const UPVGravitySettings* Settings = InContext->GetInputSettings<UPVGravitySettings>();
	check(Settings);
	
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if(const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();
		
			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

#if WITH_EDITORONLY_DATA
			PVE_LOOP_DEBUG_INIT(Settings, LoopDebugStepper, OutManagedArrayCollectionData);
#endif
			
			if (Settings->GravitySettings.Gravity != 0.0f)
			{
				FPVGravity::ApplyGravity(Settings->GravitySettings, OutCollection);
			}

			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);

			PVE_LOOP_DEBUG_END();
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
