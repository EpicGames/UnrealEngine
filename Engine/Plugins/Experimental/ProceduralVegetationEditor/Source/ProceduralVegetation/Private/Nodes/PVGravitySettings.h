// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Implementations/PVGravity.h"
#include "PVGravitySettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGravitySettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationGravity")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category="Gravity", meta=(ShowOnlyInnerProperties, PCG_Overridable))
	FPVGravityParams GravitySettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Loop Debug", meta=(EditCondition = "bDebug", EditConditionHides))
	FLoopDebugStepper LoopDebugStepper;
#endif
};

class FPVGravityElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
