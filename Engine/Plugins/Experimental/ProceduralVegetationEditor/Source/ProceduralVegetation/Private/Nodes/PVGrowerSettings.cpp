// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerSettings.h"

#include "ProceduralVegetationModule.h"
#include "PVCommon.h"
#include "GrowerSettings/PVGrowerPhyllotaxySettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "DataAssets/ProceduralVegetationGrowerPreset.h"
#include "DataTypes/PVGrowerSettingsData.h"
#include "DataTypes/PVGrowerParamsData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Implementations/PVSeedGenerator.h"
#include "Implementations/PVLightDetection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Engine/StaticMesh.h"
#include "Helpers/PCGHelpers.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGEdge.h"
#include "GrowerSettings/PVGrowerPhototropismSettings.h"
#include "GrowerSettings/PVGrowerAuxinSettings.h"
#include "GrowerSettings/PVGrowerDirectionalSettings.h"
#include "GrowerSettings/PVGrowerBifurcationSettings.h"

#define LOCTEXT_NAMESPACE "PVGrowerSettings"

void UPVGrowerSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		GrowerParams.FoliageMesh = LoadObject<UStaticMesh>(nullptr,
			TEXT("/ProceduralVegetationEditor/DefaultAssets/Visualization/SM_SM_LeafScaled.SM_SM_LeafScaled"));
		if (GrowerParams.FoliageMesh)
		{
			FPVLightDetection::BuildLeafMeshGeometry(GrowerParams.FoliageMesh, GrowerParams.CachedLeafMeshGeometry);
		}
	}
}

void UPVGrowerSettings::PostLoad()
{
	Super::PostLoad();

	if (GrowerParams.Auxin.MinGravitationalDot != 0.0f)
	{
		GrowerParams.GravityParams.MinGravitationalDot = GrowerParams.Auxin.MinGravitationalDot;
		GrowerParams.Auxin.MinGravitationalDot = 0.0f;
	}

	if (GrowerParams.FoliageMesh)
	{
		FPVLightDetection::BuildLeafMeshGeometry(GrowerParams.FoliageMesh, GrowerParams.CachedLeafMeshGeometry);
	}
}

#if WITH_EDITOR
FLinearColor UPVGrowerSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Growth;
}

FText UPVGrowerSettings::GetCategoryOverride() const
{
	return PV::Categories::Growth;
}


FText UPVGrowerSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Grower"); 
}

FText UPVGrowerSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Grow a plant from a seed point using hormone-driven simulation."
		"\n\n"
		"The core PVE node. Takes one or more seed points and runs a multi-cycle growth simulation that produces a branch skeleton. When no seed is provided a default is automatically produced internally. Configure every aspect of growth — phyllotaxy, gravity, phototropism, senescence, bifurcation, foliage — through this node's inline parameters or via Standalone Override nodes."
	);
}

TMap<UObject*, FTransform> UPVGrowerSettings::GetViewportObjects() const
{
	TMap<UObject*, FTransform> ViewportObjects;
	for (auto Item : GrowerParams.ColliderSettings)
	{
		ViewportObjects.Add(Item.Mesh.Get(), Item.Transform);
	}

	return ViewportObjects;
}

void UPVGrowerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPVGrowerSettings, Preset))
	{
		if (Preset)
		{
			GrowerParams = Preset->GrowthParams;
		}
	}

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UPVGrowerSettings, GrowerParams)
		&& PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, FoliageMesh))
	{
		GrowerParams.CachedLeafMeshGeometry.Reset();
		if (GrowerParams.FoliageMesh)
		{
			FPVLightDetection::BuildLeafMeshGeometry(GrowerParams.FoliageMesh, GrowerParams.CachedLeafMeshGeometry);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

#if WITH_EDITOR
bool UPVGrowerSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	if (!Node)
	{
		return true;
	}

	// Helper: returns true (editable) when the named pin is NOT connected at all.
	auto Disconnected = [Node](const FName& PinLabel) -> bool
	{
		return !Node->IsInputPinConnected(PinLabel);
	};

	// Helper: aggregates target info across all nodes connected to a given input pin.
	// bTrunk / bBranch are OR-ed together so that if any connected node targets
	// a side, the corresponding params are disabled.
	// Returns a zeroed struct (both false) when no node is connected.
	auto GetCombinedTargets = [Node](const FName& PinLabel) -> FPVGrowerSettingsTargetInfo
	{
		FPVGrowerSettingsTargetInfo Combined;
		Combined.bTrunk  = false;
		Combined.bBranch = false;

		const UPCGPin* InputPin = Node->GetInputPin(PinLabel);
		if (!InputPin)
			return Combined;

		for (const UPCGEdge* Edge : InputPin->Edges)
		{
			if (!Edge || !Edge->InputPin || !Edge->InputPin->Node)
				continue;

			const UPCGSettings* S = Edge->InputPin->Node->GetSettings();
			if (!S)
				continue;

			const FPVGrowerSettingsTargetInfo* Targets = nullptr;
			if (const UPVGrowerPhyllotaxySettings* PhyllotaxyS = Cast<UPVGrowerPhyllotaxySettings>(S))
				Targets = &PhyllotaxyS->ParamsWithTargets.Targets;
			else if (const UPVGrowerPhototropismSettings* PhototropismS = Cast<UPVGrowerPhototropismSettings>(S))
				Targets = &PhototropismS->ParamsWithTargets.Targets;
			else if (const UPVGrowerAuxinSettings* AuxinS = Cast<UPVGrowerAuxinSettings>(S))
				Targets = &AuxinS->ParamsWithTargets.Targets;
			else if (const UPVGrowerDirectionalSettings* DirectionalS = Cast<UPVGrowerDirectionalSettings>(S))
				Targets = &DirectionalS->ParamsWithTargets.Targets;
			else if (const UPVGrowerBifurcationSettings* BifurcationS = Cast<UPVGrowerBifurcationSettings>(S))
				Targets = &BifurcationS->ParamsWithTargets.Targets;

			if (Targets)
			{
				Combined.bTrunk  |= Targets->bTrunk;
				Combined.bBranch |= Targets->bBranch;
			}
			else
			{
				// Unknown settings type: conservatively disable both sides.
				Combined.bTrunk = Combined.bBranch = true;
			}
		}

		return Combined;
	};

	// Helper: returns true if any Phyllotaxy Settings node on the given pin has bFoliage enabled.
	auto GetCombinedFoliageTarget = [Node](const FName& PinLabel) -> bool
	{
		const UPCGPin* InputPin = Node->GetInputPin(PinLabel);
		if (!InputPin) return false;
		for (const UPCGEdge* Edge : InputPin->Edges)
		{
			if (!Edge || !Edge->InputPin || !Edge->InputPin->Node) continue;
			if (const UPVGrowerPhyllotaxySettings* PhyllotaxyS = Cast<UPVGrowerPhyllotaxySettings>(
				Edge->InputPin->Node->GetSettings()))
			{
				if (PhyllotaxyS->ParamsWithTargets.Targets.bFoliage) return true;
			}
		}
		return false;
	};

	const UStruct* OwnerStruct = InProperty->GetOwnerStruct();

	// For target-based params (Phyllotaxy, Phototropism, Auxin, Directional, Bifurcation),
	// leaf properties are NOT checked by struct type here because the same struct is reused
	// for both trunk and branch instances (e.g. FPVPhototropismParams for both Phototropism
	// and BranchPhototropism). Instead, disabling the parent struct-handle property in the
	// FPVGrowerParams name-based section below causes FPropertyNode::IsEditConst to cascade
	// the read-only state to all child properties automatically.

	if (OwnerStruct == FPVGrowerParams::StaticStruct())
	{
		const FName PropName = InProperty->GetFName();

		// ── Phyllotaxy (target-aware) ───────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, Phyllotaxy))
		{
			return !GetCombinedTargets(PV::Pins::GrowerPhyllotaxyInputLabel).bTrunk;
		}
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, bBranchPhyllotaxySameAsTrunk)
			|| PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, BranchPhyllotaxy))
		{
			return !GetCombinedTargets(PV::Pins::GrowerPhyllotaxyInputLabel).bBranch;
		}
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, bLeafPhyllotaxySameAsBranch)
			|| PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, LeafPhyllotaxy))
		{
			return !GetCombinedFoliageTarget(PV::Pins::GrowerPhyllotaxyInputLabel);
		}

		// ── Trunk growth (no targets) ───────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, TrunkGrowth))
			return Disconnected(PV::Pins::GrowerGrowthInputLabel);

		// ── Phototropism (target-aware) ─────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, Phototropism))
		{
			return !GetCombinedTargets(PV::Pins::GrowerPhototropismInputLabel).bTrunk;
		}
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, bBranchPhototropismSameAsTrunk)
			|| PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, BranchPhototropism))
		{
			return !GetCombinedTargets(PV::Pins::GrowerPhototropismInputLabel).bBranch;
		}

		// ── Light senescence, Gravity, Age senescence (no targets) ──────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, LightSenescence))
			return Disconnected(PV::Pins::GrowerLightSenescenceInputLabel);

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, GravityParams))
			return Disconnected(PV::Pins::GrowerGravityInputLabel);

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, AgeSenescence))
			return Disconnected(PV::Pins::GrowerAgeSenescenceInputLabel);

		// ── Bifurcation (target-aware) ──────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, Bifurcation))
		{
			return !GetCombinedTargets(PV::Pins::GrowerBifurcationInputLabel).bTrunk;
		}
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, BranchBifurcation))
		{
			return !GetCombinedTargets(PV::Pins::GrowerBifurcationInputLabel).bBranch;
		}

		// ── Directional (target-aware) ──────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, Directional))
		{
			return !GetCombinedTargets(PV::Pins::GrowerDirectionalInputLabel).bTrunk;
		}
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, bBranchDirectionalSameAsTrunk)
			|| PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, BranchDirectional))
		{
			return !GetCombinedTargets(PV::Pins::GrowerDirectionalInputLabel).bBranch;
		}

		// ── Auxin (target-aware) ────────────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, Auxin))
		{
			return !GetCombinedTargets(PV::Pins::GrowerAuxinInputLabel).bTrunk;
		}
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, bBranchAuxinConditionSameAsTrunk)
			|| PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, BranchAuxin))
		{
			return !GetCombinedTargets(PV::Pins::GrowerAuxinInputLabel).bBranch;
		}

		// ── Foliage (no targets) ────────────────────────────────────────────────
		if (PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, Foliage)
			|| PropName == GET_MEMBER_NAME_CHECKED(FPVGrowerParams, FoliageMesh))
			return Disconnected(PV::Pins::GrowerFoliageInputLabel);
	}

	return true;
}
#endif

TArray<FPCGPinProperties> UPVGrowerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& Pin = Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, GetInputPinTypeIdentifier());
	Pin.SetAllowMultipleConnections(false);
	Pin.SetNormalPin();
	Pin.bAllowMultipleData = false;

	FPCGPinProperties& PhyllotaxyPin = Properties.Emplace_GetRef(PV::Pins::GrowerPhyllotaxyInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerPhyllotaxy::AsId()});
	PhyllotaxyPin.SetAllowMultipleConnections(true);
	PhyllotaxyPin.SetAdvancedPin();

	FPCGPinProperties& GrowthPin = Properties.Emplace_GetRef(PV::Pins::GrowerGrowthInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerTrunkGrowth::AsId()});
	GrowthPin.SetAllowMultipleConnections(false);
	GrowthPin.SetAdvancedPin();
	GrowthPin.bAllowMultipleData = false;

	FPCGPinProperties& PhototropismPin = Properties.Emplace_GetRef(PV::Pins::GrowerPhototropismInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerPhototropism::AsId()});
	PhototropismPin.SetAllowMultipleConnections(true);
	PhototropismPin.SetAdvancedPin();

	FPCGPinProperties& LightSenescencePin = Properties.Emplace_GetRef(PV::Pins::GrowerLightSenescenceInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerLightSenescence::AsId()});
	LightSenescencePin.SetAllowMultipleConnections(false);
	LightSenescencePin.SetAdvancedPin();
	LightSenescencePin.bAllowMultipleData = false;

	FPCGPinProperties& GravityPin = Properties.Emplace_GetRef(PV::Pins::GrowerGravityInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerGravity::AsId()});
	GravityPin.SetAllowMultipleConnections(false);
	GravityPin.SetAdvancedPin();
	GravityPin.bAllowMultipleData = false;

	FPCGPinProperties& AgeSenescencePin = Properties.Emplace_GetRef(PV::Pins::GrowerAgeSenescenceInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerAgeSenescence::AsId()});
	AgeSenescencePin.SetAllowMultipleConnections(false);
	AgeSenescencePin.SetAdvancedPin();
	AgeSenescencePin.bAllowMultipleData = false;

	FPCGPinProperties& BifurcationPin = Properties.Emplace_GetRef(PV::Pins::GrowerBifurcationInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerBifurcation::AsId()});
    BifurcationPin.SetAllowMultipleConnections(true);
    BifurcationPin.SetAdvancedPin();
    BifurcationPin.bAllowMultipleData = false;

	FPCGPinProperties& DirectionalPin = Properties.Emplace_GetRef(PV::Pins::GrowerDirectionalInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerDirectional::AsId()});
	DirectionalPin.SetAllowMultipleConnections(true);
	DirectionalPin.SetAdvancedPin();

	FPCGPinProperties& FoliageGrowthPin = Properties.Emplace_GetRef(PV::Pins::GrowerFoliageInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerFoliage::AsId()});
	FoliageGrowthPin.SetAllowMultipleConnections(false);
	FoliageGrowthPin.SetAdvancedPin();
	FoliageGrowthPin.bAllowMultipleData = false;

	FPCGPinProperties& AuxinPin = Properties.Emplace_GetRef(PV::Pins::GrowerAuxinInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerAuxin::AsId()});
	AuxinPin.SetAllowMultipleConnections(true);
	AuxinPin.SetAdvancedPin();

	return Properties;
}

TArray<FPCGPinProperties> UPVGrowerSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties = Super::OutputPinProperties();

	FPCGPinProperties& ParamsPin = Properties.Emplace_GetRef(PV::Pins::GrowerParamsOutputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerParams::AsId() });
	ParamsPin.SetAdvancedPin();
	ParamsPin.SetAllowMultipleConnections(true);
	ParamsPin.bAllowMultipleData = false;

	return Properties;
}

FPCGDataTypeIdentifier UPVGrowerSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}


FPCGDataTypeIdentifier UPVGrowerSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVGrowerSettings::CreateElement() const
{
	return MakeShared<FPVGrowerElement>();
}

bool FPVGrowerElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowerElement::Execute);

	check(InContext);

	if (const UPVGrowerSettings* Settings = InContext->GetInputSettings<UPVGrowerSettings>())
	{
		FPVGrowerParams GrowerParams = Settings->GrowerParams;
		
		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerPhyllotaxyInputLabel))
		{
			if (const UPVGrowerPhyllotaxyData* PhyllotaxyData = Cast<UPVGrowerPhyllotaxyData>(TaggedData.Data))
			{
				if (PhyllotaxyData->ParamsWithTargets.Targets.bTrunk)
				{
					GrowerParams.Phyllotaxy = PhyllotaxyData->ParamsWithTargets.Params;
				}

				if (PhyllotaxyData->ParamsWithTargets.Targets.bBranch)
				{
					GrowerParams.bBranchPhyllotaxySameAsTrunk = false;
					GrowerParams.BranchPhyllotaxy = PhyllotaxyData->ParamsWithTargets.Params;
				}

				if (PhyllotaxyData->ParamsWithTargets.Targets.bFoliage)
				{
					GrowerParams.bLeafPhyllotaxySameAsBranch = false;
					GrowerParams.LeafPhyllotaxy = PhyllotaxyData->ParamsWithTargets.Params;
				}
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerAuxinInputLabel))
		{
			if (const UPVGrowerAuxinData* Data = Cast<UPVGrowerAuxinData>(TaggedData.Data))
			{
				if (Data->ParamsWithTargets.Targets.bTrunk)
				{
					GrowerParams.Auxin = Data->ParamsWithTargets.Params;
				}

				if (Data->ParamsWithTargets.Targets.bBranch)
				{
					GrowerParams.bBranchAuxinConditionSameAsTrunk = false;
					GrowerParams.BranchAuxin = Data->ParamsWithTargets.Params;
				}
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerGrowthInputLabel))
		{
			if (const UPVGrowerTrunkGrowthData* Data = Cast<UPVGrowerTrunkGrowthData>(TaggedData.Data))
			{
				GrowerParams.TrunkGrowth = Data->Params;
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerLightSenescenceInputLabel))
		{
			if (const UPVGrowerLightSenescenceData* Data = Cast<UPVGrowerLightSenescenceData>(TaggedData.Data))
			{
				GrowerParams.LightSenescence = Data->Params;
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerAgeSenescenceInputLabel))
		{
			if (const UPVGrowerAgeSenescenceData* Data = Cast<UPVGrowerAgeSenescenceData>(TaggedData.Data))
			{
				GrowerParams.AgeSenescence = Data->Params;
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerDirectionalInputLabel))
		{
			if (const UPVGrowerDirectionalData* Data = Cast<UPVGrowerDirectionalData>(TaggedData.Data))
			{
				if (Data->ParamsWithTargets.Targets.bTrunk)
				{
					GrowerParams.Directional = Data->ParamsWithTargets.Params;
				}

				if (Data->ParamsWithTargets.Targets.bBranch)
				{
					GrowerParams.bBranchDirectionalSameAsTrunk = false;
					GrowerParams.BranchDirectional = Data->ParamsWithTargets.Params;
				}
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerGravityInputLabel))
		{
			if (const UPVGrowerGravityData* Data = Cast<UPVGrowerGravityData>(TaggedData.Data))
			{
				GrowerParams.GravityParams = Data->Params;
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerBifurcationInputLabel))
		{
			if (const UPVGrowerBifurcationData* Data = Cast<UPVGrowerBifurcationData>(TaggedData.Data))
			{
				if (Data->ParamsWithTargets.Targets.bTrunk)
				{
					GrowerParams.Bifurcation = Data->ParamsWithTargets.Params;	
				}
				
				if (Data->ParamsWithTargets.Targets.bBranch)
				{
					GrowerParams.BranchBifurcation = Data->ParamsWithTargets.Params;
				}
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerPhototropismInputLabel))
		{
			if (const UPVGrowerPhototropismData* Data = Cast<UPVGrowerPhototropismData>(TaggedData.Data))
			{
				if (Data->ParamsWithTargets.Targets.bTrunk)
				{
					GrowerParams.Phototropism = Data->ParamsWithTargets.Params;	
				}
				
				if (Data->ParamsWithTargets.Targets.bBranch)
				{
					GrowerParams.bBranchPhototropismSameAsTrunk = false;
					GrowerParams.BranchPhototropism = Data->ParamsWithTargets.Params;
				}
			}
		}

		for (const FPCGTaggedData& TaggedData : InContext->InputData.GetInputsByPin(PV::Pins::GrowerFoliageInputLabel))
		{
			if (const UPVGrowerFoliageData* Data = Cast<UPVGrowerFoliageData>(TaggedData.Data))
			{
				GrowerParams.Foliage = Data->Params;
			}
		}

		// Compute the growth collection first so we can share it with the params output for visualization.
		FManagedArrayCollection OutCollection;
		bool bHasCollection = false;

		const TArray<FPCGTaggedData>& Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
		
		if (Inputs.IsValidIndex(0))
		{
			if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Inputs[0].Data))
			{
				const FManagedArrayCollection& SkeletonCollection = InputData->GetCollection();

				if (!PV::Utilities::IsValidGrowthData(SkeletonCollection))
				{
					PCGLog::LogWarningOnGraph(LOCTEXT("InvalidSkeletonInput", "Invalid Input"), InContext);
					return true;
				}

				FPVGrower::Grow(SkeletonCollection, GrowerParams, OutCollection, Settings->bUseSplitPoints);
				bHasCollection = true;
			}
			else
			{
				UE_LOGF(LogProceduralVegetation, Log, "Invalid Input Data for FPVGrowerElement");
			}
		}
		else
		{
			FPVSeedPoint SeedPoint;
			FPVGrower::Grow(SeedPoint, GrowerParams, OutCollection, Settings->bUseSplitPoints);
			bHasCollection = true;
		}

		// Emit growth data on the default pin (owns the collection via move).
		UPVGrowthData* OutGrowthData = nullptr;
		
		if (bHasCollection)
		{
			OutGrowthData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			OutGrowthData->Initialize(MoveTemp(OutCollection));

			FPCGTaggedData& GrowthOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
			GrowthOutput.Data = OutGrowthData;
			GrowthOutput.Pin = PCGPinConstants::DefaultOutputLabel;
		}

		// Emit params data on the params pin; share the growth collection so the viewport can visualize it.
		UPVGrowerParamsData* OutParamsData = FPCGContext::NewObject_AnyThread<UPVGrowerParamsData>(InContext);
		OutParamsData->Params = GrowerParams;
		
		if (OutGrowthData)
		{
			OutParamsData->GetSharedCollection() = OutGrowthData->GetSharedCollection();
		}

		FPCGTaggedData& GrowthParamsOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
		GrowthParamsOutput.Data = OutParamsData;
		GrowthParamsOutput.Pin = PV::Pins::GrowerParamsOutputLabel;
	}
	else
	{
		PCGLog::InputOutput::LogInvalidInputDataError(InContext);
		return true;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
