// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <DataTypes/PVGrowerParams.h>

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Implementations/PVGrower.h"
#include "PVGrowerSettings.generated.h"

class UProceduralVegetationGrowerPreset;

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowerSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationGrower")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::PointData, EPVRenderType::Leaf }; }
	virtual TMap<UObject*, FTransform> GetViewportObjects() const override;
#endif

	virtual bool CanCullTaskIfUnwired() const override { return false; }
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

public:

	UPROPERTY(EditAnywhere, Category = Preset, meta=(Tooltip="Optional grower preset to populate the Settings below.\n\nAssigning a preset copies its grower parameters into Settings. Use this to start from a known-good configuration or share configurations across assets. Edits to Settings after applying are not written back to the source preset."))
	TObjectPtr<UProceduralVegetationGrowerPreset> Preset;

	UPROPERTY()
	bool bUseSplitPoints = false;

	UPROPERTY(EditAnywhere, Category = Settings, meta=( ShowOnlyInnerProperties, PCG_Overridable))
	FPVGrowerParams GrowerParams;
};

class FPVGrowerElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;

	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
};
