// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowthDataLoader.h"
#include "PCGContext.h"
#include "ProceduralVegetationModule.h"
#include "DataAssets/ProceduralVegetationGrowthDataAsset.h"
#include "DataTypes/PVGrowthData.h"
#include "Facades/PVPointFacade.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVGrowthDataLoaderSettings"

#if WITH_EDITOR
FLinearColor UPVGrowthDataLoaderSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::InputOutput;
}

FText UPVGrowthDataLoaderSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}


FText UPVGrowthDataLoaderSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Growth Data Loader"); 
}

FText UPVGrowthDataLoaderSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Load a saved Growth Data asset and emit each named variation as a separate output pin."
		"\n\n"
		"Loads a Procedural Vegetation Growth Data asset, which holds one or more named plant variations — each a complete grown skeleton with attributes. One output pin is created per valid variation in the asset."
	);
}

void UPVGrowthDataLoaderSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPVGrowthDataLoaderSettings, GrowthAsset))
	{
		FillGrowthVariationsInfo();
	}
}
#endif

void UPVGrowthDataLoaderSettings::PostLoad()
{
	Super::PostLoad();
	
	if (GrowthVariations.IsEmpty())
	{
		FillGrowthVariationsInfo();
	}
}

TArray<FPCGPinProperties> UPVGrowthDataLoaderSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPVGrowthDataLoaderSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	if (GrowthAsset)
	{
		for (const auto& [VariantName, VariantData] : GrowthAsset->Variants)
		{
			if (PV::Utilities::IsValidGrowthData(VariantData))
			{
				FPCGPinProperties& Pin = Properties.Emplace_GetRef(*VariantName, GetOutputPinTypeIdentifier());
				Pin.bAllowMultipleData = false;
			}
		}
	}

	return Properties;
}

FPCGDataTypeIdentifier UPVGrowthDataLoaderSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVGrowthDataLoaderSettings::CreateElement() const
{
	return MakeShared<FPVGrowthDataLoaderElement>();
}

void UPVGrowthDataLoaderSettings::FillGrowthVariationsInfo()
{
	GrowthVariations.Empty();

	if (!GrowthAsset)
	{
		UE_LOGF(LogProceduralVegetation, Log, "Growth asset is null, no preset info can be filled.");
		return;
	}

	if (GrowthAsset->GrowthVariations.IsEmpty())
	{
		// GrowthAsset->Fill();
	}

	GrowthVariations = GrowthAsset->GrowthVariations;
}

bool FPVGrowthDataLoaderElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVGrowthDataLoaderElement::ExecuteInternal);

	check(InContext);

	const UPVGrowthDataLoaderSettings* Settings = InContext->GetInputSettings<UPVGrowthDataLoaderSettings>();
	check(Settings);

	if (!Settings->GrowthAsset)
	{
		PCGLog::LogWarningOnGraph(NSLOCTEXT("PVGrowthDataLoaderSettings", "MissingDataAsset", "Growth asset is not set"), InContext);
		return true;
	}

	bool bValidDataExist = false;
	for (const auto& [VariantName, VariantData] : Settings->GrowthAsset->Variants)
	{
		if (PV::Utilities::IsValidGrowthData(VariantData))
		{
			UPVGrowthData* OutVariantData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			OutVariantData->Initialize(CopyTemp(VariantData));

			FPCGTaggedData& CollectionOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
			CollectionOutput.Data = OutVariantData;
			CollectionOutput.Pin = *VariantName;
			bValidDataExist = true;
		}
		else
		{
			PCGLog::LogWarningOnGraph(FText::FromString(FString::Format(TEXT("Skipping variation {0} due to invalid data."), {*VariantName})), InContext);
		}
	}

	if (!bValidDataExist)
	{
		PCGLog::LogErrorOnGraph(FText::FromString(FString::Format(TEXT("No valid variation found in data asset {0}"), { Settings->GrowthAsset.GetFName().ToString()})), InContext);
		return true;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
