// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "PVGrowthDataLoader.generated.h"

struct FPVGrowthVariationInfo;

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowthDataLoaderSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	virtual void PostLoad() override;
	
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GrowthDataLoader")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Tooltip="The Growth Data asset to load.\n\nA Procedural Vegetation Growth Data asset stores one or more named plant variations — each a complete grown skeleton with hormones, branches, foliage attachment points, and material references."))
	TObjectPtr<class UProceduralVegetationGrowthDataAsset> GrowthAsset = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta=(Tooltip="List of variations available within the loaded asset.\n\nAuto-populated when the Growth Asset is set. Each entry corresponds to one output pin emitted by this node."))
	TArray<FPVGrowthVariationInfo> GrowthVariations;

private:
	void FillGrowthVariationsInfo();
};

class FPVGrowthDataLoaderElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
