// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVImporterSettings.h"
#include "PCGContext.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVImporterSettings"

UPVImporterSettings::UPVImporterSettings()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bExposeToLibrary = false;
	}
#endif
}

#if WITH_EDITOR
FText UPVImporterSettings::GetCategoryOverride() const
{
	return PV::Categories::Development;
}

FText UPVImporterSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "[DEPRECATED] Procedural Vegetation Importer");
}

FText UPVImporterSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "DEPRECATED: This node no longer produces output. Use the Grower or Extract from Mesh/Image nodes to create new growth data.");
}
#endif

TArray<FPCGPinProperties> UPVImporterSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPVImporterSettings::OutputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGElementPtr UPVImporterSettings::CreateElement() const
{
	return MakeShared<FPVImporterElement>();
}

bool FPVImporterElement::ExecuteInternal(FPCGContext* InContext) const
{
	PCGLog::LogErrorOnGraph(FText::FromString(TEXT("This node doesn't produce any output. Use the Grower or Extract from Mesh/Image nodes to create new growth data.")));
	
	return true;
}

#undef LOCTEXT_NAMESPACE
