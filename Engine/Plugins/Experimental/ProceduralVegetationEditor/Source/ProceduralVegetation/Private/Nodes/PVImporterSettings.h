// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Nodes/PVBaseSettings.h"

#include "PVImporterSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(DeprecatedNode, DeprecationMessage="PVImporterSettings is deprecated and produces no output. Remove this node from your graph."))
class UPVImporterSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVImporterSettings();

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationImporter")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(VisibleAnywhere, Category="Skeleton")
	FFilePath SkeletonFile;
};

class FPVImporterElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
