// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVManualEditSettings.h"

#include "DataTypes/PVGrowthData.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVBudVectorsFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"

#include "Implementations/PVCarve.h"

#include "Helpers/PVAttributesHelper.h"

#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVManualEditSettings"

#if WITH_EDITOR

FName UPVManualEditSettings::GetDefaultNodeName() const
{
	return FName(TEXT("Manual Edit"));
}

FText UPVManualEditSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Hand-edit individual skeleton points and branches in the viewport."
		"\n\n"
		"Provides interactive editing of a grown plant: translate/rotate selected points, or remove branches by clicking them. Useful for fine artistic adjustments that procedural rules can't easily express. Edits are stored as transforms on top of the original skeleton, so the source is non-destructively preserved."
	);
}

FLinearColor UPVManualEditSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVManualEditSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}

#endif

FPCGDataTypeIdentifier UPVManualEditSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()};
}

FPCGDataTypeIdentifier UPVManualEditSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()};
}

#if WITH_EDITOR
void UPVManualEditSettings::ClearTranformations()
{
	ManualEditSettings.RelativeOffsets.Init(FVector3f::ZeroVector, ManualEditSettings.RelativeOffsets.Num());
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::ExternalModification);
}

void UPVManualEditSettings::ClearRemovals()
{
	ManualEditSettings.RemovedPoints.Init(false, ManualEditSettings.RemovedPoints.Num());
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::ExternalModification);
}
#endif // WITH_EDITOR

FPCGElementPtr UPVManualEditSettings::CreateElement() const
{
	return MakeShared<FPVManualEditElement>();
}

bool FPVManualEditElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVManualEditElement::ExecuteInternal);

	check(InContext);

	const UPVManualEditSettings* InputSettings = InContext->GetInputSettings<UPVManualEditSettings>();
	check(InputSettings);

	const TArray<FPCGTaggedData>& Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (!Inputs.IsEmpty())
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Inputs[0].Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();

			PV::Facades::FPointFacade PointFacade(OutCollection);
			PV::Facades::FBranchFacade BranchFacade(OutCollection);
			PV::Facades::FBudVectorsFacade BudVectorsFacade(OutCollection);

			if (!PointFacade.IsValid() || !BranchFacade.IsValid() || !BudVectorsFacade.IsValid())
			{
				PCGLog::LogWarningOnGraph(LOCTEXT("InvalidSkeletonInput", "Invalid Input"), InContext);
				return true;
			}

			const auto GetParentPoint = [&](const int32 BranchIndex) -> int32
				{
					const int32 RootPointIndex = BranchFacade.GetRootPoint(BranchIndex);
					if (RootPointIndex != INDEX_NONE)
					{
						const float PointLFR = PointFacade.GetLengthFromRoot(RootPointIndex);
						const int32 ParentBranchIndex = BranchFacade.GetParentBranchIndex(BranchIndex);
						if (ParentBranchIndex != INDEX_NONE)
						{
							const TArray<int32>& ParentBranchPoints = BranchFacade.GetPoints(ParentBranchIndex);
							for (int32 BranchPointIndex = 0; BranchPointIndex < ParentBranchPoints.Num(); BranchPointIndex++)
							{
								if (PointFacade.GetLengthFromRoot(ParentBranchPoints[BranchPointIndex]) >= PointLFR)
								{
									return ParentBranchPoints[BranchPointIndex];
								}
							}
						}
					}
					return INDEX_NONE;
				};

			const TArray<FVector3f>& Offsets = InputSettings->ManualEditSettings.RelativeOffsets;

			const int32* MaxBudNumber = Algo::MaxElement(PointFacade.GetBudNumbersAttribute());
			if (MaxBudNumber && Offsets.Num() >= *MaxBudNumber + 1)
			{
				TManagedArray<FVector3f>& Positions = PointFacade.ModifyPositions();
				TManagedArray<TArray<FVector3f>>& BudDirections = BudVectorsFacade.ModifyBudDirections();

				const TArray<FVector3f> OriginalPositions = Positions.GetConstArray();

				for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++)
				{
					const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
					for (int32 BranchPointIndex = !BranchFacade.IsTrunk(BranchIndex); BranchPointIndex < BranchPoints.Num(); BranchPointIndex++)
					{
						const int32 PointIndex = BranchPoints[BranchPointIndex];
						const int32 PointBudNumber = PointFacade.GetBudNumber(PointIndex);
						Positions[PointIndex] += Offsets[PointBudNumber];

						const int32 PreviousPointIndex = BranchPointIndex > 0
							? BranchPoints[BranchPointIndex - 1]
							: GetParentPoint(BranchIndex);
						if (PreviousPointIndex != INDEX_NONE)
						{
							const FVector3f OldDirection = OriginalPositions[PointIndex] - OriginalPositions[PreviousPointIndex];
							const FVector3f NewDirection = Positions[PointIndex] - Positions[PreviousPointIndex];
							const FQuat4f Rotation = FQuat4f::FindBetweenVectors(OldDirection, NewDirection);

							TArray<FVector3f>& BudDirection = BudDirections[PointIndex];
							BudDirection[PV::Facades::BudDirectionsApical] = Rotation.RotateVector(BudDirection[PV::Facades::BudDirectionsApical]);
							BudDirection[PV::Facades::BudDirectionsAxillary] = Rotation.
								RotateVector(BudDirection[PV::Facades::BudDirectionsAxillary]);
							BudDirection[PV::Facades::BudDirectionsGuideCurve] = Rotation.RotateVector(
								BudDirection[PV::Facades::BudDirectionsGuideCurve]);
							BudDirection[PV::Facades::BudDirectionsUpVector] = Rotation.
								RotateVector(BudDirection[PV::Facades::BudDirectionsUpVector]);
						}
					}
				}

				PV::AttributesHelper::ComputeLengthFromRoot(OutCollection);
			}

			TArray<bool> RemovedPoints = InputSettings->ManualEditSettings.RemovedPoints;

			TArray<bool> PointsToRemove;
			TArray<bool> BranchesToRemove;
			if (MaxBudNumber && RemovedPoints.Num() >= *MaxBudNumber + 1)
			{
				PointsToRemove.Init(false, PointFacade.GetElementCount());
				BranchesToRemove.Init(false, BranchFacade.GetElementCount());

				for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++)
				{
					const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
					bool bAllPointsRemoved = true;
					int32 FirstPointRemovedIndex = INDEX_NONE;
					for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num(); BranchPointIndex++)
					{
						const int32 PointIndex = BranchPoints[BranchPointIndex];
						const int32 PointBudNumber = PointFacade.GetBudNumber(PointIndex);
						if (!PointsToRemove[PointIndex])
						{
							PointsToRemove[PointIndex] = RemovedPoints[PointBudNumber];
						}

						if (BranchPointIndex != 0)
						{
							bAllPointsRemoved &= PointsToRemove[PointIndex];
						}

						if (PointsToRemove[PointIndex])
						{
							if (FirstPointRemovedIndex == INDEX_NONE)
							{
								FirstPointRemovedIndex = BranchPointIndex;
							}
						}
					}
					BranchesToRemove[BranchIndex] = bAllPointsRemoved;
					if (bAllPointsRemoved)
					{
						const TManagedArray<int32>& BranchNumbers = BranchFacade.GetBranchNumbers();
						for (int32 BranchChildNumber : BranchFacade.GetChildren(BranchIndex))
						{
							if (const int32 ChildBranchIndex = BranchNumbers.Find(BranchChildNumber); ChildBranchIndex != INDEX_NONE)
							{
								for (int32 ChildPoint : BranchFacade.GetPoints(ChildBranchIndex))
								{
									PointsToRemove[ChildPoint] = true;
								}
								BranchesToRemove[ChildBranchIndex] = true;
							}
						}
					}

					if (FirstPointRemovedIndex != INDEX_NONE)
					{
						const float Ratio = static_cast<float>(FirstPointRemovedIndex) / static_cast<float>(BranchPoints.Num());
						// Rescale the branches based on the cut point
						for (int32 Index = !BranchFacade.IsTrunk(BranchIndex); Index <= FirstPointRemovedIndex; ++Index)
						{
							PointFacade.SetPointScale(BranchPoints[Index], PointFacade.GetPointScale(BranchPoints[Index]) * Ratio);
						}
						for (int32 ChildBranch : BranchFacade.GetChildren(BranchIndex))
						{
							if (const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch); ChildBranchIndex != INDEX_NONE)
							{
								const TArray<int32>& ChildBranchPoints = BranchFacade.GetPoints(ChildBranchIndex);
								for (int32 ChildBranchPointIndex = 1; ChildBranchPointIndex < ChildBranchPoints.Num(); ChildBranchPointIndex++)
								{
									const int32 ChildBranchPoint = ChildBranchPoints[ChildBranchPointIndex];
									PointFacade.SetPointScale(ChildBranchPoint, PointFacade.GetPointScale(ChildBranchPoint) * Ratio);
								}
							}
						}
					}
				}

				// Foliage Instances are not being used now with the new Grower
				TArray<bool> FoliageInstancesToRemove;
				PV::Facades::FTreeFacade::RemoveEntriesAndReIndexAttributes(
					OutCollection,
					PointsToRemove,
					BranchesToRemove,
					FoliageInstancesToRemove
				);
			}

			UPVData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
