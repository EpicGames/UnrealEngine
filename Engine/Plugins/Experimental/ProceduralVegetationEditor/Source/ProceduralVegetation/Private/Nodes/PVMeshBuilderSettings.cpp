// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVMeshBuilderSettings.h"
#include "ProceduralVegetationModule.h"
#include "PVCommon.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "DataTypes/PVMeshData.h"
#include "Data/PCGBasePointData.h"
#include "DataTypes/PVTrunkTextureSetupData.h"
#include "DataTypes/PVPlantProfileData.h"
#include "DataTypes/PVMeshBuilderSettingsData.h"
#include "PCGNode.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVProfileFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Helpers/PVAnalyticsHelper.h"
#include "Implementations/PVMeshBuilder.h"
#include "Implementations/PVTrunkTextureSetup.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"

#define LOCTEXT_NAMESPACE "PVMeshBuilderSettings"

#if WITH_EDITOR
FLinearColor UPVMeshBuilderSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Mesh;
}

FText UPVMeshBuilderSettings::GetCategoryOverride() const
{
	return PV::Categories::Mesh;
}


FText UPVMeshBuilderSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Mesh Builder"); 
}

FText UPVMeshBuilderSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Build a 3D mesh from the plant skeleton with materials, displacement, and detail control."
		"\n\n"
		"Converts the grown skeleton into a renderable mesh. Configure mesh resolution, branch radius shaping, profile detail, displacement, skeleton noise, and material assignments."
	);
}

void UPVMeshBuilderSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderDisplacementParams, Texture))
	{
		DisplacementWarnings.Empty();
		if (MesherSettings.Displacement.Texture)
		{
			FPVMeshBuilder::ExtractDisplacementData(MesherSettings.Displacement.Texture, MesherSettings.DisplacementValues, DisplacementWarnings);
		}
		else
		{
			MesherSettings.DisplacementValues.Empty();
		}

		Modify();
	}
}

void UPVMeshBuilderSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;

	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty && Property)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPVMeshBuilderSettings, MesherSettings))
		{
			FName Name = Property->GetFName();
			
			if(Name == "Material")
			{
				int Index = PropertyChangedEvent.GetArrayIndex("MaterialSetups");

				if (Index >= 0 && MesherSettings.MaterialDetails.MaterialSetups.Num() > Index)
				{
					if (UMaterialInterface* Mat = MesherSettings.MaterialDetails.MaterialSetups[Index].Material)
					{
						FString MatPath = Mat->GetPathName();
						PV::Analytics::SendMaterialChangeEvent(MatPath);	
					}
					
				}
			}
		}
	}
}

void UPVMeshBuilderSettings::PostLoad()
{
	Super::PostLoad();

	DisplacementWarnings.Empty();
	if (MesherSettings.Displacement.Texture)
	{
		FPVMeshBuilder::ExtractDisplacementData(MesherSettings.Displacement.Texture, MesherSettings.DisplacementValues, DisplacementWarnings);
	}
	else
	{
		MesherSettings.DisplacementValues.Empty();
	}
}

bool UPVMeshBuilderSettings::CanEditChange(const FProperty* InProperty) const
{
	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	const UPCGNode* Node = Cast<UPCGNode>(GetOuter());
	if (!Node)
	{
		return true;
	}

	auto Disconnected = [Node](const FName& PinLabel) -> bool
	{
		return !Node->IsInputPinConnected(PinLabel);
	};

	if (InProperty->GetOwnerStruct() == FPVMeshBuilderParams::StaticStruct())
	{
		const FName PropName = InProperty->GetFName();

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderParams, MeshDetails))
		{
			return Disconnected(PV::Pins::MeshBuilderMeshDetailsInputLabel);
		}

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderParams, ProfileDetails))
		{
			return Disconnected(PV::Pins::MeshBuilderProfileDetailsInputLabel);
		}

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderParams, BranchRadius))
		{
			return Disconnected(PV::Pins::MeshBuilderBranchRadiusInputLabel);
		}

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderParams, Displacement))
		{
			return Disconnected(PV::Pins::MeshBuilderDisplacementInputLabel);
		}

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderParams, SkeletonShaping))
		{
			return Disconnected(PV::Pins::MeshBuilderSkeletonShapingInputLabel);
		}

		if (PropName == GET_MEMBER_NAME_CHECKED(FPVMeshBuilderParams, MaterialDetails))
		{
			return Disconnected(PV::Pins::MeshBuilderMaterialDetailsInputLabel);
		}
	}

	return true;
}
#endif

FPCGDataTypeIdentifier UPVMeshBuilderSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVMeshBuilderSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoMesh::AsId() };
}

FPCGElementPtr UPVMeshBuilderSettings::CreateElement() const
{
	return MakeShared<FPVMeshBuilderElement>();
}

TArray<FPCGPinProperties> UPVMeshBuilderSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties = Super::InputPinProperties();

	FPCGPinProperties& ProfilePin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderProfileInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoPlantProfile::AsId()});
	ProfilePin.SetAllowMultipleConnections(false);
	ProfilePin.SetAdvancedPin();
	ProfilePin.bAllowMultipleData = false;

	FPCGPinProperties& PinTrunkTextureSetup = Properties.Emplace_GetRef(TEXT("Trunk Texture Setup"), FPCGDataTypeIdentifier{ FPVDataTypeInfoTrunkTextureSetup::AsId()});
	PinTrunkTextureSetup.SetAllowMultipleConnections(false);
	PinTrunkTextureSetup.SetOverrideOrUserParamPin();
	PinTrunkTextureSetup.bAllowMultipleData = false;

	FPCGPinProperties& SkeletonShapingSettingsPin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderSkeletonShapingInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderSkeletonShaping::AsId() });
	SkeletonShapingSettingsPin.SetAllowMultipleConnections(false);
	SkeletonShapingSettingsPin.SetAdvancedPin();
	SkeletonShapingSettingsPin.bAllowMultipleData = false;

	FPCGPinProperties& BranchRadiusSettingsPin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderBranchRadiusInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderBranchRadius::AsId() });
	BranchRadiusSettingsPin.SetAllowMultipleConnections(false);
	BranchRadiusSettingsPin.SetAdvancedPin();
	BranchRadiusSettingsPin.bAllowMultipleData = false;

	FPCGPinProperties& PlantProfileSettingsPin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderProfileDetailsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderProfileDetail::AsId() });
	PlantProfileSettingsPin.SetAllowMultipleConnections(false);
	PlantProfileSettingsPin.SetAdvancedPin();
	PlantProfileSettingsPin.bAllowMultipleData = false;

	FPCGPinProperties& MeshSettingsPin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderMeshDetailsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderMeshDetail::AsId() });
	MeshSettingsPin.SetAllowMultipleConnections(false);
	MeshSettingsPin.SetAdvancedPin();
	MeshSettingsPin.bAllowMultipleData = false;

	FPCGPinProperties& MaterialSettingsPin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderMaterialDetailsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderMaterialDetail::AsId() });
	MaterialSettingsPin.SetAllowMultipleConnections(false);
	MaterialSettingsPin.SetAdvancedPin();
	MaterialSettingsPin.bAllowMultipleData = false;

	FPCGPinProperties& DisplacementSettingsPin = Properties.Emplace_GetRef(PV::Pins::MeshBuilderDisplacementInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoMeshBuilderDisplacement::AsId() });
	DisplacementSettingsPin.SetAllowMultipleConnections(false);
	DisplacementSettingsPin.SetAdvancedPin();
	DisplacementSettingsPin.bAllowMultipleData = false;

	return Properties;
}

bool FPVMeshBuilderElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVMeshBuilderElement::Execute);

	check(InContext);

	const UPVMeshBuilderSettings* Settings = InContext->GetInputSettings<UPVMeshBuilderSettings>();
	check(Settings);

	FManagedArrayCollection PlantProfileCollection;

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderProfileInputLabel))
	{
		if (const UPVPlantProfileData* Profile = Cast<UPVPlantProfileData>(Input.Data))
		{
			PlantProfileCollection = Profile->GetCollection();
		}
	}

	const TArray<FPCGTaggedData>& Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	const TArray<FPCGTaggedData>& InputTrunkTextureSetups = InContext->InputData.GetInputsByPin(TEXT("Trunk Texture Setup"));

	if (!Inputs.IsEmpty())
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Inputs[0].Data))
		{
			const FManagedArrayCollection& SkeletonCollection = InputData->GetCollection();

			// Build a local copy of the mesh params so we can patch per-entry
			// bDeriveFromTrunkTextureSetup overrides without mutating the settings asset.
			FPVMeshBuilderParams ResolvedParams = Settings->MesherSettings;

			for (const FPCGTaggedData& TD : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderMeshDetailsInputLabel))
			{
				if (const UPVMeshBuilderMeshDetailData* Data = Cast<UPVMeshBuilderMeshDetailData>(TD.Data))
				{
					ResolvedParams.MeshDetails = Data->Params;
				}
			}

			for (const FPCGTaggedData& TD : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderProfileDetailsInputLabel))
			{
				if (const UPVMeshBuilderProfileDetailData* Data = Cast<UPVMeshBuilderProfileDetailData>(TD.Data))
				{
					ResolvedParams.ProfileDetails = Data->Params;
				}
			}

			for (const FPCGTaggedData& TD : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderBranchRadiusInputLabel))
			{
				if (const UPVMeshBuilderBranchRadiusData* Data = Cast<UPVMeshBuilderBranchRadiusData>(TD.Data))
				{
					ResolvedParams.BranchRadius = Data->Params;
				}
			}

			for (const FPCGTaggedData& TD : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderSkeletonShapingInputLabel))
			{
				if (const UPVMeshBuilderSkeletonShapingData* Data = Cast<UPVMeshBuilderSkeletonShapingData>(TD.Data))
				{
					ResolvedParams.SkeletonShaping = Data->Params;
				}
			}

			for (const FPCGTaggedData& TD : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderMaterialDetailsInputLabel))
			{
				if (const UPVMeshBuilderMaterialDetailData* Data = Cast<UPVMeshBuilderMaterialDetailData>(TD.Data))
				{
					ResolvedParams.MaterialDetails = Data->Params;
				}
			}

			// Displacement override: consume the pre-extracted values cached on the override-node
			// data object. The override node performs the extraction once in its
			// PostEditChangeProperty / PostLoad, so the Mesher just copies the cached array.
			for (const FPCGTaggedData& TD : InContext->InputData.GetInputsByPin(PV::Pins::MeshBuilderDisplacementInputLabel))
			{
				if (const UPVMeshBuilderDisplacementData* Data = Cast<UPVMeshBuilderDisplacementData>(TD.Data))
				{
					ResolvedParams.Displacement = Data->Params;
					ResolvedParams.DisplacementValues = Data->Values;
				}
			}

			const UPVTrunkTextureSetupData* TrunkTextureSetup = nullptr;
			if (InputTrunkTextureSetups.IsValidIndex(0))
			{
				TrunkTextureSetup = Cast<UPVTrunkTextureSetupData>(InputTrunkTextureSetups[0].Data);
			}

			// For each MaterialSetup entry marked bDeriveFromTrunkTextureSetup, resolve Material
			// and URange from the Trunk Texture Setup pin matched by generation index.
			// If no matching entry exists (pin not connected or index out of range), the
			// Material is set to null and URange resets to the full default (0, 1).
			const TArray<FPVGenerationUVRange>* GenUVs =
				TrunkTextureSetup ? &TrunkTextureSetup->TrunkTextureSetupInfo.GenerationUVs : nullptr;
			for (int32 i = 0; i < ResolvedParams.MaterialDetails.MaterialSetups.Num(); ++i)
			{
				FTrunkGenerationMaterialSetup& Entry = ResolvedParams.MaterialDetails.MaterialSetups[i];
				if (!Entry.bDeriveFromTrunkTextureSetup)
					continue;

				if (GenUVs && GenUVs->IsValidIndex(i))
				{
					// Only override Material if the TrunkTextureSetup provides a valid one.
					if (TrunkTextureSetup->TrunkTextureSetupInfo.Material)
					{
						Entry.Material = TrunkTextureSetup->TrunkTextureSetupInfo.Material;
					}
					Entry.URange = FFloatRange((*GenUVs)[i].OffsetXStart, (*GenUVs)[i].OffsetXEnd);
				}
				else
				{
					// No matching TrunkTextureSetup generation: set Material null,
					// reset X Range to the full default so nothing unexpected is applied.
					Entry.URange = FFloatRange(0.f, 1.f);
					Entry.Material = nullptr;
				}
			}

			if (!Settings->DisplacementWarnings.IsEmpty())
			{
				PCGLog::LogWarningOnGraph(FText::FromString(FString::Format(TEXT("{0} "), {*Settings->DisplacementWarnings})), InContext);
			}

			UPVMeshData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVMeshData>(InContext);

			PVE_PARAM_DEBUG_INIT(Settings, OutManagedArrayCollectionData);

			FGeometryCollection OutGeometryCollection;
			FPVMeshBuilder::GenerateGeometryCollection(SkeletonCollection, PlantProfileCollection, ResolvedParams, OutGeometryCollection);

			PVE_PARAM_DEBUG_END()

			FManagedArrayCollection ManagedArrayCollection;
			OutGeometryCollection.CopyTo(&ManagedArrayCollection);
			OutManagedArrayCollectionData->Initialize(MoveTemp(ManagedArrayCollection));

			FPCGTaggedData& CollectionOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
			CollectionOutput.Data = OutManagedArrayCollectionData;
			CollectionOutput.Pin = PCGPinConstants::DefaultOutputLabel;
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
