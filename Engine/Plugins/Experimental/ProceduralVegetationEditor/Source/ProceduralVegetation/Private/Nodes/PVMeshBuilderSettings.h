// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Implementations/PVMeshBuilder.h"
#include "Implementations/PVMaterialSettings.h"
#include "Nodes/PVBaseSettings.h"
#include "PVMeshBuilderSettings.generated.h"

class UMaterialInterface;

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVMeshBuilderSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationMesher")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::Mesh }; }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{ EPVRenderType::PointData, EPVRenderType::Mesh, EPVRenderType::FoliageAttachments }; }

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

public:	
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPVMeshBuilderParams MesherSettings;

	UPROPERTY()
	FString DisplacementWarnings;
};

class FPVMeshBuilderElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
