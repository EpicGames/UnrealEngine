// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVObjectInteractionSettings.h"

#include "DataTypes/PVGrowthData.h"
#include "Engine/StaticMesh.h"

#include "Helpers/PVUtilities.h"

#include "Implementations/PVObjectInteraction.h"
#include "PVCommon.h"

FPCGDataTypeIdentifier UPVObjectInteractionSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()};
}

FPCGDataTypeIdentifier UPVObjectInteractionSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()};
}

FPCGElementPtr UPVObjectInteractionSettings::CreateElement() const
{
	return MakeShared<FPVObjectInteractionElement>();
}

bool FPVObjectInteractionElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVObjectInteractionElement::ExecuteInternal);

	check(InContext);

	const UPVObjectInteractionSettings* InputSettings = InContext->GetInputSettings<UPVObjectInteractionSettings>();
	check(InputSettings);

	const TArray<FPCGTaggedData>& Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (!Inputs.IsEmpty())
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Inputs[0].Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();
			UPVData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

#if WITH_EDITORONLY_DATA
			PVE_LOOP_DEBUG_INIT(InputSettings, LoopDebugStepper, OutManagedArrayCollectionData);
#endif
			
			for (const auto& Collider : InputSettings->ObjectInteractionSettings.Colliders)
			{
				PVE_OUTER_LOOP_DEBUG_CHECK(break);
				
				FPVObjectInteraction::ObjectInteraction(OutCollection, Collider);
			}

			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);

			PVE_LOOP_DEBUG_END();
		}
	}

	return true;
}

#if WITH_EDITOR
FLinearColor UPVObjectInteractionSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVObjectInteractionSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}

#endif
