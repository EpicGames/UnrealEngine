// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVPlantProfileLoaderSettings.h"
#include "PCGContext.h"
#include "ProceduralVegetationModule.h"
#include "DataAssets/ProceduralVegetationPlantProfileDataAsset.h"
#include "DataTypes/PVPlantProfileData.h"
#include "Facades/PVProfileFacade.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVPlantProfileLoaderSettings"

#if WITH_EDITOR
FLinearColor UPVPlantProfileLoaderSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::InputOutput;
}

FText UPVPlantProfileLoaderSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}


FText UPVPlantProfileLoaderSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Plant Profile Loader"); 
}

FText UPVPlantProfileLoaderSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Load a saved Plant Profile data asset containing trunk shape."
		"\n\n"
		"Loads a Procedural Vegetation Plant Profile asset, which contains data defining the shape of a trunk."
	);
}

void UPVPlantProfileLoaderSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPVPlantProfileLoaderSettings, PlantProfileData))
	{
		if (PlantProfileData)
		{
			Profiles = PlantProfileData->Profiles;	
		}
		else
		{
			Profiles.Empty();
		}
	}
}
#endif

TArray<FPCGPinProperties> UPVPlantProfileLoaderSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPVPlantProfileLoaderSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	if (PlantProfileData)
	{
		for (const auto& [ProfileName, ProfilePoints] : PlantProfileData->Profiles)
		{
			if (!ProfileName.IsEmpty() && ProfilePoints.Num() > 0)
			{
				FPCGPinProperties& Pin = Properties.Emplace_GetRef(*ProfileName, GetOutputPinTypeIdentifier());
				Pin.bAllowMultipleData = false;
			}
		}
	}

	return Properties;
}

FPCGDataTypeIdentifier UPVPlantProfileLoaderSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoPlantProfile::AsId() };
}

FPCGElementPtr UPVPlantProfileLoaderSettings::CreateElement() const
{
	return MakeShared<FPVPlantProfileLoaderElement>();
}

bool FPVPlantProfileLoaderElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVPresetLoaderElement::ExecuteInternal);

	check(InContext);

	const UPVPlantProfileLoaderSettings* Settings = InContext->GetInputSettings<UPVPlantProfileLoaderSettings>();
	check(Settings);

	if (!Settings->PlantProfileData)
	{
		PCGLog::LogWarningOnGraph(NSLOCTEXT("PVPlantProfileLoaderSettings", "MissingDataAsset", "Preset data asset is not set"), InContext);
		return true;
	}

	bool bValidDataExist = false;
	for (const auto& [ProfileName, ProfilePoints] : Settings->PlantProfileData->Profiles)
	{
		if (!ProfileName.IsEmpty() && ProfilePoints.Num() > 0)
		{
			FManagedArrayCollection Collection;

			PV::Facades::FPlantProfileFacade Facade(Collection);

			Facade.AddProfileEntry(ProfilePoints);
			
			UPVPlantProfileData* OutVariantData = FPCGContext::NewObject_AnyThread<UPVPlantProfileData>(InContext);
			OutVariantData->Initialize(CopyTemp(Collection));
			
			FPCGTaggedData& CollectionOutput = InContext->OutputData.TaggedData.Emplace_GetRef();
			CollectionOutput.Data = OutVariantData;
			CollectionOutput.Pin = *ProfileName;
			bValidDataExist = true;
		}
		else
		{
			PCGLog::LogWarningOnGraph(FText::FromString(FString::Format(TEXT("Skipping variation {0} due to invalid data."), {*ProfileName})), InContext);
		}
	}

	if (!bValidDataExist)
	{
		PCGLog::LogErrorOnGraph(FText::FromString(FString::Format(TEXT("No valid variation found in data asset {0}"), { Settings->PlantProfileData.GetFName().ToString()})), InContext);
		return true;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
