// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "PVPlantProfileLoaderSettings.generated.h"

struct FPVPlantProfile;

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVPlantProfileLoaderSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationPlantProfileLoader")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PlantProfile }; }

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = Settings, meta=(Tooltip="The Plant Profile asset to load.\n\nA procedural vegetation profile data asset contains data for a shape of the trunk."))
	TObjectPtr<class UProceduralVegetationPlantProfileDataAsset> PlantProfileData = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta=(Tooltip="List of variations available within the loaded profile.\n\nWhen the loaded asset contains multiple variations, this array exposes them. Each entry typically defines one variant."))
	TArray<FPVPlantProfile> Profiles;
};

class FPVPlantProfileLoaderElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
