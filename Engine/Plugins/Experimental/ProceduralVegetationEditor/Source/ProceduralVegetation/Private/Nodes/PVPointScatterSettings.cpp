// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVPointScatterSettings.h"
#include "ProceduralVegetationPreset.h"
#include "Data/PCGBasePointData.h"
#include "Helpers/PCGHelpers.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/StaticMesh.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVPointScatterSettings"

UPVPointScatterSettings::UPVPointScatterSettings(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
#if WITH_EDITOR
		DebugSettings.PointScale = 0.5f;
		DebugSettings.ScaleMethod = EPCGDebugVisScaleMethod::Relative;

		static ConstructorHelpers::FObjectFinder<UStaticMesh> PointMeshFinder(TEXT("/ProceduralVegetationEditor/DefaultAssets/Visualization/SM_Cone.SM_Cone"));

		if (PointMeshFinder.Succeeded())
		{
			DebugSettings.PointMesh = PointMeshFinder.Object;
		}
#endif
		
		TWeakObjectPtr WeakThis(this);
		TSoftObjectPtr<UPCGGraphInterface> DefaultGraphSoftPtr = TSoftObjectPtr<UPCGGraphInterface>(FSoftObjectPath(TEXT("/ProceduralVegetationEditor/DefaultAssets/PCG/SeedPointScatter.SeedPointScatter")));
		DefaultGraphSoftPtr.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateLambda([WeakThis, this](const FSoftObjectPath&, UObject* InObject)
		{
			if (!WeakThis.IsValid())
			{
				return;
			}
				
			if (InObject)
			{
				WeakThis.Get()->SetSubgraph(Cast<UPCGGraphInterface>(InObject));
			}
		}));
	}
}

FPCGElementPtr UPVPointScatterSettings::CreateElement() const
{
	return MakeShared<FPVPointScatterElement>();
}

bool FPVPointScatterElement::ExecuteInternal(FPCGContext* InContext) const
{
	return FPCGSubgraphElement::ExecuteInternal(InContext);
}

#if WITH_EDITOR
FLinearColor UPVPointScatterSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Seed;
}
#endif

#undef LOCTEXT_NAMESPACE
