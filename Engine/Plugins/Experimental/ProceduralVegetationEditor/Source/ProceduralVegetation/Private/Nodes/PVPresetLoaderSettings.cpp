// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVPresetLoaderSettings.h"
#include "PCGContext.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVPresetLoaderSettings"

UPVPresetLoaderSettings::UPVPresetLoaderSettings()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		bExposeToLibrary = false;
	}
#endif
}

#if WITH_EDITOR
FText UPVPresetLoaderSettings::GetCategoryOverride() const
{
	return PV::Categories::Development;
}

FText UPVPresetLoaderSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "[DEPRECATED] Procedural Vegetation Preset Loader");
}

FText UPVPresetLoaderSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "DEPRECATED: This node no longer produces output. Use the Grower or Extract from Mesh/Image nodes to create new growth data.");
}
#endif

TArray<FPCGPinProperties> UPVPresetLoaderSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

TArray<FPCGPinProperties> UPVPresetLoaderSettings::OutputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}

FPCGElementPtr UPVPresetLoaderSettings::CreateElement() const
{
	return MakeShared<FPVPresetLoaderElement>();
}

bool FPVPresetLoaderElement::ExecuteInternal(FPCGContext* InContext) const
{
	PCGLog::LogErrorOnGraph(FText::FromString(TEXT("This node doesn't produce any output. Use the Grower or Extract from Mesh/Image nodes to create new growth data.")));
	
	return true;
}

#undef LOCTEXT_NAMESPACE
