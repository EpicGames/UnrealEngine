// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"

#include "PVPresetLoaderSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural), meta=(DeprecatedNode, DeprecationMessage="PVPresetLoaderSettings is deprecated and produces no output. Remove this node from your graph."))
class UPVPresetLoaderSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVPresetLoaderSettings();
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationPresetLoader")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(VisibleAnywhere, Category="Preset")
	TObjectPtr<class UProceduralVegetationPreset> Preset = nullptr;
};

class FPVPresetLoaderElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
