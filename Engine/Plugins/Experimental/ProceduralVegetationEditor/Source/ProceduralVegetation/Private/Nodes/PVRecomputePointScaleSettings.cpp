// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVRecomputePointScaleSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Implementations/PVRecomputePointScale.h"
#include "Helpers/PVAttributesHelper.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVRecomputePointScaleSettings"

UPVRecomputePointScaleSettings::UPVRecomputePointScaleSettings()
{
	TaperProfile.InitializeLinearCurve();
}

#if WITH_EDITOR
FLinearColor UPVRecomputePointScaleSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVRecomputePointScaleSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVRecomputePointScaleSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Recompute Point Scale"); 
}

FText UPVRecomputePointScaleSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Recompute each skeleton point's scale attribute (its radius along the plant)."
		"\n\n"
		"Replaces existing per-point scale values with newly-computed ones. Useful after importing or hand-editing a skeleton where the scales are wrong, or to apply a consistent taper. Three modes: explicit trunk radius, max-scale auto-detection, or smooth tip-to-base taper."
	);
}
#endif

FPCGDataTypeIdentifier UPVRecomputePointScaleSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVRecomputePointScaleSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVRecomputePointScaleSettings::CreateElement() const
{
	return MakeShared<FPVRecomputePointScaleElement>();
}

bool FPVRecomputePointScaleElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVRecomputePointScaleElement::Execute);

	check(InContext);

	const UPVRecomputePointScaleSettings* Settings = InContext->GetInputSettings<UPVRecomputePointScaleSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data);
		if (!InputData)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			continue;
		}

		FManagedArrayCollection OutCollection = InputData->GetCollection();

		UPVGrowthData* OutGrowthData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

		if (Settings->Mode == EPVRecomputePointScaleMode::UserTrunkRadius
			|| Settings->Mode == EPVRecomputePointScaleMode::MaxScaleAsTrunkRadius)
		{
			PV::FPointPlantGradientAttributeView PointPlantGradientAttribute = PV::FPointPlantGradientAttribute::AddAttribute(OutCollection);
			PV::AttributesHelper::ComputePointPlantGradient(OutCollection);
		}

		switch (Settings->Mode)
		{
		case EPVRecomputePointScaleMode::UserTrunkRadius:
			PV::ComputePointScales_UserTrunkScale(
				OutCollection, 
				Settings->TrunkRadius,
				Settings->TaperProfile.GetRichCurveConst()
			);
			break;
		case EPVRecomputePointScaleMode::MaxScaleAsTrunkRadius:
			PV::ComputePointScales_MaxScaleAsTrunkScale(
				OutCollection,
				Settings->TrunkRadiusScale,
				Settings->TaperProfile.GetRichCurveConst()
			);
			break;
		case EPVRecomputePointScaleMode::SmoothTaper:
			PV::ComputePointScales_SmoothTaper(
				OutCollection,
				Settings->TaperTolerance
			);
			break;
		}

#if WITH_EDITOR
		TArray<FPVVisualizationSettings> VisualizationSettings;
		FPVVisualizationSettings& RadiusVisualizationSettings = VisualizationSettings.AddDefaulted_GetRef();
		RadiusVisualizationSettings.DebugType = EPVDebugType::Custom;
		RadiusVisualizationSettings.bShowAnchorPoints = false;
		RadiusVisualizationSettings.bUsePivotAsPosition = true;
		RadiusVisualizationSettings.bDrawSphereAsMesh = true;
		RadiusVisualizationSettings.AttributeToFilter = PV::AttributeNames::PointScale;
		RadiusVisualizationSettings.CustomGroupToFilter = PV::GroupNames::PointGroup;
		RadiusVisualizationSettings.CustomPivotPositionAttributeName = PV::AttributeNames::PointPosition;
		RadiusVisualizationSettings.CustomPivotScaleAttributeName = NAME_None;
		RadiusVisualizationSettings.VisualizationMode = EPVDebugValueVisualizationMode::Sphere;

		OutGrowthData->AddVisualizationCollection(OutCollection, VisualizationSettings);
#endif

		OutGrowthData->Initialize(MoveTemp(OutCollection));
		InContext->OutputData.TaggedData.Emplace(OutGrowthData);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
