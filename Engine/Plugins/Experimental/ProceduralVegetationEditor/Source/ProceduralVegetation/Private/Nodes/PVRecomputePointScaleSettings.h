// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Utils/PVFloatRamp.h"
#include "PVRecomputePointScaleSettings.generated.h"

UENUM()
enum class EPVRecomputePointScaleMode
{
	// Scale is derived from a user-supplied absolute trunk radius value (TrunkRadius).
	UserTrunkRadius,
	// The largest scale component across all points is treated as the trunk radius.
	MaxScaleAsTrunkRadius,
	// Scale tapers smoothly from the base to the tip
	SmoothTaper
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVRecomputePointScaleSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVRecomputePointScaleSettings();

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("RecomputePointScale")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

public:

	UPROPERTY(EditAnywhere, Category = "RecomputePointScale", meta=(Tooltip="How to compute the new scales.\n\n`UserTrunkRadius`: set the trunk radius; everything else scales proportionally. `MaxScaleAsTrunkRadius`: largest existing scale is treated as the trunk; others are rescaled. `SmoothTaper`: scales taper smoothly from base to tip using a ramp profile."))
	EPVRecomputePointScaleMode Mode = EPVRecomputePointScaleMode::UserTrunkRadius;

	UPROPERTY(EditAnywhere, Category = "RecomputePointScale",
		meta=(EditCondition="Mode == EPVRecomputePointScaleMode::UserTrunkRadius", EditConditionHides, Tooltip="Trunk radius in cm.\n\nThe absolute radius (in cm) the trunk's thickest point should have. Other points scale relative to this."))
	float TrunkRadius = 1.f;

	UPROPERTY(EditAnywhere, Category = "RecomputePointScale",
		meta = (EditCondition = "Mode == EPVRecomputePointScaleMode::MaxScaleAsTrunkRadius", EditConditionHides, Tooltip="Multiplier applied to the auto-detected trunk radius.\n\nScales the auto-detected trunk radius by this factor. Use to thicken or thin the whole plant relative to its original values."))
	float TrunkRadiusScale = 1.f;

	UPROPERTY(EditAnywhere, Category = "RecomputePointScale",
		meta = (EditCondition = "Mode == EPVRecomputePointScaleMode::SmoothTaper", EditConditionHides, ClampMin = "1.0", Tooltip="Smoothing tolerance for the taper algorithm.\n\nHigher = smoother taper curves at the cost of less faithful reproduction of the original skeleton's local variations. 1 = minimal smoothing."))
	float TaperTolerance = 3.0f;

	UPROPERTY(EditAnywhere, Category = "RecomputePointScale",
		meta = (XAxisMin = 0.0f, XAxisMax = 1.0f, YAxisMin = -1.0f, YAxisMax = 1.0f,
				EditCondition = "Mode != EPVRecomputePointScaleMode::SmoothTaper", EditConditionHides, Tooltip="Curve controlling the per-position taper modulation.\n\nLets you override the default linear taper with a custom curve. X = position from base to tip. Y = scale modifier. Note: visible in non-SmoothTaper modes — `SmoothTaper` uses its own internal algorithm."))
	FPVFloatRamp TaperProfile;
};

class FPVRecomputePointScaleElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
