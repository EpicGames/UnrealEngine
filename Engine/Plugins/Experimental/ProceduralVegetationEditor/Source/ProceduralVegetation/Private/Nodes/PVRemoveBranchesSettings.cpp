// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVRemoveBranchesSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVRemoveBranches.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVRemoveBranchesSettings"

#if WITH_EDITOR
FLinearColor UPVRemoveBranchesSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVRemoveBranchesSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVRemoveBranchesSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Remove Branches");
}

FText UPVRemoveBranchesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Remove branches that meet a criterion (length, radius, light, age, or generation)."
		"\n\n"
		"Filters out branches based on a single per-branch attribute. Use to mimic natural pruning (shade-pruning short or thin twigs), to simplify a plant for performance, or to clean up the silhouette."
	);
}
#endif

FPCGDataTypeIdentifier UPVRemoveBranchesSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVRemoveBranchesSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVRemoveBranchesSettings::CreateElement() const
{
	return MakeShared<FPVRemoveBranchesElement>();
}

bool FPVRemoveBranchesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVRemoveBranchesElement::Execute);

	check(InContext);

	const UPVRemoveBranchesSettings* Settings = InContext->GetInputSettings<UPVRemoveBranchesSettings>();
	check(Settings);
	
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if(const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();
		
			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			
			FPVRemoveBranches::ApplyRemoveBranches(Settings->BranchRemoveBasis, Settings->Threshold, OutCollection);
			
			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
