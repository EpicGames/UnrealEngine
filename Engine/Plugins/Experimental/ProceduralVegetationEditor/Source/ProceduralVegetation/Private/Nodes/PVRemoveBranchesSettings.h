// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/PVBaseSettings.h"
#include "Implementations/PVRemoveBranches.h"
#include "PVRemoveBranchesSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVRemoveBranchesSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationRemoveBranches")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = Settings, meta=(PCG_Overridable, Tooltip="Which attribute decides whether a branch is removed.\n\nLength: short branches are removed. Radius: thin branches are removed. Light: shaded branches are removed. Age: old branches are removed. Generation: deep-hierarchy branches are removed."))
	ERemoveBranchesBasis BranchRemoveBasis = ERemoveBranchesBasis::Length;

	UPROPERTY(EditAnywhere, Category = Settings, meta=(PCG_Overridable, ClampMin = 0f, ClampMax = 1.0f, UIMin = 0f, UIMax = 1.0f, Tooltip="Cutoff threshold for the chosen basis.\n\nBranches exceeding (or falling below, depending on basis) this normalized threshold are removed. Use lower/higher values to prune sparsely or aggressively. 0.1-0.2 = light pruning; 0.5+ = heavy pruning."))
	float Threshold = 0;
};

class FPVRemoveBranchesElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
