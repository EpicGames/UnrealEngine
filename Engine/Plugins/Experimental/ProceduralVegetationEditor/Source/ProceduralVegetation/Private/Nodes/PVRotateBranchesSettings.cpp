// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVRotateBranchesSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVRotateBranchesSettings"

#if WITH_EDITOR
FLinearColor UPVRotateBranchesSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVRotateBranchesSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVRotateBranchesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "RotateBranches");
}

FText UPVRotateBranchesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Rotate branches around their parent's growth direction."
		"\n\n"
		"Rotates the axillary branches around the apical direction of its parent branch. Use to spread or twist branches independently of the original phyllotaxy. Combine ramp curves to vary the rotation along plant or branch length."
	);
}
#endif

FPCGDataTypeIdentifier UPVRotateBranchesSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVRotateBranchesSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVRotateBranchesSettings::CreateElement() const
{
	return MakeShared<FPVRotateBranchesElement>();
}

bool FPVRotateBranchesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVRotateBranchesElement::Execute);

	check(InContext);

	const UPVRotateBranchesSettings* Settings = InContext->GetInputSettings<UPVRotateBranchesSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();

			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

			FPVRotateBranches::ApplyRotateBranches(Settings->RotateBranchesParams, OutCollection);

			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
