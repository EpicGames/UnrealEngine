// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVSaveGrowerPresetSettings.h"

#include "PCGAssetExporterUtils.h"
#include "PCGContext.h"
#include "PVCommon.h"
#include "DataTypes/PVGrowerParamsData.h"
#include "DataAssets/ProceduralVegetationGrowerPreset.h"
#include "Utils/PCGLogErrors.h"

#define LOCTEXT_NAMESPACE "PVSaveGrowerPresetSettings"

namespace
{
	const FName GrowerParamsInputLabel = PCGPinConstants::DefaultInputLabel;
}

#if WITH_EDITOR
FLinearColor UPVSaveGrowerPresetSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::InputOutput;
}

FText UPVSaveGrowerPresetSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}


FText UPVSaveGrowerPresetSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Save Grower Preset");
}

FText UPVSaveGrowerPresetSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Save the current Grower's configuration as a reusable preset asset."
		"\n\n"
		"Captures the upstream Grower's fully-resolved parameters (including any Standalone Override nodes' contributions) and writes them to a ProceduralVegetationGrowerPreset asset on disk. Use to share configurations across graphs or to build a library of your authored plants."
	);
}
#endif

FPCGElementPtr UPVSaveGrowerPresetSettings::CreateElement() const
{
	return MakeShared<FPVSaveGrowerPresetElement>();
}

TArray<FPCGPinProperties> UPVSaveGrowerPresetSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;

	FPCGPinProperties& Pin = Properties.Emplace_GetRef(GrowerParamsInputLabel, FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowerParams::AsId() }, false, false);
	Pin.SetRequiredPin();

	return Properties;
}

bool FPVSaveGrowerPresetElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVSaveGrowerPresetElement::Execute);

	check(InContext);

#if WITH_EDITOR
	const UPVSaveGrowerPresetSettings* Settings = InContext->GetInputSettings<UPVSaveGrowerPresetSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(GrowerParamsInputLabel);

	if (Inputs.IsEmpty())
	{
		return true;
	}

	if (Inputs.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(GrowerParamsInputLabel, InContext);
	}

	const UPVGrowerParamsData* ParamsData = Cast<const UPVGrowerParamsData>(Inputs[0].Data);

	if (!ParamsData)
	{
		PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::Param, GrowerParamsInputLabel, InContext);
		return true;
	}

	UPCGAssetExporterUtils::CreateAsset<UProceduralVegetationGrowerPreset>(Settings->ExportParams, [&ParamsData](const FString&, UObject* Asset)
	{
		if (UProceduralVegetationGrowerPreset* GrowerPreset = Cast<UProceduralVegetationGrowerPreset>(Asset))
		{
			GrowerPreset->GrowthParams = ParamsData->Params;
			return true;
		}

		return false;

	}, InContext);

#else
	PCGLog::LogWarningOnGraph(LOCTEXT("CannotSaveInNonEditor", "Can't save a grower preset in a non-editor build."), InContext);
#endif

	return true;
}

#undef LOCTEXT_NAMESPACE
