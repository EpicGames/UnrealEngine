// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PCGAssetExporter.h"
#include "Nodes/PVBaseSettings.h"
#include "PVSaveGrowerPresetSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVSaveGrowerPresetSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SaveGrowerPreset")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
#endif

	virtual bool CanCullTaskIfUnwired() const override { return false; }

	UPROPERTY(EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable, Tooltip="Asset export configuration: target folder, filename, overwrite policy.\n\nWhere to save, what to name it, and what to do if the asset already exists. The preset asset can then be referenced from any Grower node's `Preset` field."))
	FPCGAssetExporterParameters ExportParams;

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override { return {}; }
	virtual FPCGElementPtr CreateElement() const override;
};

class FPVSaveGrowerPresetElement : public FPVBaseElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
