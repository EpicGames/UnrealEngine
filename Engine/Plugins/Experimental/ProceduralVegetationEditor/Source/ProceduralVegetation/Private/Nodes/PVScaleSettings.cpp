// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVScaleSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVScale.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVScaleSettings"

#if WITH_EDITOR
FLinearColor UPVScaleSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVScaleSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVScaleSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Scale"); 
}

FText UPVScaleSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Uniformly scale a grown plant up or down."
		"\n\n"
		"Multiplies every point's position and scale attribute by a single factor."
	);
}
#endif

FPCGDataTypeIdentifier UPVScaleSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVScaleSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVScaleSettings::CreateElement() const
{
	return MakeShared<FPVScaleElement>();
}

bool FPVScaleElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVScaleElement::Execute);

	check(InContext);

	const UPVScaleSettings* Settings = InContext->GetInputSettings<UPVScaleSettings>();
	check(Settings);
	
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if(const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();
		
			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			
			FPVScale::ApplyScale(Settings->Scale, OutCollection);
			
			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
