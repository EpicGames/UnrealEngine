// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/PVSeedGeneratorSettings.h"

#include "PCGContext.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "Data/PCGPointData.h"
#include "Facades/PVPointFacade.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVSeedGeneratorSettings"

FPCGDataTypeIdentifier UPVSeedGeneratorSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPCGDataTypeInfoPoint::AsId() };
}

FPCGDataTypeIdentifier UPVSeedGeneratorSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVSeedGeneratorSettings::CreateElement() const
{
	return MakeShared<FPVSeedGeneratorElement>();
}

bool FPVSeedGeneratorElement::ExecuteInternal(FPCGContext* InContext) const
{
	check(InContext);

	const UPVSeedGeneratorSettings* Settings = InContext->GetInputSettings<UPVSeedGeneratorSettings>();
	check(Settings);
	
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if(const UPCGPointData* InputData = Cast<UPCGPointData>(Input.Data))
		{
			FManagedArrayCollection OutCollection;
			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);
			
			FPVConvertToSeedPointImplementation::FillCollection(OutCollection, Settings->Params, InputData);
			
			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}
	
	return true;
}

#if WITH_EDITOR
FLinearColor UPVSeedGeneratorSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Seed;
}

FText UPVSeedGeneratorSettings::GetCategoryOverride() const
{
	return PV::Categories::Seed;
}


#endif

#undef LOCTEXT_NAMESPACE
