// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PCGSubgraph.h"
#include "Nodes/PVBaseSettings.h"
#include "DataTypes/PVData.h"
#include "Implementations/PVSeedGenerator.h"
#include "PVSeedGeneratorSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVSeedGeneratorSettings : public UPVBaseSettings
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("SeedGenerator")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("UPVSeedGeneratorSettings", "NodeTitle", "Seed Generator"); }
	virtual FText GetNodeTooltipText() const override
	{
		return NSLOCTEXT("UPVSeedGeneratorSettings", "NodeTooltip",
			"Convert PCG point data into Procedural Vegetation seed points.\n\n"
			"Takes PCG point data as input and produces seed points the Grower can consume. "
			"Each input point becomes one seed; per-seed parameters can bias the initial growth direction to avoid nearby seeds or the cluster center, producing more naturally spaced plant populations.");
	}
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::Seed }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	
	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = AvoidanceSettings, meta=(ShowOnlyInnerProperties))
	FPVConvertToSeedPointParams Params;
};

class FPVSeedGeneratorElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
