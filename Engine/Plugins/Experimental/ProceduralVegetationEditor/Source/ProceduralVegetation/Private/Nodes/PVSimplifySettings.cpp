// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSimplifySettings.h"
#include "DataTypes/PVGrowthData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Implementations/PVSimplifyPlantSkeleton.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVSimplifySettings"

#if WITH_EDITOR
FLinearColor UPVSimplifySettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVSimplifySettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVSimplifySettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Simplify");
}

FText UPVSimplifySettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Simplify the plant skeleton by removing redundant points."
		"\n\n"
		"Reduces skeleton complexity by removing points that don't significantly contribute to the branch shape. Uses the Douglas-Peucker line simplification algorithm. Useful for performance optimization without changing the overall plant silhouette."
	);
}
#endif

FPCGDataTypeIdentifier UPVSimplifySettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVSimplifySettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVSimplifySettings::CreateElement() const
{
	return MakeShared<FPVSimplifyElement>();
}

bool FPVSimplifyElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVSimplifyElement::Execute);

	check(InContext);

	const UPVSimplifySettings* Settings = InContext->GetInputSettings<UPVSimplifySettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();

			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

			if (Settings->Amount > 0.f)
			{
				PV::SimplifyPlantSkeleton(OutCollection, Settings->Amount);
			}

			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
