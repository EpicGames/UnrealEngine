// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSlopeSettings.h"
#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "Data/PCGBasePointData.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "Implementations/PVSlope.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVSlopeSettings"

#if WITH_EDITOR
FLinearColor UPVSlopeSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::PostGrowthModifiers;
}

FText UPVSlopeSettings::GetCategoryOverride() const
{
	return PV::Categories::PostGrowthModifiers;
}


FText UPVSlopeSettings::GetDefaultNodeTitle() const
{ 
	return LOCTEXT("NodeTitle", "Slope"); 
}

FText UPVSlopeSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip",
		"Simulate vegetation growing on a slope."
		"\n\n"
		"Tilts a fully-grown plant to mimic real-world slope adaptation: the trunk starts growing perpendicular to the slope surface and gradually curves toward vertical as it gains height. Useful for placing vegetation on hillsides, cliffs, or any sloped terrain."
	);
}
#endif

FPCGDataTypeIdentifier UPVSlopeSettings::GetInputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGDataTypeIdentifier UPVSlopeSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoGrowth::AsId() };
}

FPCGElementPtr UPVSlopeSettings::CreateElement() const
{
	return MakeShared<FPVSlopeElement>();
}

bool FPVSlopeElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVSlopeElement::Execute);

	check(InContext);

	const UPVSlopeSettings* Settings = InContext->GetInputSettings<UPVSlopeSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPVGrowthData* InputData = Cast<UPVGrowthData>(Input.Data))
		{
			FManagedArrayCollection OutCollection = InputData->GetCollection();

			UPVGrowthData* OutManagedArrayCollectionData = FPCGContext::NewObject_AnyThread<UPVGrowthData>(InContext);

			if (Settings->SlopeParams.SlopeAngle != 0.0f)
			{
				FPVSlope::ApplySlope(Settings->SlopeParams, OutCollection);
			}

			OutManagedArrayCollectionData->Initialize(MoveTemp(OutCollection));
			InContext->OutputData.TaggedData.Emplace(OutManagedArrayCollectionData);
		}
		else
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			return true;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
