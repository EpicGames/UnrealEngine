// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Nodes/PVSubgraphSettings.h"

#include "ProceduralVegetation.h"
#include "ProceduralVegetationModule.h"
#include "Helpers/PCGSettingsHelpers.h"
#include "PVCommon.h"

#define LOCTEXT_NAMESPACE "PVSubgraph"

UPCGGraphInterface* UPVSubgraphSettings::GetSubgraphInterface() const
{
	return SubgraphInstance ? SubgraphInstance->GetGraph() : nullptr;
}

bool FPVSubgraphElement::ExecuteInternal(FPCGContext* InContext) const
{
	if (const UPVSubgraphSettings* Settings = InContext->GetInputSettings<UPVSubgraphSettings>())
	{
		if (const UPCGGraph* Subgraph = Settings->GetSubgraph())
		{
			const UPCGGraph* CurrentGraph = nullptr;
			if (const FPCGStack* Stack = InContext->GetStack())
			{
				CurrentGraph = Stack->GetGraphForCurrentFrame();
			}
			else if (InContext->ExecutionSource.Get())
			{
				CurrentGraph = InContext->ExecutionSource->GetExecutionState().GetGraph();
			}

			if (CurrentGraph && (Subgraph == CurrentGraph || Subgraph->Contains(CurrentGraph)))
			{
				PCGE_LOG_C(Warning, GraphAndLog, InContext,
					FText::Format(LOCTEXT("PVSubgraphCyclic", "Subgraph '{0}' forms a cyclic reference back to the current graph."),
						FText::FromString(Subgraph->GetName())));
			}
		}
	}

	return FPCGSubgraphElement::ExecuteInternal(InContext);
}

void FPVSubgraphElement::PostExecuteInternal(FPCGContext* InContext) const
{
	FPCGSubgraphElement::PostExecuteInternal(InContext);

#if WITH_EDITORONLY_DATA
	const UPVSubgraphSettings* Settings = InContext->GetInputSettings<UPVSubgraphSettings>();
	check(Settings);

	for (FPCGTaggedData& OutputData : InContext->OutputData.TaggedData)
	{
		if (OutputData.Data && OutputData.Data.IsA<UPVData>())
		{
			if (const UPVData* ProceduralVegetationData = Cast<UPVData>(OutputData.Data))
			{
				FPVDebugSettings DebugSettings = ProceduralVegetationData->GetDebugSettings();
				DebugSettings.VisualizationSettings = Settings->GetDebugVisualizationSettings().VisualizationSettings;
				ProceduralVegetationData->SetDebugSettings(MoveTemp(DebugSettings));
			}
		}
	}
#endif
}

#if WITH_EDITOR
FLinearColor UPVSubgraphSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::Subgraph;
}


void UPVSubgraphSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPVSubgraphSettings, SubgraphInstance))
	{
		// Also rebuild the overrides
		InitializeCachedOverridableParams(/*bReset=*/true);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

EPCGChangeType UPVSubgraphSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	// Mark SubgraphInstance changes Structural so the base class rebinds OnGraphChangedDelegate when
	// the user retargets, undoes, or redoes the source PV graph reference.
	if ((InPropertyName == NAME_None) || (InPropertyName == GET_MEMBER_NAME_CHECKED(UPVSubgraphSettings, SubgraphInstance)))
	{
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}

UObject* UPVSubgraphSettings::GetJumpTargetForDoubleClick() const
{
	if (SubgraphInstance && SubgraphInstance->GetProceduralVegetationAsset() &&
		SubgraphInstance->GetProceduralVegetationAsset()->GetGraph())
	{
		return SubgraphInstance->GetProceduralVegetationAsset()->GetGraph();
	}
	
	return nullptr;
}

#endif

UPVSubgraphSettings::UPVSubgraphSettings(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	SubgraphInstance = InObjectInitializer.CreateDefaultSubobject<UProceduralVegetationInstance>(this, TEXT("ProceduralVegetationSubgraphInstance"));
}


#if WITH_EDITOR
void UPVSubgraphSettings::SetCurrentRenderType(TArray<EPVRenderType> InRenderTypes)
{
	Visualization.CurrentRenderType = MoveTemp(InRenderTypes);
}
#endif

UPCGNode* UPVSubgraphSettings::CreateNode() const
{
	return NewObject<UPCGSubgraphNode>();
}

FString UPVSubgraphSettings::GetAdditionalTitleInformation() const
{
	if (auto PVSubgraph = SubgraphInstance ? SubgraphInstance->GetProceduralVegetationAsset() : nullptr)
	{
		FString GraphName; 
		
		if (PVSubgraph->GetGraph() && 
			PVSubgraph->GetGraph()->GetGraph() && 
			PVSubgraph->GetGraph()->GetGraph()->IsEmbeddedSubgraph())
		{
			UPCGGraph* SourceGraph = PVSubgraph->GetGraph()->GetGraph();
			GraphName =  SourceGraph->GetDisplayName().ToString();
		}
		else
		{
			GraphName = PVSubgraph->GetName();
		}
		
		// Use the same transformation than in the palette view to add spaces between uppercase characters
		return FName::NameToDisplayString(GraphName, /*bIsBool=*/false);
	}
	
	return LOCTEXT("NodeTitleExtendedInvalidSubgraph", "Missing Subgraph").ToString();
}

FPCGElementPtr UPVSubgraphSettings::CreateElement() const
{
	return MakeShared<FPVSubgraphElement>();
}

void UPVSubgraphSettings::SetSubgraphInternal(UPCGGraphInterface* InGraph)
{
	if (SubgraphInstance && SubgraphInstance->GraphInstance)
	{
		if (UProceduralVegetationGraph* ProceduralVegetationGraph = Cast<UProceduralVegetationGraph>(InGraph))
		{
			if (ProceduralVegetationGraph->IsEmbeddedSubgraph())
			{
				UProceduralVegetation* ProceduralVegetation = NewObject<UProceduralVegetation>(this,
					NAME_None,
					RF_Transactional | RF_Public);

				if (ensure(ProceduralVegetation))
				{
					ProceduralVegetation->SetGraph(ProceduralVegetationGraph);
					SubgraphInstance->SetProceduralVegetationAsset(ProceduralVegetation);
				}
			}
			else
			{
				// Standalone PV graph: reuse the wrapping asset; don't create a per-node wrapper.
				if (UProceduralVegetation* OuterAsset = ProceduralVegetationGraph->GetTypedOuter<UProceduralVegetation>())
				{
					SubgraphInstance->SetProceduralVegetationAsset(OuterAsset);
				}
			}

			SubgraphInstance->GraphInstance->SetGraph(InGraph);
		}
		else
		{
			SubgraphInstance->SetProceduralVegetationAsset(nullptr);
			SubgraphInstance->GraphInstance->SetGraph(nullptr);
		}
	}
}

#undef LOCTEXT_NAMESPACE
