// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Nodes/PVTrunkTextureSetupSettings.h"
#include "DataTypes/PVGrowthData.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture.h"
#include "Implementations/PVTrunkTextureSetup.h"
#include "Misc/Paths.h"
#include "PVCommon.h"

class UTexture2D;

UPVTrunkTextureSetupSettings::UPVTrunkTextureSetupSettings()
{
	TrunkTextureSetupParams.BakeTextureFolder.Path = "/Game/ProceduralVegetation";
}

#if WITH_EDITOR
FLinearColor UPVTrunkTextureSetupSettings::GetNodeTitleColor() const
{
	return PV::NodeColors::InputOutput;
}

FText UPVTrunkTextureSetupSettings::GetCategoryOverride() const
{
	return PV::Categories::InputOutput;
}



// Rebuilds the trim-sheet render targets (needed for UV data and live preview) while
// preserving the saved baked UTexture2D references in the material.
// CreateTrimSheetTexture empties Channels and fills it with transient render targets, then
// calls SetupMaterial — which would leave the material referencing those transient objects
// after a load or undo.  When baked, we snapshot the serialised UTexture2D refs first,
// let CreateTrimSheetTexture run, then restore the baked assets and call SetupMaterial again.
static void RebuildTrimSheetPreservingBakedTextures(FPVTrunkTextureSetupParams& Params,
                                                    FPVTrunkTextureSetupInfo& Info,
                                                    const bool bBaked)
{
	TMap<FString, TObjectPtr<UTexture>> BakedChannels;
	if (bBaked)
	{
		BakedChannels = Info.Channels;
	}

	FPVTrunkTextureSetupImplementation::CreateTrimSheetTexture(Params, Info);

	if (bBaked && !BakedChannels.IsEmpty())
	{
		for (auto& Pair : BakedChannels)
		{
			if (Pair.Value)
			{
				Info.Channels.Add(Pair.Key, Pair.Value);
			}
		}
		FPVTrunkTextureSetupImplementation::SetupMaterial(Params.Material, Info);
	}
}

void UPVTrunkTextureSetupSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	
	if (!ensure(PropertyChangedEvent.PropertyChain.GetActiveNode()))
	{
		return;
	}
	
	auto Node = (PropertyChangedEvent.PropertyChain.GetActiveNode()->GetPrevNode());
	
	if (Node && Node->GetValue()->GetName() == GET_MEMBER_NAME_CHECKED(FPVTextureChannelParams, TextureFilePath))
	{
		int GenerationIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_CHECKED(FPVTrunkTextureSetupParams, Generations).ToString());
		int ChannelIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_CHECKED(FPVTextureGenerationParams, Channels).ToString());

		TrunkTextureSetupParams.LoadGenerationChannelTexture(GenerationIndex, ChannelIndex);
	}

	const EPropertyChangeType::Type ChangeType = PropertyChangedEvent.ChangeType;

	if (ChangeType == EPropertyChangeType::ValueSet)
	{
		// Some properties do not affect the baked texture content and must not
		// invalidate the baked state: Mode (UI-only switch), Material (binding only),
		// AssetNamePrefix / BakeTextureFolder (output path, not texture data).
		// BakeTextureFolder is an FDirectoryPath struct so its leaf property is "Path";
		// we detect it via the prev-node in the chain, same as TextureFilePath above.
		const FName ChangedPropName = PropertyChangedEvent.GetPropertyName();
		const auto* PrevNode = PropertyChangedEvent.PropertyChain.GetActiveNode()->GetPrevNode();
		const FName PrevNodeName = (PrevNode && PrevNode->GetValue()) ? FName(PrevNode->GetValue()->GetName()) : NAME_None;

		const bool bBakeStatePreserved =
			ChangedPropName == GET_MEMBER_NAME_CHECKED(FPVTrunkTextureSetupParams, Mode)            ||
			ChangedPropName == GET_MEMBER_NAME_CHECKED(FPVTrunkTextureSetupParams, Material)        ||
			ChangedPropName == GET_MEMBER_NAME_CHECKED(FPVTrunkTextureSetupParams, AssetNamePrefix) ||
			PrevNodeName    == GET_MEMBER_NAME_CHECKED(FPVTrunkTextureSetupParams, BakeTextureFolder);

		if (!bBakeStatePreserved)
		{
			FPVTrunkTextureSetupImplementation::CreateTrimSheetTexture(TrunkTextureSetupParams, TrunkTextureSetupInfo);
			bTexturesBaked = false;
			BakingErrors.Empty();
		}
		else if (ChangedPropName == GET_MEMBER_NAME_CHECKED(FPVTrunkTextureSetupParams, Material))
		{
			// A new material was assigned — apply the current channels to it.
			// When baked, use RebuildTrimSheetPreservingBakedTextures (same as PostLoad/PostEditUndo)
			// so that UTexture2D baked refs are snapshotted, a fresh trim sheet is built,
			// then the baked refs are restored before SetupMaterial runs.
			// When not yet baked, just wire the current render-target channels into the material.
			if (bTexturesBaked)
			{
				RebuildTrimSheetPreservingBakedTextures(TrunkTextureSetupParams, TrunkTextureSetupInfo, true);
			}
			else if (TrunkTextureSetupParams.Material)
			{
				FPVTrunkTextureSetupImplementation::SetupMaterial(TrunkTextureSetupParams.Material, TrunkTextureSetupInfo);
			}
		}
	}
	else if (ChangeType == EPropertyChangeType::ArrayRemove ||
	         ChangeType == EPropertyChangeType::ArrayAdd    ||
	         ChangeType == EPropertyChangeType::ArrayClear  ||
	         ChangeType == EPropertyChangeType::Duplicate)
	{
		// Structural array changes (adding/removing/clearing generations or channels)
		// always require a full trim-sheet rebuild and invalidate any prior bake.
		FPVTrunkTextureSetupImplementation::CreateTrimSheetTexture(TrunkTextureSetupParams, TrunkTextureSetupInfo);
		bTexturesBaked = false;
		BakingErrors.Empty();
	}
}

void UPVTrunkTextureSetupSettings::PostLoad()
{
	Super::PostLoad();

	if (TrunkTextureSetupParams.Mode == EPVTrunkTextureSetupMode::TextureCreate && !bTexturesBaked)
	{
		int GenerationIndex = 0;
		for (auto Generation : TrunkTextureSetupParams.Generations)
		{
			int ChannelIndex = 0;
			for (auto Channel : Generation.Channels)
			{
				TrunkTextureSetupParams.LoadGenerationChannelTexture(GenerationIndex, ChannelIndex);
				ChannelIndex++;
			}
			GenerationIndex++;
		}
		
		RebuildTrimSheetPreservingBakedTextures(TrunkTextureSetupParams, TrunkTextureSetupInfo, bTexturesBaked);
	}
}

void UPVTrunkTextureSetupSettings::PostEditUndo()
{
	Super::PostEditUndo();
	if (TrunkTextureSetupParams.Mode == EPVTrunkTextureSetupMode::TextureCreate && !bTexturesBaked)
	{
		RebuildTrimSheetPreservingBakedTextures(TrunkTextureSetupParams, TrunkTextureSetupInfo, bTexturesBaked);
	}
}

#endif

TArray<FPCGPinProperties> UPVTrunkTextureSetupSettings::InputPinProperties() const
{
	return TArray<FPCGPinProperties>();
}
 
FPCGElementPtr UPVTrunkTextureSetupSettings::CreateElement() const
{
	return MakeShared<FPVTrunkTextureSetupElement>();
}
 
FPCGDataTypeIdentifier UPVTrunkTextureSetupSettings::GetOutputPinTypeIdentifier() const
{
	return FPCGDataTypeIdentifier{ FPVDataTypeInfoTrunkTextureSetup::AsId() };
}

void UPVTrunkTextureSetupSettings::BakeTextures()
{
#if WITH_EDITOR
	BakingErrors.Empty();
	for (auto& Channel : TrunkTextureSetupInfo.Channels)
	{
		FString Error;
		if (UTexture* ConvertedTexture = Cast<UTexture>(FPVTrunkTextureSetupImplementation::ConvertRenderTargetToTexture2D(Cast<UTextureRenderTarget2D>(Channel.Value),
				TrunkTextureSetupParams.BakeTextureFolder.Path, Error)))
		{
			Channel.Value = ConvertedTexture;
		}

		if (!Error.IsEmpty())
		{
			Error = Error.Replace(TEXT("Export Path"), TEXT("Bake Path"));
			BakingErrors += Error + "\n";
		}
	}
	
	if (BakingErrors.IsEmpty())
	{
		FPVTrunkTextureSetupImplementation::SetupMaterial(TrunkTextureSetupParams.Material, TrunkTextureSetupInfo);
		bTexturesBaked = true;
		FPropertyChangedEvent SettingsChangedEvent(nullptr);
		PostEditChangeProperty(SettingsChangedEvent);
	}
#else
	UE_LOGF(LogTemp, Display, "Baking Textures is not supported");
#endif
}

bool FPVTrunkTextureSetupElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVImporterElement::Execute);
 
	check(InContext);
 
	const UPVTrunkTextureSetupSettings* Settings = InContext->GetInputSettings<UPVTrunkTextureSetupSettings>();
	check(Settings);
	
	UPVTrunkTextureSetupData* OutTrunkTextureSetupData = FPCGContext::NewObject_AnyThread<UPVTrunkTextureSetupData>(InContext);
	
	OutTrunkTextureSetupData->TrunkTextureSetupInfo = Settings->TrunkTextureSetupInfo;
	InContext->OutputData.TaggedData.Emplace(OutTrunkTextureSetupData);

	const EPVTrunkTextureSetupMode Mode = Settings->TrunkTextureSetupParams.Mode;

	if (Mode == EPVTrunkTextureSetupMode::TextureCreate)
	{
		const bool bIsBaked = Settings->GetTextureBaked();
		const bool bNoGenerations = Settings->TrunkTextureSetupParams.Generations.IsEmpty();

		bool bAnyGenerationHasNoChannels = false;
		for (const FPVTextureGenerationParams& Gen : Settings->TrunkTextureSetupParams.Generations)
		{
			if (Gen.Channels.IsEmpty())
			{
				bAnyGenerationHasNoChannels = true;
				break;
			}
		}

		// Warning 1: No generations configured and textures not yet baked.
		if (!bIsBaked && bNoGenerations)
		{
			PCGLog::LogWarningOnGraph(FText::FromString(TEXT("No generations added. Add at least one generation with channels to configure the texture layout.")), InContext);
		}
		// Warning 2: Generations present but some have no channels, and textures not yet baked.
		else if (!bIsBaked && bAnyGenerationHasNoChannels)
		{
			PCGLog::LogWarningOnGraph(FText::FromString(TEXT("One or more generations have no channels. Add channels to all generations before baking.")), InContext);
		}
		else
		{
			// Width ratio and overflow are only meaningful once generations and channels exist.
			int GenerationIndex = 0;
			for (const FPVTextureGenerationParams& Generation : Settings->TrunkTextureSetupParams.Generations)
			{
				if (!Generation.IsWidthRatioSame())
				{
					PCGLog::LogWarningOnGraph(FText::FromString(FString::Printf(TEXT("Generation %i does not have the same width to height ratio for all channels. "
						"\nMake sure all channels in a generation have the same width height ratio."), GenerationIndex)), InContext);
				}
				++GenerationIndex;
			}

			if (Settings->TrunkTextureSetupInfo.IsOverflowing())
			{
				PCGLog::LogWarningOnGraph(FText::FromString(TEXT("Generations are not fitting in the texture, consider scaling down the XScale values in generation settings.")), InContext);
			}

			if (!Settings->BakingErrors.IsEmpty())
			{
				PCGLog::LogWarningOnGraph(FText::FromString(FString::Printf(TEXT("Texture baking errors: %s"), *Settings->BakingErrors)), InContext);
			}

			// Warning 3: Everything is configured but textures have not been baked yet.
			if (!bIsBaked)
			{
				PCGLog::LogWarningOnGraph(FText::FromString(TEXT("Textures have not been baked. Click 'Bake Textures' to generate and save them.")), InContext);
			}
			else
			{
				PCGLog::LogWarningOnGraph(FText::FromString(TEXT("Textures are baked. Switch to Texture Load mode to use the baked textures in the material.")), InContext);
			}
		}

		// Warning 4: No material assigned (independent of bake/generation state).
		if (!Settings->TrunkTextureSetupParams.Material)
		{
			PCGLog::LogWarningOnGraph(FText::FromString(TEXT("No material assigned. Assign a material so baked textures can be applied to it.")), InContext);
		}
		// Warning 5: Material assigned but missing texture parameters for one or more channels.
		// Mirrors SetupMaterial's matching exactly: for each material param, normalise its
		// name via GetTextureChannelFromString to find the channel it covers. Custom-mapped
		// params fall back to a literal name match. This handles spaces, underscores, dashes
		// and case differences between param names and channel keys.
		else if (!Settings->TrunkTextureSetupInfo.Channels.IsEmpty())
		{
			TArray<FMaterialParameterInfo> TextureParamInfos;
			TArray<FGuid> TextureParamGuids;
			Settings->TrunkTextureSetupParams.Material->GetAllTextureParameterInfo(TextureParamInfos, TextureParamGuids);

			// Build the set of channel names that at least one material param covers.
			TSet<FString> CoveredChannelNames;
			for (const FMaterialParameterInfo& Info : TextureParamInfos)
			{
				const FString ParamName = Info.Name.ToString();
				const EPVTextureChannel MappedChannel = FPVTrunkTextureSetupImplementation::GetTextureChannelFromString(ParamName);
				if (MappedChannel != EPVTextureChannel::Custom)
				{
					CoveredChannelNames.Add(StaticEnum<EPVTextureChannel>()->GetNameStringByValue((int64)MappedChannel));
				}
				else
				{
					// For unrecognised params, fall back to the literal name (custom channels).
					CoveredChannelNames.Add(ParamName);
				}
			}

			TArray<FString> MissingChannels;
			for (const auto& Channel : Settings->TrunkTextureSetupInfo.Channels)
			{
				if (!CoveredChannelNames.Contains(Channel.Key))
				{
					MissingChannels.Add(Channel.Key);
				}
			}

			if (!MissingChannels.IsEmpty())
			{
				PCGLog::LogWarningOnGraph(FText::FromString(FString::Printf(
					TEXT("Material '%s' has no texture parameters defined for the following channels: %s."),
					*Settings->TrunkTextureSetupParams.Material->GetName(),
					*FString::Join(MissingChannels, TEXT(", ")))), InContext);
			}
		}

		// Validate texture file paths for all channels.
		// Paths containing commas break file-system lookups and fail silently; report them
		// explicitly. Also report any non-empty path whose texture still failed to load.
		for (int32 GenIdx = 0; GenIdx < Settings->TrunkTextureSetupParams.Generations.Num(); ++GenIdx)
		{
			const FPVTextureGenerationParams& Gen = Settings->TrunkTextureSetupParams.Generations[GenIdx];
			for (int32 ChIdx = 0; ChIdx < Gen.Channels.Num(); ++ChIdx)
			{
				const FPVTextureChannelParams& Channel = Gen.Channels[ChIdx];
				const FString& FilePath = Channel.TextureFilePath.FilePath;
				if (FilePath.IsEmpty())
				{
					continue;
				}
				// First check for characters that are never valid anywhere in a file path.
				// '/', '' and ':' are excluded because they have legitimate uses as path
				// separators and the drive-letter specifier respectively.
				TCHAR FoundInvalidChar = 0;
				for (const TCHAR Ch : FPaths::GetInvalidFileSystemChars())
				{
					if (Ch == TEXT('/') || Ch == TEXT('\\') || Ch == TEXT(':'))
					{
						continue;
					}
					if (FilePath.Contains(FString(1, &Ch)))
					{
						FoundInvalidChar = Ch;
						break;
					}
				}
				if (FoundInvalidChar != 0)
				{
					PCGLog::LogErrorOnGraph(FText::FromString(FString::Printf(
						TEXT("Generation %d, Channel '%s': texture file path contains an invalid character ('%c'). Remove it from the file name or folder path. Path: '%s'"),
						GenIdx, *Channel.GetChannelName(), FoundInvalidChar, *FilePath)), InContext);
				}
				else if (!FPaths::ValidatePath(FilePath))
				{
					PCGLog::LogErrorOnGraph(FText::FromString(FString::Printf(
						TEXT("Generation %d, Channel '%s': texture file path is not a valid path. Path: '%s'"),
						GenIdx, *Channel.GetChannelName(), *FilePath)), InContext);
				}
				else if (!Channel.Texture)
				{
					PCGLog::LogErrorOnGraph(FText::FromString(FString::Printf(
						TEXT("Generation %d, Channel '%s': texture file not found or could not be loaded. Path: '%s'"),
						GenIdx, *Channel.GetChannelName(), *FilePath)), InContext);
				}
			}
		}
	}
	else if (Mode == EPVTrunkTextureSetupMode::TextureLoad)
	{
		if (!Settings->GetTextureBaked())
		{
			PCGLog::LogErrorOnGraph(FText::FromString(FString::Printf(TEXT("No baked textures found at '%s'. Switch to Texture Create mode and bake the textures first."),
				*Settings->TrunkTextureSetupParams.BakeTextureFolder.Path)), InContext);
		}
	}

	return true;
}
