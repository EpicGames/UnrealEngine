// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace PV::Pins
{
	const FName GrowerPhyllotaxyInputLabel = TEXT("Phyllotaxy");
	const FName GrowerPhototropismInputLabel = TEXT("Phototropism");
	const FName GrowerBifurcationInputLabel = TEXT("Bifurcation");
	const FName GrowerAgeSenescenceInputLabel = TEXT("AgeSenescence");
	const FName GrowerLightSenescenceInputLabel = TEXT("LightSenescence");
	const FName GrowerGuideInputLabel = TEXT("Guide");
	const FName GrowerGravityInputLabel = TEXT("Gravity");
	const FName GrowerFoliageInputLabel = TEXT("Foliage");
	const FName GrowerGrowthInputLabel = TEXT("Growth");
	const FName GrowerAuxinInputLabel = TEXT("Auxin");
	const FName GrowerDirectionalInputLabel = TEXT("Directional");
	const FName GrowerParamsOutputLabel = TEXT("Params");
	
	const FName FoliageDistributorFoliageInputLabel = TEXT("Foliage");

	const FName DistributionParametricSettingsInputLabel   = TEXT("ParametricSettings");
	const FName DistributionHormoneBasedSettingsInputLabel = TEXT("HormoneBasedSettings");
	const FName DistributionVectorSettingsInputLabel       = TEXT("VectorSettings");
	const FName DistributionConditionSettingsInputLabel    = TEXT("ConditionSettings");

	const FName MeshBuilderProfileInputLabel = TEXT("Profile");

	const FName MeshBuilderSkeletonShapingInputLabel = TEXT("SkeletonShaping");
	const FName MeshBuilderBranchRadiusInputLabel    = TEXT("BranchRadius");
	const FName MeshBuilderProfileDetailsInputLabel  = TEXT("ProfileDetails");
	const FName MeshBuilderMeshDetailsInputLabel     = TEXT("MeshDetails");
	const FName MeshBuilderMaterialDetailsInputLabel = TEXT("MaterialDetails");
	const FName MeshBuilderDisplacementInputLabel    = TEXT("Displacement");
}

namespace PV::NodeColors
{
	inline const FLinearColor Growth                = FLinearColor(FColor::FromHex(TEXT("EFB09EFF")));
	inline const FLinearColor InputOutput           = FLinearColor(FColor::FromHex(TEXT("EFB441FF")));
	inline const FLinearColor Foliage               = FLinearColor(FColor::FromHex(TEXT("7BF851FF")));
	inline const FLinearColor PostGrowthModifiers   = FLinearColor(FColor::FromHex(TEXT("FCFD58FF")));
	inline const FLinearColor Mesh                  = FLinearColor(FColor::FromHex(TEXT("7DEDF1FF")));
	inline const FLinearColor Seed                  = FLinearColor(FColor::FromHex(TEXT("EFB09EFF")));
	inline const FLinearColor Subgraph              = FLinearColor(FColor::FromHex(TEXT("E55348FF")));
	inline const FLinearColor BoneReduction         = FLinearColor(FColor::FromHex(TEXT("E640C4FF")));
	inline const FLinearColor Development           = FLinearColor(FColor::Black);
}

namespace PV::Categories
{
	inline const FText Growth                = NSLOCTEXT("PVNodeCategories", "CategoryGrowthLabel",               "Growth");
	inline const FText InputOutput           = NSLOCTEXT("PVNodeCategories", "CategoryInputOutputLabel",          "Input Output");
	inline const FText Foliage               = NSLOCTEXT("PVNodeCategories", "CategoryFoliageLabel",              "Foliage");
	inline const FText PostGrowthModifiers   = NSLOCTEXT("PVNodeCategories", "CategoryPostGrowthModifiersLabel",  "Post Growth Modifiers");
	inline const FText Mesh                  = NSLOCTEXT("PVNodeCategories", "CategoryMeshLabel",                 "Mesh");
	inline const FText Seed                  = NSLOCTEXT("PVNodeCategories", "CategorySeedLabel",                 "Seed");
	inline const FText DistributionSettings  = NSLOCTEXT("PVNodeCategories", "CategoryDistributionSettingsLabel", "Distribution Settings");
	inline const FText GrowthSettings        = NSLOCTEXT("PVNodeCategories", "CategoryGrowthSettingsLabel",       "Growth Settings");
	inline const FText MeshBuilderSettings   = NSLOCTEXT("PVNodeCategories", "CategoryMeshBuilderSettingsLabel",  "Mesh Builder Settings");
	inline const FText Development           = NSLOCTEXT("PVNodeCategories", "CategoryDevelopmentLabel",          "Development");
}