// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace PVImportNames
{
	// Common
	inline static const FName BranchHierarchyGroup = TEXT("BranchCurves");
	inline static const FName BranchHierarchyAttribute = TEXT("Curve");
	inline static const FName LabelsGroup = TEXT("Labels");
	inline static const FName LabelPositionAttribute = TEXT("Position");
	inline static const FName LabelScaleAttribute = TEXT("Scale");
	inline static const FName LabelTextAttribute = TEXT("Text");
	inline static const FName TipVisualizationGroup = TEXT("EndPoints");
	inline static const FName TipVisualizationSizeAttribute = TEXT("Size");
	inline static const FName TipVisualizationPositionAttribute = TEXT("Vertex");

	// 2D specific
	inline static const FName PixelsDebugGroup = TEXT("PixelDebug");
	inline static const FName PixelPositionsAttribute = TEXT("Position");
	inline static const FName PixelValueAttribute = TEXT("Value");
	inline static const FName PerimeterCurveGroup = TEXT("PerimeterCurves");
	inline static const FName PerimeterCurveAttribute = TEXT("PerimeterCurve");
	inline static const FName UserRootPositionsGroup = TEXT("RootPositions");
	inline static const FName UserRootPositionsAttribute = TEXT("Vertex");
};