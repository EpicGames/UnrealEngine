// Copyright Epic Games, Inc. All Rights Reserved.

#include "Params/PVExportParams.h"
#include "Helpers/PVUtilities.h"

#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"

UClass* FPVExportParams::GetMeshClass() const
{
	return ExportMeshType == EPVExportMeshType::StaticMesh ? UStaticMesh::StaticClass() : USkeletalMesh::StaticClass();
}

bool FPVExportParams::Validate(FString& OutError) const
{
	const FString& OutputPath = ContentBrowserFolder.Path;

	if (!PV::Utilities::ValidateAssetPathAndName(MeshName.ToString(), OutputPath, GetMeshClass(), OutError))
	{
		return false;
	}

	if (ExportMeshType == EPVExportMeshType::SkeletalMesh)
	{
		if (!PV::Utilities::ValidateAssetPathAndName(GetOutputSkeletonName(), OutputPath, USkeleton::StaticClass(), OutError))
		{
			return false;
		}
	}

	if (ReplacePolicy == EPVAssetReplacePolicy::Replace)
	{
		const TCHAR* ConflictingAssetError = TEXT("Create a Unique name for the output as another asset of different type {0} exists with same name {1}");

		const FString MeshPackagePath = GetOutputMeshPackagePath();
		if (PV::Utilities::DoesConflictingPackageExist(MeshPackagePath, GetMeshClass()))
		{
			OutError = FString::Format(ConflictingAssetError, { *GetMeshClass()->GetName(), *MeshPackagePath });
			return false;
		}

		if (ExportMeshType == EPVExportMeshType::SkeletalMesh)
		{
			const FString SkeletonPackagePath = GetOutputSkeletonPackagePath();
			if (PV::Utilities::DoesConflictingPackageExist(SkeletonPackagePath, USkeleton::StaticClass()))
			{
				OutError = FString::Format(ConflictingAssetError, { *USkeleton::StaticClass()->GetName(), *SkeletonPackagePath });
				return false;
			}
		}
	}

	return true;
}