// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/EngineTypes.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "PVImportStaticMeshParams.generated.h"

class UStaticMesh;

UENUM(BlueprintType)
enum class EPVImportStaticMeshDebugState : uint8
{
	None,
	InitialMesh,
	ComputeNormals,
	ComputeVerticesCenterAndRadius,
	CollapseBranchesToLine,
	FindEndPoints,
	FindRootPoints,
	ConnectBranchesToParent,
	ComputeBranchHierarchies,
	MAX UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FPVImportStaticMeshParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Asset", meta=(Tooltip="The static mesh to trace.\n\nPick a Static Mesh asset from the content browser. The mesh's geometry is analysed to identify branch tips, roots, and the hierarchy connecting them."))
	TObjectPtr<UStaticMesh> StaticMeshAsset = nullptr;

	UPROPERTY(EditAnywhere, Category = "Asset", meta=(Tooltip="Names of materials to include in the trace.\n\nOnly mesh sections using one of these material slot names contribute to the extracted skeleton. Use to ignore leaf or foliage materials and keep only the bark/structural geometry."))
	TArray<FName> MaterialsToKeep;
};

USTRUCT(BlueprintType)
struct FPVImportStaticMeshDebugParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "DebugSettings")
	EPVImportStaticMeshDebugState DebugState = EPVImportStaticMeshDebugState::None;

	UPROPERTY(EditAnywhere, Category = "DebugSettings")
	bool bShowFoundTips = true;

	UPROPERTY(EditAnywhere, Category = "DebugSettings")
	bool bShowFoundBranchCurves = true;
};

struct FPVImportStaticMeshOutput
{
	TOptional<FManagedArrayCollection> DebugCollection;
	FManagedArrayCollection VisualizationCollection;
	FManagedArrayCollection GrowthDataCollection;
};
