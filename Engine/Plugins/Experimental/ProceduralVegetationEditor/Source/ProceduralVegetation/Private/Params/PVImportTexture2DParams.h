// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/EngineTypes.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Math/Matrix.h"
#include "Data/PCGTextureData.h"
#include "PVImportTexture2DParams.generated.h"

class UTexture2D;

namespace PVTexture2DImporterVisualization
{
	constexpr float Scale = 100.f;
	inline const FVector3f Offset = FVector3f(0, -Scale / 2.f, 25.f);

	inline FMatrix44f GetUVToWorldMatrix()
	{
		// Maps FVector3f(u, v, 0) → FVector3f(0, u*Scale+Offset.Y, (1-v)*Scale+Offset.Z)
		return FMatrix44f(
			FPlane4f(0.f, Scale,     0.f,              0.f),
			FPlane4f(0.f, 0.f,      -Scale,            0.f),
			FPlane4f(0.f, 0.f,       0.f,              0.f),
			FPlane4f(0.f, Offset.Y,  Scale + Offset.Z, 1.f)
		);
	}

	inline FMatrix44f GetWorldToUVMatrix()
	{
		// GetUVToWorldMatrix() maps to the YZ plane (X is always 0), making it singular and non-invertible.
		// Derive the inverse analytically: u = (Y - Offset.Y) / Scale, v = 1 - (Z - Offset.Z) / Scale
		const float InvScale = 1.f / Scale;
		return FMatrix44f(
			FPlane4f(0.f,                   0.f,                       0.f, 0.f),
			FPlane4f(InvScale,              0.f,                       0.f, 0.f),
			FPlane4f(0.f,                  -InvScale,                  0.f, 0.f),
			FPlane4f(-Offset.Y * InvScale,  1.f + Offset.Z * InvScale, 0.f, 1.f)
		);
	}
};

UENUM(BlueprintType)
enum class EPVImportTexture2DDebugState : uint8
{
	None,
	TriangulateCurves,
	ComputeVertexScale,
	ComputeBranchHierarchies,
	MAX UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FPVImportTexture2DPlantSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(Tooltip="2D position in the image where this plant's root sits.\n\nWhen the image contains multiple plants, this anchors each one's root in 2D image coordinates."))
	FVector2f RootPosition = FVector2f::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(Tooltip="Rotation applied to the extracted skeleton in degrees.\n\nUseful for rotating the plant after extraction so it faces the correct direction in 3D space."))
	float Rotation = 0;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(Tooltip="Uniform scale applied to the extracted skeleton.\n\nDefault 100 = image units treated as cm. Adjust to match your scene scale."))
	float Scale = 100.f;
};

USTRUCT(BlueprintType)
struct FPVImportTexture2DParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Texture", meta=(Tooltip="The texture asset to trace.\n\nPick a texture from the content browser."))
	TObjectPtr<UTexture2D> Texture2DAsset = nullptr;

	UPROPERTY(EditAnywhere, Category = "Texture",
		meta = (EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Invert image colors (use for dark plant on light background).\n\nWhen on, the brightness is inverted before tracing. Use when your source image has dark plant shapes on a light background."))
	bool bInvertImage = false;

	UPROPERTY(EditAnywhere, Category = "Texture",
		meta = (ClampMin = 0, ClampMax = 1, UIMin = 0, UIMax = 1, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Brightness threshold for 'plant' vs. 'background'.\n\nPixels above this brightness are considered plant; below are background. Default 0.01 = nearly any non-black pixel is plant. Raise to ignore noise; lower to include faint pixels."))
	float WhiteLevel = 0.01f;

	UPROPERTY(EditAnywhere, Category = "Texture",
		meta = (ClampMin = 0, UIMin = 0, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Which color channel to read brightness from.\n\nWhich color channel to read brightness from Red / Green / Blue / Alpha."))
	EPCGTextureColorChannel ColorChannel = EPCGTextureColorChannel::Red;

	UPROPERTY(EditAnywhere, Category = "Image Trace",
		meta = (ClampMin = 0, UIMin = 0, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Texture sampling resolution.\n\nHow densely the image is sampled during tracing. Higher = more detail, slower."))
	int32 SampleResolution = 1024;

    // Any plants with a bounds area lower than this value will be ignored.
    // Value is normalized meaning a value of 0.1 will filter out any plants whose bounds cover less than 10% of the image.
	UPROPERTY(EditAnywhere, Category = "Image Trace",
		meta = (ClampMin = 0, UIMin = 0, UIMax = 0.2, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Minimum region area to count as a plant region (fraction of image).\n\nFilters out small specks. 0.05 = ignore regions smaller than 5% of the image area. Lower to detect tiny plants in a sparse image; raise to focus only on the main shape."))
	float MinBoundsArea = 0.05f;

	UPROPERTY(EditAnywhere, Category = "Image Trace",
		meta = (ClampMin = 0, ClampMax = 100, UIMin = 0, UIMax = 10, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="How many passes of smoothing applied to the traced outline.\n\nHigher = smoother branch curves with less jagged outline noise. 0 = raw trace. Excessive smoothing may round away plant features."))
	int32 SmoothingIterations = 2;

	UPROPERTY(EditAnywhere, Category = "Image Trace",
		meta = (ClampMin = 0, ClampMax = 100, UIMin = 0, UIMax = 10.0, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="How much to simplify the traced skeleton.\n\nHigher = fewer skeleton points, simpler structure. 0 = full detail. 1.0 = light simplification (default). Higher = heavy simplification."))
	float SimplificationAmount = 1.0;

	UPROPERTY(EditAnywhere, Category = "Tip detection",
		meta = (ClampMin = 0.001, ClampMax = 180, UIMin = 0.01, UIMax = 180, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Minimum corner angle to detect a branch tip.\n\nSharper corners (smaller angles) are detected as branch tips. Lower values = more detected tips (more sensitive). Higher values = only sharp endpoints qualify."))
	float TipAngleThresholdInDegrees = 100.0;

	UPROPERTY(EditAnywhere, Category = "Tip detection",
		meta = (ClampMin = 0, ClampMax = 0.01, UIMin = 0, UIMax = 0.01, EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Search distance for confirming detected tips.\n\nHow far the algorithm looks from a candidate tip to confirm it. Larger values catch tips on smooth curves; smaller values are stricter."))
	float MaxTipAngleSearchDist = 0.005f;

	UPROPERTY(EditAnywhere, Category = "Plant Geneneration",
		meta = (EditCondition = "Texture2DAsset != nullptr", EditConditionHides, Tooltip="Per-plant placement settings (root position, rotation, scale)."))
	TArray<FPVImportTexture2DPlantSettings> PlantSettings;
};

USTRUCT(BlueprintType)
struct FPVImportTexture2DDebugParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "DebugSettings")
	EPVImportTexture2DDebugState DebugState = EPVImportTexture2DDebugState::None;
};

struct FPVImportTexture2DOutput
{
	TOptional<FManagedArrayCollection> DebugCollection;
	TArray<FManagedArrayCollection> PlantCollections;
};