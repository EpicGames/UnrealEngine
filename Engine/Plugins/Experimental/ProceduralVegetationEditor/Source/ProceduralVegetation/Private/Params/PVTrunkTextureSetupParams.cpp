// Copyright Epic Games, Inc. All Rights Reserved.

#include "Params/PVTrunkTextureSetupParams.h"

#include "ImageUtils.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Engine/Texture2D.h"

void FPVTextureChannelParams::LoadTexture()
{
	// Reject paths that contain characters invalid on the OS file system before
	// attempting any disk access; they will never resolve successfully.
	if (!FPaths::ValidatePath(TextureFilePath.FilePath))
	{
		ResetTexture();
		return;
	}
	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(* TextureFilePath.FilePath))
	{
		if (UTexture2D* OriginalTexture = FImageUtils::ImportFileAsTexture2D(* TextureFilePath.FilePath))
		{
			Texture = Cast<UTexture>(OriginalTexture);
			Texture->UpdateResource();

			if (ensure(Texture->GetSurfaceWidth() > 0))
			{
				WidthRatio = Texture->GetSurfaceHeight() / (float) Texture->GetSurfaceWidth();
			}
			else
			{
				ResetTexture();
			}
		}
		else
		{
			ResetTexture();
		}
	}
	else
	{
		ResetTexture();
	}
}

void FPVTextureChannelParams::ResetTexture()
{
	Texture = nullptr;
	WidthRatio = 1.0f;
}

FString FPVTextureChannelParams::GetChannelName() const
{
	if (EPVTextureChannel::Custom == Channel)
	{
		return ChannelName;
	}
	else
	{
		return  StaticEnum<EPVTextureChannel>()->GetNameStringByValue((int64)Channel);
	}
}

void FPVTextureGenerationParams::LoadChannelTexture(int ChannelIndex)
{
	if (Channels.IsValidIndex(ChannelIndex))
	{
		FPVTextureChannelParams& Channel = Channels[ChannelIndex];
		Channel.LoadTexture();
	}
}

TArray<FString> FPVTextureGenerationParams::GetChannelNames()
{
	TArray<FString> ChannelNames;
	for (const auto& Channel : Channels)
	{
		ChannelNames.Add(Channel.GetChannelName());
	}

	return ChannelNames;
}

const FPVTextureChannelParams* FPVTextureGenerationParams::GetChannel(const FString& Name)
{
	FPVTextureChannelParams* FoundChannel = nullptr;
	for (auto& Channel : Channels)
	{
		if (Channel.GetChannelName() == Name)
		{
			FoundChannel = &Channel;
		}
	}

	return FoundChannel;
}

float FPVTextureGenerationParams::GetWidthRatio() const
{
	if (Channels.IsValidIndex(0))
	{
		return Channels[0].WidthRatio;
	}

	return 1.0;
}

bool FPVTextureGenerationParams::IsWidthRatioSame() const
{
	if (!Channels.IsValidIndex(0))
	{
		return true;
	}
	
	float PreviousWidthRatio = Channels[0].WidthRatio;
	for (int Channel = 1; Channel < Channels.Num(); ++Channel)
	{
		if (!FMath::IsNearlyEqual(PreviousWidthRatio,Channels[Channel].WidthRatio))
		{
			return false;
		}
	}
	return true;
}

void FPVTrunkTextureSetupParams::LoadGenerationChannelTexture(int GenerationIndex, int ChannelIndex)
{
	if (Generations.IsValidIndex(GenerationIndex))
	{
		FPVTextureGenerationParams& Generation = Generations[GenerationIndex];
		Generation.LoadChannelTexture(ChannelIndex);
	}
}

float FPVTrunkTextureSetupParams::GetWidthRatio(FPVTextureGenerationParams Generation) const
{
	return Generation.GetWidthRatio();
}

bool FPVTrunkTextureSetupParams::IsWidthRatioSame(FPVTextureGenerationParams Generation) const
{
	return Generation.IsWidthRatioSame();
}
