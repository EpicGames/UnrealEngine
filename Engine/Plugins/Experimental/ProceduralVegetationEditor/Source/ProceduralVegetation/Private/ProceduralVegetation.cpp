// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralVegetation.h"

namespace PV
{
	const static FName DefaultGraphName = TEXT("ProceduralVegetationGraph");
};

void UProceduralVegetationGraph::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (HasAllFlags(RF_Transactional) == false)
	{
		SetFlags(RF_Transactional);
	}
#endif
}

void UProceduralVegetation::CreateGraph(const UProceduralVegetationGraph* InGraph /*= nullptr*/)
{
	if (InGraph)
	{
		Graph = DuplicateObject(InGraph, this);
		Graph->SetFlags(RF_Transactional);
	}
	else
	{
		Graph = NewObject<UProceduralVegetationGraph>(this, PV::DefaultGraphName, RF_Transactional);
	}
}

UProceduralVegetationInstance::UProceduralVegetationInstance(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	GraphInstance = InObjectInitializer.CreateDefaultSubobject<UProceduralVegetationGraphInstance>(this, TEXT("ProceduralVegetationGraphInstance"));
}

#if WITH_EDITOR
void UProceduralVegetationGraphInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UProceduralVegetationGraphInstance, ProceduralVegetation))
	{
		SetGraph(ProceduralVegetation ? ProceduralVegetation->GetGraph() : nullptr);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

void UProceduralVegetationInstance::PostLoad()
{
	Super::PostLoad();

	if (GraphInstance && GraphInstance->ProceduralVegetation)
	{
		GraphInstance->SetGraph(GraphInstance->ProceduralVegetation->GetGraph());
	}
}
