// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProceduralVegetationModule.h"

#include "ShaderCore.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY(LogProceduralVegetation);

void FProceduralVegetationModule::StartupModule()
{
	check(IPluginManager::Get().FindPlugin(TEXT("ProceduralVegetationEditor")));
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ProceduralVegetationEditor"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugins/Experimental/ProceduralVegetationEditor"), PluginShaderDir);
}

void FProceduralVegetationModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FProceduralVegetationModule, ProceduralVegetation)