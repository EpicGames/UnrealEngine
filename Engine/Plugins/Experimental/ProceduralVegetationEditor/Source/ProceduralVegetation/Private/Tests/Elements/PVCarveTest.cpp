// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVTestsCommon.h"
#include "DataTypes/PVGrowthData.h"
#include "Helpers/PVImportHelpers.h"
#include "Implementations/PVCarve.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"
#include "Nodes/PVCarveSettings.h"
#include "PCGContext.h"

namespace PVCarveTest
{
	TUniquePtr<FPCGContext> GenerateMockTreeDataAndRun(const FPVCarveParams& InCarveParams)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPVCarveSettings>(TestData);
		UPVCarveSettings* Settings = CastChecked<UPVCarveSettings>(TestData.Settings);
		Settings->CarveSettings = InCarveParams;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;

		UPVGrowthData* PVGrowthData = NewObject<UPVGrowthData>();
		PVGrowthData->Initialize(MoveTemp(*PVMockTreeCollection::CreateCollection()));
		Inputs.Data = PVGrowthData;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		return Context;
	}
};

#define PV_CARVE_TEST(TestName) PV_SIMPLE_AUTOMATION_TEST(Carve, TestName)

PV_CARVE_TEST(Default)
{
	FPVCarveParams CarveParams;
	CarveParams.Carve = 0;

	TUniquePtr<FPCGContext> Context = PVCarveTest::GenerateMockTreeDataAndRun(CarveParams);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPVGrowthData* OutPVGrowthData = Cast<UPVGrowthData>(Outputs[0].Data);
	UTEST_NOT_NULL("Output PVGrowthData", OutPVGrowthData);

	const FManagedArrayCollection& Collection = OutPVGrowthData->GetCollection();
	const PVMockTreeCollection::CompareResult Result = PVMockTreeCollection::Compare(Collection, true);
	if (Result != PVMockTreeCollection::CompareResult::Success)
	{
		AddError(FString::Printf(TEXT("Collection does not contain mock tree data: %s"), PVMockTreeCollection::CompareResultToString(Result)));
		return false;
	}

	return true;
}

PV_CARVE_TEST(DeepHierarchyCarve)
{
	// Regression test: CarveFromTop must remove the full subtree of a carved branch,
	// not just direct children and grandchildren (depth 2). Without the fix, points
	// belonging to branches at depth 3+ from the cut survive as floating artefacts.
	//
	// Hierarchy (4 levels):
	//   B0 (trunk):  P0(z=0) - P1(z=100) - P2(z=200) - P3(z=300) - P4(z=400)
	//   B1 (depth1): P2(z=200) - P5 - P6
	//   B2 (depth2): P6 - P7 - P8
	//   B3 (depth3): P8 - P9 - P10   <-- before fix: P9 and P10 survive as floating points
	//
	// Carving at 0.75 cuts the trunk at ~z=100, so B1 and all descendants must be removed.

	FPVBranchHierarchyDescription Hierarchy;
	Hierarchy.Points = {
		FVector3f(  0.f, 0.f,   0.f), // P0 - trunk base
		FVector3f(  0.f, 0.f, 100.f), // P1
		FVector3f(  0.f, 0.f, 200.f), // P2 - B1 junction
		FVector3f(  0.f, 0.f, 300.f), // P3
		FVector3f(  0.f, 0.f, 400.f), // P4 - trunk tip
		FVector3f( 10.f, 0.f, 200.f), // P5
		FVector3f( 20.f, 0.f, 200.f), // P6 - B2 junction
		FVector3f( 20.f, 0.f, 210.f), // P7
		FVector3f( 20.f, 0.f, 220.f), // P8 - B3 junction
		FVector3f( 20.f, 0.f, 230.f), // P9
		FVector3f( 20.f, 0.f, 240.f), // P10
	};
	Hierarchy.PointsRadii.Init(1.0f, Hierarchy.Points.Num());

	// B0 - trunk
	FPVBranchDescription& B0 = Hierarchy.Branches.AddDefaulted_GetRef();
	B0.BranchIndex = 0;
	B0.ParentBranchIndex = INDEX_NONE;
	B0.PointIndices = {0, 1, 2, 3, 4};
	B0.ChildBranchIndices = {1};

	// B1 - direct child of trunk (depth 1 from cut)
	FPVBranchDescription& B1 = Hierarchy.Branches.AddDefaulted_GetRef();
	B1.BranchIndex = 1;
	B1.ParentBranchIndex = 0;
	B1.PointIndices = {2, 5, 6};
	B1.ChildBranchIndices = {2};

	// B2 - grandchild (depth 2 from cut)
	FPVBranchDescription& B2 = Hierarchy.Branches.AddDefaulted_GetRef();
	B2.BranchIndex = 2;
	B2.ParentBranchIndex = 1;
	B2.PointIndices = {6, 7, 8};
	B2.ChildBranchIndices = {3};

	// B3 - great-grandchild (depth 3 from cut - the regression case)
	FPVBranchDescription& B3 = Hierarchy.Branches.AddDefaulted_GetRef();
	B3.BranchIndex = 3;
	B3.ParentBranchIndex = 2;
	B3.PointIndices = {8, 9, 10};
	B3.ChildBranchIndices = {};

	Hierarchy.RootBranchIndex = 0;

	FManagedArrayCollection SourceCollection;
	const bool bOk = PV::ImportHelper::GenerateGrowthDataFromBranchHierarchy(SourceCollection, Hierarchy);
	UTEST_TRUE("GenerateGrowthDataFromBranchHierarchy succeeded", bOk);

	// Carve 75% from top: cuts the trunk at ~z=100, so B1 and all descendants should be removed
	FManagedArrayCollection OutCollection = SourceCollection;
	FPVCarve::ApplyCarve(OutCollection, SourceCollection, ECarveBasis::LengthFromRoot, 0.75f);

	const PV::Facades::FPointFacade OutPointFacade(OutCollection);
	const PV::Facades::FBranchFacade OutBranchFacade(OutCollection);

	// All branches except the trunk must be removed
	UTEST_EQUAL("Only trunk branch remains", OutBranchFacade.GetElementCount(), 1);

	// No points at z >= 200 should survive: B1/B2/B3 all start at z=200 or higher.
	// Before the fix, B3 points at z=230 and z=240 would survive as floating artefacts.
	for (int32 i = 0; i < OutPointFacade.GetElementCount(); ++i)
	{
		const FVector3f& Pos = OutPointFacade.GetPosition(i);
		UTEST_TRUE(FString::Printf(TEXT("Floating deep point at z=%.1f was not removed"), Pos.Z), Pos.Z < 200.0f);
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS