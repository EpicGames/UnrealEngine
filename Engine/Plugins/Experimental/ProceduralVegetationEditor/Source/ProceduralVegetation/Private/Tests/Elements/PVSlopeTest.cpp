// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVTestsCommon.h"
#include "DataTypes/PVGrowthData.h"
#include "Facades/PVPointFacade.h"
#include "Implementations/PVSlope.h"
#include "Nodes/PVSlopeSettings.h"
#include "PCGContext.h"

namespace PVSlopeTest
{
	TUniquePtr<FPCGContext> GenerateMockTreeDataAndRun(const FPVSlopeParams& InSlopeParams)
	{
		PCGTestsCommon::FTestData TestData;
		PCGTestsCommon::GenerateSettings<UPVSlopeSettings>(TestData);
		UPVSlopeSettings* Settings = CastChecked<UPVSlopeSettings>(TestData.Settings);
		Settings->SlopeParams = InSlopeParams;

		FPCGTaggedData& Inputs = TestData.InputData.TaggedData.Emplace_GetRef();
		Inputs.Pin = PCGPinConstants::DefaultInputLabel;

		UPVGrowthData* PVGrowthData = NewObject<UPVGrowthData>();
		PVGrowthData->Initialize(MoveTemp(*PVMockTreeCollection::CreateCollection()));
		Inputs.Data = PVGrowthData;

		FPCGElementPtr TestElement = TestData.Settings->GetElement();

		TUniquePtr<FPCGContext> Context = TestData.InitializeTestContext();

		while (!TestElement->Execute(Context.Get())) {}

		return Context;
	}

	// Builds a mock tree collection translated by Offset so the trunk root sits at Offset
	// rather than the origin. Used to exercise the non-zero trunk pivot paths.
	TSharedRef<FManagedArrayCollection> CreateTranslatedMockTreeCollection(const FVector3f& Offset)
	{
		TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();

		PV::Facades::FPointFacade PointFacade(*Collection);
		const int32 PointCount = PointFacade.GetElementCount();
		for (int32 i = 0; i < PointCount; ++i)
		{
			PointFacade.SetPosition(i, PointFacade.GetPosition(i) + Offset);
		}

		return Collection;
	}
};

#define PV_SLOPE_TEST(TestName) PV_SIMPLE_AUTOMATION_TEST(Slope, TestName)

PV_SLOPE_TEST(Default)
{
	FPVSlopeParams SlopeParams;
	SlopeParams.BendStrength = 0;
	SlopeParams.SlopeAngle = 0;
	SlopeParams.SlopeDirection = 0;

	TUniquePtr<FPCGContext> Context = PVSlopeTest::GenerateMockTreeDataAndRun(SlopeParams);

	const TArray<FPCGTaggedData>& Outputs = Context->OutputData.TaggedData;
	UTEST_EQUAL("Output count", Outputs.Num(), 1);

	const UPVGrowthData* OutPVGrowthData = Cast<UPVGrowthData>(Outputs[0].Data);
	UTEST_NOT_NULL("Output PVGrowthData", OutPVGrowthData);

	const FManagedArrayCollection& Collection = OutPVGrowthData->GetCollection();
	const PVMockTreeCollection::CompareResult Result = PVMockTreeCollection::Compare(Collection, true);
	if (Result != PVMockTreeCollection::CompareResult::Success)
	{
		AddError(FString::Printf(TEXT("Collection does not contain mock tree data: %s"), PVMockTreeCollection::CompareResultToString(Result)));
		return false;
	}

	return true;
}

PV_SLOPE_TEST(Origin_NonZeroTrunk)
{
	// Trunk root not at origin: Origin pivot should rotate every point around (0,0,0) by the full slope rotation.
	const FVector3f Offset(50.f, 60.f, 70.f);

	FPVSlopeParams SlopeParams;
	SlopeParams.SlopeAngle = 90.f;
	SlopeParams.SlopeDirection = 0.f;
	SlopeParams.BendStrength = 0.f; // Pow(_, 0) == 1, so every point gets the full rotation R.
	SlopeParams.TrunkPivotPoint = EPVSlopeTrunkPivotPoint::Origin;

	TSharedRef<FManagedArrayCollection> Collection = PVSlopeTest::CreateTranslatedMockTreeCollection(Offset);
	FPVSlope::ApplySlope(SlopeParams, *Collection);

	// R = Quat(UpVector, 0) * Quat(RightVector, 90deg) = Quat(RightVector, 90deg)
	const FQuat4f R(FVector3f::RightVector, FMath::DegreesToRadians(90.f));

	const PV::Facades::FPointFacade PointFacade(*Collection);
	const FVector3f ActualRoot = PointFacade.GetPosition(0);
	const FVector3f ExpectedRoot = R.RotateVector(Offset); // PMockRoot was (0,0,0), so translated root = Offset.
	UTEST_TRUE("Trunk root rotated around origin", ActualRoot.Equals(ExpectedRoot, 1e-2f));

	// Spot-check a non-root point on the trunk.
	const FVector3f OriginalP1 = PVMockTree::Point1 + Offset;
	const FVector3f ActualP1 = PointFacade.GetPosition(1);
	const FVector3f ExpectedP1 = R.RotateVector(OriginalP1);
	UTEST_TRUE("Trunk second point rotated around origin", ActualP1.Equals(ExpectedP1, 1e-2f));

	return true;
}

PV_SLOPE_TEST(Trunk_NonZeroTrunk)
{
	// Trunk root not at origin: Trunk pivot should leave the trunk root in place and rotate everything around it.
	const FVector3f Offset(50.f, 60.f, 70.f);

	FPVSlopeParams SlopeParams;
	SlopeParams.SlopeAngle = 90.f;
	SlopeParams.SlopeDirection = 0.f;
	SlopeParams.BendStrength = 0.f;
	SlopeParams.TrunkPivotPoint = EPVSlopeTrunkPivotPoint::Trunk;

	TSharedRef<FManagedArrayCollection> Collection = PVSlopeTest::CreateTranslatedMockTreeCollection(Offset);
	FPVSlope::ApplySlope(SlopeParams, *Collection);

	const FQuat4f R(FVector3f::RightVector, FMath::DegreesToRadians(90.f));

	const PV::Facades::FPointFacade PointFacade(*Collection);
	const FVector3f TrunkPosition = Offset; // PVMockTree::Point0 is (0,0,0).
	const FVector3f ActualRoot = PointFacade.GetPosition(0);
	UTEST_TRUE("Trunk root unchanged with Trunk pivot", ActualRoot.Equals(TrunkPosition, 1e-2f));

	const FVector3f OriginalP1 = PVMockTree::Point1 + Offset;
	const FVector3f ActualP1 = PointFacade.GetPosition(1);
	const FVector3f ExpectedP1 = TrunkPosition + R.RotateVector(OriginalP1 - TrunkPosition);
	UTEST_TRUE("Trunk second point rotated around trunk root", ActualP1.Equals(ExpectedP1, 1e-2f));

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS