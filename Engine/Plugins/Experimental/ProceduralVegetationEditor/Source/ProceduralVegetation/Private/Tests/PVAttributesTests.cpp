// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVTestsCommon.h"
#include "Helpers/PVAttributesHelper.h"

#define PV_ATTRIBUTES_TEST(TestName) PV_SIMPLE_AUTOMATION_TEST(Attributes, TestName)

PV_ATTRIBUTES_TEST(ComputeLengthFromRoot)
{
	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();

	PV::FPointLengthFromRootAttributeConstView LengthFromRootAttribute = PV::FPointLengthFromRootAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", LengthFromRootAttribute.IsValid());
	UTEST_TRUE("Has 12 elements", LengthFromRootAttribute.Num() >= 12);

	UTEST_NEARLY_EQUAL("Point0_LengthFromRoot", LengthFromRootAttribute[0], PVMockTree::Point0_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point1_LengthFromRoot", LengthFromRootAttribute[1], PVMockTree::Point1_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point2_LengthFromRoot", LengthFromRootAttribute[2], PVMockTree::Point2_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point3_LengthFromRoot", LengthFromRootAttribute[3], PVMockTree::Point3_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point4_LengthFromRoot", LengthFromRootAttribute[4], PVMockTree::Point4_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point5_LengthFromRoot", LengthFromRootAttribute[5], PVMockTree::Point5_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point6_LengthFromRoot", LengthFromRootAttribute[6], PVMockTree::Point6_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point7_LengthFromRoot", LengthFromRootAttribute[7], PVMockTree::Point7_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point8_LengthFromRoot", LengthFromRootAttribute[8], PVMockTree::Point8_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point9_LengthFromRoot", LengthFromRootAttribute[9], PVMockTree::Point9_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point10_LengthFromRoot", LengthFromRootAttribute[10], PVMockTree::Point10_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point11_LengthFromRoot", LengthFromRootAttribute[11], PVMockTree::Point11_LengthFromRoot, 0.01f);

	const PV::FPointLengthFromSeedAttributeView LengthFromSeedAttribute = PV::FPointLengthFromSeedAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", LengthFromSeedAttribute.IsValid());
	UTEST_TRUE("Has 12 elements", LengthFromSeedAttribute.Num() >= 12);

	UTEST_NEARLY_EQUAL("Point0_LengthFromRoot", LengthFromSeedAttribute[0], PVMockTree::Point0_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point1_LengthFromRoot", LengthFromSeedAttribute[1], PVMockTree::Point1_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point2_LengthFromRoot", LengthFromSeedAttribute[2], PVMockTree::Point2_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point3_LengthFromRoot", LengthFromSeedAttribute[3], PVMockTree::Point3_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point4_LengthFromRoot", LengthFromSeedAttribute[4], PVMockTree::Point4_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point5_LengthFromRoot", LengthFromSeedAttribute[5], PVMockTree::Point5_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point6_LengthFromRoot", LengthFromSeedAttribute[6], PVMockTree::Point6_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point7_LengthFromRoot", LengthFromSeedAttribute[7], PVMockTree::Point7_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point8_LengthFromRoot", LengthFromSeedAttribute[8], PVMockTree::Point8_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point9_LengthFromRoot", LengthFromSeedAttribute[9], PVMockTree::Point9_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point10_LengthFromRoot", LengthFromSeedAttribute[10], PVMockTree::Point10_LengthFromRoot, 0.01f);
	UTEST_NEARLY_EQUAL("Point11_LengthFromRoot", LengthFromSeedAttribute[11], PVMockTree::Point11_LengthFromRoot, 0.01f);

	return true;
}

PV_ATTRIBUTES_TEST(ComputeBudDevelopment)
{
	struct FBudDevelopmentDataLayout
	{
		int32 Generation;
		int32 Age;
		int32 Branch_Age;
		int32 AgeSenescence;
		int32 Light_Senescence;
		int32 Relative_Age;

		bool operator==(const TArray<int32>& Other) const
		{
			return Other.Num() == 6
				&& Other[0] == Generation
				&& Other[1] == Age
				&& Other[2] == Branch_Age
				&& Other[3] == AgeSenescence
				&& Other[4] == Light_Senescence
				&& Other[5] == Relative_Age;
		}
	};

	const FBudDevelopmentDataLayout Point0_ExpectedBudDevelopment = {
		.Generation = 1,
		.Age = 6,
		.Branch_Age = 6,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 6
	};
	const FBudDevelopmentDataLayout Point1_ExpectedBudDevelopment = {
		.Generation = 1,
		.Age = 5,
		.Branch_Age = 6,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 5
	};
	const FBudDevelopmentDataLayout Point2_ExpectedBudDevelopment = {
		.Generation = 1,
		.Age = 4,
		.Branch_Age = 6,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 4
	};
	const FBudDevelopmentDataLayout Point3_ExpectedBudDevelopment = {
		.Generation = 1,
		.Age = 3,
		.Branch_Age = 6,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 3
	};
	const FBudDevelopmentDataLayout Point4_ExpectedBudDevelopment = {
		.Generation = 1,
		.Age = 2,
		.Branch_Age = 6,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 2
	};
	const FBudDevelopmentDataLayout Point5_ExpectedBudDevelopment = {
		.Generation = 1,
		.Age = 1,
		.Branch_Age = 6,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 1
	};
	const FBudDevelopmentDataLayout Point6_ExpectedBudDevelopment = {
		.Generation = 2,
		.Age = 1,
		.Branch_Age = 1,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 1
	};
	const FBudDevelopmentDataLayout Point7_ExpectedBudDevelopment = {
		.Generation = 2,
		.Age = 3,
		.Branch_Age = 3,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 3
	};
	const FBudDevelopmentDataLayout Point8_ExpectedBudDevelopment = {
		.Generation = 2,
		.Age = 2,
		.Branch_Age = 3,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 2
	};
	const FBudDevelopmentDataLayout Point9_ExpectedBudDevelopment = {
		.Generation = 2,
		.Age = 1,
		.Branch_Age = 3,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 1
	};
	const FBudDevelopmentDataLayout Point10_ExpectedBudDevelopment = {
		.Generation = 3,
		.Age = 2,
		.Branch_Age = 2,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 2
	};
	const FBudDevelopmentDataLayout Point11_ExpectedBudDevelopment = {
		.Generation = 3,
		.Age = 1,
		.Branch_Age = 2,
		.AgeSenescence = 0,
		.Light_Senescence = 0,
		.Relative_Age = 1
	};

	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();
	const PV::FBudDevelopmentAttributeConstView BudDevelopmentAttribute = PV::FBudDevelopmentAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", BudDevelopmentAttribute);
	UTEST_TRUE("Has 12 elements", BudDevelopmentAttribute.Num() >= 12);

	UTEST_TRUE("Point0_BudDevelopment",  BudDevelopmentAttribute[0].Array  == Point0_ExpectedBudDevelopment);
	UTEST_TRUE("Point1_BudDevelopment",  BudDevelopmentAttribute[1].Array  == Point1_ExpectedBudDevelopment);
	UTEST_TRUE("Point2_BudDevelopment",  BudDevelopmentAttribute[2].Array  == Point2_ExpectedBudDevelopment);
	UTEST_TRUE("Point3_BudDevelopment",  BudDevelopmentAttribute[3].Array  == Point3_ExpectedBudDevelopment);
	UTEST_TRUE("Point4_BudDevelopment",  BudDevelopmentAttribute[4].Array  == Point4_ExpectedBudDevelopment);
	UTEST_TRUE("Point5_BudDevelopment",  BudDevelopmentAttribute[5].Array  == Point5_ExpectedBudDevelopment);
	UTEST_TRUE("Point6_BudDevelopment",  BudDevelopmentAttribute[6].Array  == Point6_ExpectedBudDevelopment);
	UTEST_TRUE("Point7_BudDevelopment",  BudDevelopmentAttribute[7].Array  == Point7_ExpectedBudDevelopment);
	UTEST_TRUE("Point8_BudDevelopment",  BudDevelopmentAttribute[8].Array  == Point8_ExpectedBudDevelopment);
	UTEST_TRUE("Point9_BudDevelopment",  BudDevelopmentAttribute[9].Array  == Point9_ExpectedBudDevelopment);
	UTEST_TRUE("Point10_BudDevelopment", BudDevelopmentAttribute[10].Array == Point10_ExpectedBudDevelopment);
	UTEST_TRUE("Point11_BudDevelopment", BudDevelopmentAttribute[11].Array == Point11_ExpectedBudDevelopment);

	return true;
}

PV_ATTRIBUTES_TEST(ComputeGradients)
{
	const float Point0_ExpectedPlantGradient  = 1;
	const float Point1_ExpectedPlantGradient  = 0.8f;
	const float Point2_ExpectedPlantGradient  = 0.6f;
	const float Point3_ExpectedPlantGradient  = 0.4f;
	const float Point4_ExpectedPlantGradient  = 0.2f;
	const float Point5_ExpectedPlantGradient  = 0;
	const float Point6_ExpectedPlantGradient  = 0;
	const float Point7_ExpectedPlantGradient  = FMath::Lerp(Point2_ExpectedPlantGradient, 0.f, 1.f / 3);
	const float Point8_ExpectedPlantGradient  = FMath::Lerp(Point2_ExpectedPlantGradient, 0.f, 2.f / 3);
	const float Point9_ExpectedPlantGradient  = 0;
	const float Point10_ExpectedPlantGradient = FMath::Lerp(Point8_ExpectedPlantGradient, 0.f, 1.f / 2);
	const float Point11_ExpectedPlantGradient = 0;

	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();
	
	const PV::FPointPlantGradientAttributeView PointPlantGradientAttribute = PV::FPointPlantGradientAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", PointPlantGradientAttribute.IsValid());
	UTEST_TRUE("Has 12 elements", PointPlantGradientAttribute.Num() >= 12);

	UTEST_TRUE("Point0_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[0],  Point0_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point1_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[1],  Point1_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point2_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[2],  Point2_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point3_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[3],  Point3_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point4_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[4],  Point4_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point5_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[5],  Point5_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point6_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[6],  Point6_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point7_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[7],  Point7_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point8_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[8],  Point8_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point9_ExpectedPlantGradient",  FMath::IsNearlyEqual(PointPlantGradientAttribute[9],  Point9_ExpectedPlantGradient,  0.01f));
	UTEST_TRUE("Point10_ExpectedPlantGradient", FMath::IsNearlyEqual(PointPlantGradientAttribute[10], Point10_ExpectedPlantGradient, 0.01f));
	UTEST_TRUE("Point11_ExpectedPlantGradient", FMath::IsNearlyEqual(PointPlantGradientAttribute[11], Point11_ExpectedPlantGradient, 0.01f));

	return true;
}

PV_ATTRIBUTES_TEST(ComputeBudStatus)
{
	struct FBudStatusDataLayout {
		int32 ApicalMeristem;
		int32 Codominant;
		int32 Axillary;
		int32 Seed;
		int32 Dormant;
		int32 Triggered;
		int32 NumTriggered;
		int32 Inactive;
		int32 BrokenTip;
		int32 Broken;

		bool operator==(const TArray<int32>& Other) const
		{
			return Other.Num() == 10
				&& ApicalMeristem == Other[0]
				&& Codominant == Other[1]
				&& Axillary == Other[2]
				&& Seed == Other[3]
				&& Dormant == Other[4]
				&& Triggered == Other[5]
				&& NumTriggered == Other[6]
				&& Inactive == Other[7]
				&& BrokenTip == Other[8]
				&& Broken == Other[9];
		}
	};

	const FBudStatusDataLayout Point0_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 1,
		.Seed = 1,
		.Dormant = 1,
		.Triggered = 0,
		.NumTriggered = 0,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point1_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point2_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 1,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point3_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 1,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point4_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point5_ExpectedBudStatus = {
		.ApicalMeristem = 1,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 1,
		.Triggered = 0,
		.NumTriggered = 0,
		.Inactive = 0,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point6_ExpectedBudStatus = {
		.ApicalMeristem = 1,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 1,
		.Triggered = 0,
		.NumTriggered = 0,
		.Inactive = 0,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point7_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point8_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 1,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point9_ExpectedBudStatus = {
		.ApicalMeristem = 1,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 1,
		.Triggered = 0,
		.NumTriggered = 0,
		.Inactive = 0,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point10_ExpectedBudStatus = {
		.ApicalMeristem = 0,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 0,
		.Triggered = 1,
		.NumTriggered = 1,
		.Inactive = 1,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const FBudStatusDataLayout Point11_ExpectedBudStatus = {
		.ApicalMeristem = 1,
		.Codominant = 0,
		.Axillary = 0,
		.Seed = 0,
		.Dormant = 1,
		.Triggered = 0,
		.NumTriggered = 0,
		.Inactive = 0,
		.BrokenTip = 0,
		.Broken = 0,
	};

	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();

	const PV::FBudStatusAttributeConstView BudStatusAttribute = PV::FBudStatusAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", BudStatusAttribute);
	UTEST_TRUE("Has 12 elements", BudStatusAttribute.Num() >= 12);

	UTEST_TRUE("Point0_BudStatus",  BudStatusAttribute[0].Array  == Point0_ExpectedBudStatus);
	UTEST_TRUE("Point1_BudStatus",  BudStatusAttribute[1].Array  == Point1_ExpectedBudStatus);
	UTEST_TRUE("Point2_BudStatus",  BudStatusAttribute[2].Array  == Point2_ExpectedBudStatus);
	UTEST_TRUE("Point3_BudStatus",  BudStatusAttribute[3].Array  == Point3_ExpectedBudStatus);
	UTEST_TRUE("Point4_BudStatus",  BudStatusAttribute[4].Array  == Point4_ExpectedBudStatus);
	UTEST_TRUE("Point5_BudStatus",  BudStatusAttribute[5].Array  == Point5_ExpectedBudStatus);
	UTEST_TRUE("Point6_BudStatus",  BudStatusAttribute[6].Array  == Point6_ExpectedBudStatus);
	UTEST_TRUE("Point7_BudStatus",  BudStatusAttribute[7].Array  == Point7_ExpectedBudStatus);
	UTEST_TRUE("Point8_BudStatus",  BudStatusAttribute[8].Array  == Point8_ExpectedBudStatus);
	UTEST_TRUE("Point9_BudStatus",  BudStatusAttribute[9].Array  == Point9_ExpectedBudStatus);
	UTEST_TRUE("Point10_BudStatus", BudStatusAttribute[10].Array == Point10_ExpectedBudStatus);
	UTEST_TRUE("Point11_BudStatus", BudStatusAttribute[11].Array == Point11_ExpectedBudStatus);

	return true;
}

PV_ATTRIBUTES_TEST(ComputeBudHormoneLevels)
{
	struct FBudHormoneLevelsDataLayout
	{
		float Apical;
		float Axillary;
		float AxillaryInhibition;
		float Radical;
		float Ethylene;
		float Cytokinin;

		bool operator==(const TArray<float>& Other) const
		{
			return Other.Num() == 6
				&& FMath::IsNearlyEqual(Other[0], Apical, 0.01f)
				&& FMath::IsNearlyEqual(Other[1], Axillary, 0.01f)
				&& FMath::IsNearlyEqual(Other[2], AxillaryInhibition, 0.01f)
				&& FMath::IsNearlyEqual(Other[3], Radical, 0.01f)
				&& FMath::IsNearlyEqual(Other[4], Ethylene, 0.01f)
				&& FMath::IsNearlyEqual(Other[5], Cytokinin, 0.01f);
		}
	};

	// Apical is always 1 for all points.
	// Axillary = GetMappedRangeValueClamped([0,1] -> [0.35, 1.0], InverseGradient), where InverseGradient = 1 - PlantGradient.
	// Ethylene = PlantGradient.
	const FBudHormoneLevelsDataLayout Point0_ExpectedBudHormoneLevels = {
		.Apical = 1,
		.Axillary = 0.35f,  // Gradient=1, InverseGradient=0 => mapped to 0.35
		.AxillaryInhibition = 0,
		.Radical = 0,
		.Ethylene = 1,
		.Cytokinin = 0,
	};
	const FBudHormoneLevelsDataLayout Point5_ExpectedBudHormoneLevels = {
		.Apical = 1,
		.Axillary = 1.0f,   // Gradient=0, InverseGradient=1 => mapped to 1.0
		.AxillaryInhibition = 0,
		.Radical = 0,
		.Ethylene = 0,
		.Cytokinin = 0,
	};
	const FBudHormoneLevelsDataLayout Point6_ExpectedBudHormoneLevels = {
		.Apical = 1,
		.Axillary = 1.0f,   // Gradient=0, InverseGradient=1 => mapped to 1.0
		.AxillaryInhibition = 0,
		.Radical = 0,
		.Ethylene = 0,
		.Cytokinin = 0,
	};
	const FBudHormoneLevelsDataLayout Point9_ExpectedBudHormoneLevels = {
		.Apical = 1,
		.Axillary = 1.0f,   // Gradient=0, InverseGradient=1 => mapped to 1.0
		.AxillaryInhibition = 0,
		.Radical = 0,
		.Ethylene = 0,
		.Cytokinin = 0,
	};
	const FBudHormoneLevelsDataLayout Point11_ExpectedBudHormoneLevels = {
		.Apical = 1,
		.Axillary = 1.0f,   // Gradient=0, InverseGradient=1 => mapped to 1.0
		.AxillaryInhibition = 0,
		.Radical = 0,
		.Ethylene = 0,
		.Cytokinin = 0,
	};

	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();

	const PV::FBudHormoneLevelsAttributeConstView BudHormoneLevelsAttribute = PV::FBudHormoneLevelsAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", BudHormoneLevelsAttribute.IsValid());
	UTEST_TRUE("Has 12 elements", BudHormoneLevelsAttribute.Num() >= 12);

	UTEST_TRUE("Point0_ExpectedBudHormoneLevels",  BudHormoneLevelsAttribute[0].Array  == Point0_ExpectedBudHormoneLevels);
	UTEST_TRUE("Point5_ExpectedBudHormoneLevels",  BudHormoneLevelsAttribute[5].Array  == Point5_ExpectedBudHormoneLevels);
	UTEST_TRUE("Point6_ExpectedBudHormoneLevels",  BudHormoneLevelsAttribute[6].Array  == Point6_ExpectedBudHormoneLevels);
	UTEST_TRUE("Point9_ExpectedBudHormoneLevels",  BudHormoneLevelsAttribute[9].Array  == Point9_ExpectedBudHormoneLevels);
	UTEST_TRUE("Point11_ExpectedBudHormoneLevels", BudHormoneLevelsAttribute[11].Array == Point11_ExpectedBudHormoneLevels);

	return true;
}

PV_ATTRIBUTES_TEST(ComputeBudLateralMeristem)
{
	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();
	
	const PV::FBudLateralMeristemAttributeConstView BudLateralMeristemAttribute = PV::FBudLateralMeristemAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", BudLateralMeristemAttribute.IsValid());
	UTEST_TRUE("Has 12 elements", BudLateralMeristemAttribute.Num() >= 12);

	// All points receive a non-zero Davinci value because EstimatedLateralElongation (> 0) is
	// accumulated starting from each branch's tip, so even tip points are non-zero.
	UTEST_TRUE("Point0_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[0].Davinci));
	UTEST_TRUE("Point1_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[1].Davinci));
	UTEST_TRUE("Point2_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[2].Davinci));
	UTEST_TRUE("Point3_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[3].Davinci));
	UTEST_TRUE("Point4_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[4].Davinci));
	UTEST_TRUE("Point5_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[5].Davinci));
	UTEST_TRUE("Point6_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[6].Davinci));
	UTEST_TRUE("Point7_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[7].Davinci));
	UTEST_TRUE("Point8_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[8].Davinci));
	UTEST_TRUE("Point9_DavinciIsNotZero",  !FMath::IsNearlyZero(BudLateralMeristemAttribute[9].Davinci));
	UTEST_TRUE("Point10_DavinciIsNotZero", !FMath::IsNearlyZero(BudLateralMeristemAttribute[10].Davinci));
	UTEST_TRUE("Point11_DavinciIsNotZero", !FMath::IsNearlyZero(BudLateralMeristemAttribute[11].Davinci));

	return true;
}

PV_ATTRIBUTES_TEST(ComputeBudDirections)
{
	// Apical direction rule: first point on a branch looks forward (Next - Current),
	// all other points look backward along their incoming segment (Current - Prev).
	// Points shared at branch junctions are split, so each branch start uses its own copy.
	// Branch 0 [0,1,2,3,4,5], Branch 1 [copy(2),7,8,9], Branch 2 [copy(8),10,11], Branch 3 [copy(3),6]
	const FVector3f ExpectedApical0  = (PVMockTree::Point1  - PVMockTree::Point0).GetSafeNormal(); // Branch0 BPI=0: Next-Current
	const FVector3f ExpectedApical1  = (PVMockTree::Point1  - PVMockTree::Point0).GetSafeNormal(); // Branch0 BPI=1: Current-Prev
	const FVector3f ExpectedApical2  = (PVMockTree::Point2  - PVMockTree::Point1).GetSafeNormal(); // Branch0 BPI=2: Current-Prev
	const FVector3f ExpectedApical3  = (PVMockTree::Point3  - PVMockTree::Point2).GetSafeNormal(); // Branch0 BPI=3: Current-Prev
	const FVector3f ExpectedApical4  = (PVMockTree::Point4  - PVMockTree::Point3).GetSafeNormal(); // Branch0 BPI=4: Current-Prev
	const FVector3f ExpectedApical5  = (PVMockTree::Point5  - PVMockTree::Point4).GetSafeNormal(); // Branch0 BPI=5: Current-Prev
	const FVector3f ExpectedApical6  = (PVMockTree::Point6  - PVMockTree::Point3).GetSafeNormal(); // Branch3 BPI=1: Current-Prev (prev is split copy of Point3)
	const FVector3f ExpectedApical7  = (PVMockTree::Point7  - PVMockTree::Point2).GetSafeNormal(); // Branch1 BPI=1: Current-Prev (prev is split copy of Point2)
	const FVector3f ExpectedApical8  = (PVMockTree::Point8  - PVMockTree::Point7).GetSafeNormal(); // Branch1 BPI=2: Current-Prev
	const FVector3f ExpectedApical9  = (PVMockTree::Point9  - PVMockTree::Point8).GetSafeNormal(); // Branch1 BPI=3: Current-Prev
	const FVector3f ExpectedApical10 = (PVMockTree::Point10 - PVMockTree::Point8).GetSafeNormal(); // Branch2 BPI=1: Current-Prev (prev is split copy of Point8)
	const FVector3f ExpectedApical11 = (PVMockTree::Point11 - PVMockTree::Point10).GetSafeNormal(); // Branch2 BPI=2: Current-Prev

	const int32 ExpectedNumBudDirectionValues = 6;

	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();

	const PV::FBudDirectionAttributeConstView BudDirectionAttribute = PV::FBudDirectionAttribute::FindAttribute(*Collection);
	UTEST_TRUE("HasAttribute", BudDirectionAttribute.IsValid());
	UTEST_TRUE("Has 12 elements", BudDirectionAttribute.Num() >= 12);

	UTEST_NEARLY_EQUAL("Point0_Apical", FVector(BudDirectionAttribute[0].Apical), FVector(ExpectedApical0), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point1_Apical", FVector(BudDirectionAttribute[1].Apical), FVector(ExpectedApical1), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point2_Apical", FVector(BudDirectionAttribute[2].Apical), FVector(ExpectedApical2), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point3_Apical", FVector(BudDirectionAttribute[3].Apical), FVector(ExpectedApical3), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point4_Apical", FVector(BudDirectionAttribute[4].Apical), FVector(ExpectedApical4), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point5_Apical", FVector(BudDirectionAttribute[5].Apical), FVector(ExpectedApical5), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point6_Apical", FVector(BudDirectionAttribute[6].Apical), FVector(ExpectedApical6), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point7_Apical", FVector(BudDirectionAttribute[7].Apical), FVector(ExpectedApical7), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point8_Apical", FVector(BudDirectionAttribute[8].Apical), FVector(ExpectedApical8), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point9_Apical", FVector(BudDirectionAttribute[9].Apical), FVector(ExpectedApical9), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point10_Apical", FVector(BudDirectionAttribute[10].Apical), FVector(ExpectedApical10), UE_KINDA_SMALL_NUMBER);
	UTEST_NEARLY_EQUAL("Point11_Apical", FVector(BudDirectionAttribute[11].Apical), FVector(ExpectedApical11), UE_KINDA_SMALL_NUMBER);

	// TODO: Test UpVector and Axillary

	return true;
}

PV_ATTRIBUTES_TEST(ValidateAttributeCollection)
{
	// No attributes → always valid
	UTEST_TRUE("NoAttributes", PV::ValidateAttributeCollection());

	// Single valid attribute → valid
	{
		FManagedArrayCollection Collection;
		Collection.AddGroup(PV::GroupNames::PointGroup);
		Collection.AddElements(5, PV::GroupNames::PointGroup);
		PV::FPointLengthFromRootAttribute::AddAttribute(Collection);

		const PV::FPointLengthFromRootAttributeConstView View = PV::FPointLengthFromRootAttribute::FindAttribute(Collection);
		UTEST_TRUE("SingleValid", PV::ValidateAttributeCollection(View));
	}

	// Single invalid (null) attribute → invalid
	{
		const PV::FPointLengthFromRootAttributeConstView InvalidView;
		UTEST_FALSE("SingleInvalid", PV::ValidateAttributeCollection(InvalidView));
	}

	// Two valid attributes in the same group with equal sizes → valid
	{
		FManagedArrayCollection Collection;
		Collection.AddGroup(PV::GroupNames::PointGroup);
		Collection.AddElements(5, PV::GroupNames::PointGroup);
		PV::FPointLengthFromRootAttribute::AddAttribute(Collection);
		PV::FPointScaleAttribute::AddAttribute(Collection);

		const PV::FPointLengthFromRootAttributeConstView LengthView = PV::FPointLengthFromRootAttribute::FindAttribute(Collection);
		const PV::FPointScaleAttributeConstView ScaleView = PV::FPointScaleAttribute::FindAttribute(Collection);
		UTEST_TRUE("SameGroupSameSize", PV::ValidateAttributeCollection(LengthView, ScaleView));
	}

	// Two valid attributes in the same group but from collections with different sizes → invalid
	{
		FManagedArrayCollection Collection5;
		Collection5.AddGroup(PV::GroupNames::PointGroup);
		Collection5.AddElements(5, PV::GroupNames::PointGroup);
		PV::FPointLengthFromRootAttribute::AddAttribute(Collection5);

		FManagedArrayCollection Collection10;
		Collection10.AddGroup(PV::GroupNames::PointGroup);
		Collection10.AddElements(10, PV::GroupNames::PointGroup);
		PV::FPointScaleAttribute::AddAttribute(Collection10);

		const PV::FPointLengthFromRootAttributeConstView LengthView = PV::FPointLengthFromRootAttribute::FindAttribute(Collection5);
		const PV::FPointScaleAttributeConstView ScaleView = PV::FPointScaleAttribute::FindAttribute(Collection10);
		UTEST_FALSE("SameGroupDifferentSizes", PV::ValidateAttributeCollection(LengthView, ScaleView));
	}

	// Valid attributes in different groups → sizes between groups are independent, so this is valid
	{
		FManagedArrayCollection Collection;
		Collection.AddGroup(PV::GroupNames::PointGroup);
		Collection.AddElements(5, PV::GroupNames::PointGroup);
		Collection.AddGroup(PV::GroupNames::BranchGroup);
		Collection.AddElements(3, PV::GroupNames::BranchGroup);
		PV::FPointLengthFromRootAttribute::AddAttribute(Collection);
		PV::FBranchNumberAttribute::AddAttribute(Collection);

		const PV::FPointLengthFromRootAttributeConstView LengthView = PV::FPointLengthFromRootAttribute::FindAttribute(Collection);
		const PV::FBranchNumberAttributeConstView BranchNumberView = PV::FBranchNumberAttribute::FindAttribute(Collection);
		UTEST_TRUE("DifferentGroupsDifferentSizes", PV::ValidateAttributeCollection(LengthView, BranchNumberView));
	}

	// One invalid attribute mixed with valid ones → invalid
	{
		FManagedArrayCollection Collection;
		Collection.AddGroup(PV::GroupNames::PointGroup);
		Collection.AddElements(5, PV::GroupNames::PointGroup);
		PV::FPointLengthFromRootAttribute::AddAttribute(Collection);
		PV::FPointScaleAttribute::AddAttribute(Collection);

		const PV::FPointLengthFromRootAttributeConstView LengthView = PV::FPointLengthFromRootAttribute::FindAttribute(Collection);
		const PV::FPointScaleAttributeConstView ScaleView = PV::FPointScaleAttribute::FindAttribute(Collection);
		const PV::FPointPositionAttributeConstView InvalidPositionView;
		UTEST_FALSE("OneInvalidAmongValid", PV::ValidateAttributeCollection(LengthView, InvalidPositionView, ScaleView));
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS