// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVTestsCommon.h"
#include "Helpers/PVImportHelpers.h"
#include "Facades/PVAttributesNames.h"

#define PV_IMPORTER_TEST(TestName) PV_SIMPLE_AUTOMATION_TEST(Importer, TestName)

namespace PVMockTree
{
	FPVBranchHierarchyDescription CreateBranchHierarchyDescription()
	{
		return PV::ImportHelper::CreateBranchHierarchyFromPoints(PVMockTree::Points, PVMockTree::PointsRadii, PVMockTree::BranchPointIndices);
	}
}

PV_IMPORTER_TEST(FillAttributesFromBranchHierarchy)
{
	const FPVBranchHierarchyDescription BranchHierarchy = PVMockTree::CreateBranchHierarchyDescription();

	const int32 NumPoints = BranchHierarchy.GetNumPoints();
	const int32 NumBranches = BranchHierarchy.GetNumBranches();

	FManagedArrayCollection ExpectedCollection;
	PV::ImportHelper::CreateEmptyGrowthData(ExpectedCollection, NumPoints, NumBranches);

	auto ExpectedPointPositionAttribute = PV::FPointPositionAttribute::GetAttribute(ExpectedCollection);
	auto ExpectedBranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(ExpectedCollection);
	auto ExpectedBranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(ExpectedCollection);
	auto ExpectedBranchParentsAttribute = PV::FBranchParentsAttribute::GetAttribute(ExpectedCollection);
	auto ExpectedBranchChildrenAttribute = PV::FBranchChildrenAttribute::GetAttribute(ExpectedCollection);

	// Init Expected Point Attributes
	{
		ExpectedPointPositionAttribute[0] = PVMockTree::Point0;
		ExpectedPointPositionAttribute[1] = PVMockTree::Point1;
		ExpectedPointPositionAttribute[2] = PVMockTree::Point2;
		ExpectedPointPositionAttribute[3] = PVMockTree::Point3;
		ExpectedPointPositionAttribute[4] = PVMockTree::Point4;
		ExpectedPointPositionAttribute[5] = PVMockTree::Point5;
		ExpectedPointPositionAttribute[6] = PVMockTree::Point6;
		ExpectedPointPositionAttribute[7] = PVMockTree::Point7;
		ExpectedPointPositionAttribute[8] = PVMockTree::Point8;
		ExpectedPointPositionAttribute[9] = PVMockTree::Point9;
		ExpectedPointPositionAttribute[10] = PVMockTree::Point10;
		ExpectedPointPositionAttribute[11] = PVMockTree::Point11;
	}

	// Init Expected Branch attributes
	{
		ExpectedBranchPointsAttribute[0] = PVMockTree::Branch0_PointIndices;
		ExpectedBranchPointsAttribute[1] = PVMockTree::Branch1_PointIndices;
		ExpectedBranchPointsAttribute[2] = PVMockTree::Branch2_PointIndices;
		ExpectedBranchPointsAttribute[3] = PVMockTree::Branch3_PointIndices;

		ExpectedBranchParentNumberAttribute[0] = 0;
		ExpectedBranchParentNumberAttribute[1] = PVMockTree::Branch0_BranchNumber;
		ExpectedBranchParentNumberAttribute[2] = PVMockTree::Branch1_BranchNumber;
		ExpectedBranchParentNumberAttribute[3] = PVMockTree::Branch0_BranchNumber;

		ExpectedBranchParentsAttribute[0] = { 0 }; // Root branch always has a branch parent number of "0" (0 is an invalid branch parent number in this case)
		ExpectedBranchParentsAttribute[1] = { 0, PVMockTree::Branch0_BranchNumber };
		ExpectedBranchParentsAttribute[2] = { 0, PVMockTree::Branch0_BranchNumber, PVMockTree::Branch1_BranchNumber };
		ExpectedBranchParentsAttribute[3] = { 0, PVMockTree::Branch0_BranchNumber };

		ExpectedBranchChildrenAttribute[0] = { PVMockTree::Branch1_BranchNumber, PVMockTree::Branch2_BranchNumber, PVMockTree::Branch3_BranchNumber };
		ExpectedBranchChildrenAttribute[1] = { PVMockTree::Branch2_BranchNumber };
		ExpectedBranchChildrenAttribute[2] = { };
		ExpectedBranchChildrenAttribute[3] = { };
	}

	FManagedArrayCollection ActualCollection;
	PV::ImportHelper::CreateEmptyGrowthData(ActualCollection, NumPoints, NumBranches);

	UTEST_EQUAL("CorrectNrOfPointsInGroup", ActualCollection.NumElements(PV::GroupNames::PointGroup), NumPoints);
	UTEST_EQUAL("CorrectNrOfBranchesInGroup", ActualCollection.NumElements(PV::GroupNames::BranchGroup), NumBranches);

	PV::ImportHelper::FillAttributesFromBranchHierarchy(ActualCollection, { BranchHierarchy });

	const auto& ActualPointPositionAttribute = PV::FPointPositionAttribute::GetAttribute(ActualCollection);
	const auto& ActualBranchPointsAttribute = PV::FBranchPointsAttribute::GetAttribute(ActualCollection);
	const auto& ActualBranchParentNumberAttribute = PV::FBranchParentNumberAttribute::GetAttribute(ActualCollection);
	const auto& ActualBranchParentsAttribute = PV::FBranchParentsAttribute::GetAttribute(ActualCollection);
	const auto& ActualBranchChildrenAttribute = PV::FBranchChildrenAttribute::GetAttribute(ActualCollection);

	// Validate that the points are correctly indexed and in the correct location
	UTEST_TRUE("PointPosition0",  ActualPointPositionAttribute[0].Equals(ExpectedPointPositionAttribute[0]));
	UTEST_TRUE("PointPosition1",  ActualPointPositionAttribute[1].Equals(ExpectedPointPositionAttribute[1]));
	UTEST_TRUE("PointPosition2",  ActualPointPositionAttribute[2].Equals(ExpectedPointPositionAttribute[2]));
	UTEST_TRUE("PointPosition3",  ActualPointPositionAttribute[3].Equals(ExpectedPointPositionAttribute[3]));
	UTEST_TRUE("PointPosition4",  ActualPointPositionAttribute[4].Equals(ExpectedPointPositionAttribute[4]));
	UTEST_TRUE("PointPosition5",  ActualPointPositionAttribute[5].Equals(ExpectedPointPositionAttribute[5]));
	UTEST_TRUE("PointPosition6",  ActualPointPositionAttribute[6].Equals(ExpectedPointPositionAttribute[6]));
	UTEST_TRUE("PointPosition7",  ActualPointPositionAttribute[7].Equals(ExpectedPointPositionAttribute[7]));
	UTEST_TRUE("PointPosition8",  ActualPointPositionAttribute[8].Equals(ExpectedPointPositionAttribute[8]));
	UTEST_TRUE("PointPosition9",  ActualPointPositionAttribute[9].Equals(ExpectedPointPositionAttribute[9]));
	UTEST_TRUE("PointPosition10", ActualPointPositionAttribute[10].Equals(ExpectedPointPositionAttribute[10]));
	UTEST_TRUE("PointPosition11", ActualPointPositionAttribute[11].Equals(ExpectedPointPositionAttribute[11]));

	// Validate that the branch point indices are correct
	UTEST_EQUAL("BranchPoints0", ActualBranchPointsAttribute[0], ExpectedBranchPointsAttribute[0]);
	UTEST_EQUAL("BranchPoints1", ActualBranchPointsAttribute[1], ExpectedBranchPointsAttribute[1]);
	UTEST_EQUAL("BranchPoints2", ActualBranchPointsAttribute[2], ExpectedBranchPointsAttribute[2]);
	UTEST_EQUAL("BranchPoints3", ActualBranchPointsAttribute[3], ExpectedBranchPointsAttribute[3]);

	// Validate that the branch parents are correct
	UTEST_EQUAL("ParentBranchNumber0", ActualBranchParentNumberAttribute[0], ExpectedBranchParentNumberAttribute[0]);
	UTEST_EQUAL("ParentBranchNumber1", ActualBranchParentNumberAttribute[1], ExpectedBranchParentNumberAttribute[1]);
	UTEST_EQUAL("ParentBranchNumber2", ActualBranchParentNumberAttribute[2], ExpectedBranchParentNumberAttribute[2]);
	UTEST_EQUAL("ParentBranchNumber3", ActualBranchParentNumberAttribute[3], ExpectedBranchParentNumberAttribute[3]);
	UTEST_EQUAL("BranchParents0", ActualBranchParentsAttribute[0], ExpectedBranchParentsAttribute[0]);
	UTEST_EQUAL("BranchParents1", ActualBranchParentsAttribute[1], ExpectedBranchParentsAttribute[1]);
	UTEST_EQUAL("BranchParents2", ActualBranchParentsAttribute[2], ExpectedBranchParentsAttribute[2]);
	UTEST_EQUAL("BranchParents3", ActualBranchParentsAttribute[3], ExpectedBranchParentsAttribute[3]);

	const static auto AreBranchChildrenAttributesEqual = [](const TArray<int32>& BranchChildrenA, const TArray<int32>& BranchChildrenB)
	{
		const auto SetA = TSet<int32>(BranchChildrenA);
		const auto SetB = TSet<int32>(BranchChildrenB);
		return SetA.Num() == SetB.Num() && SetA.Includes(SetB);
	};

	// Validate branch children
	UTEST_TRUE("BranchChildren0", AreBranchChildrenAttributesEqual(ActualBranchChildrenAttribute[0], ExpectedBranchChildrenAttribute[0]));
	UTEST_TRUE("BranchChildren1", AreBranchChildrenAttributesEqual(ActualBranchChildrenAttribute[1], ExpectedBranchChildrenAttribute[1]));
	UTEST_TRUE("BranchChildren2", AreBranchChildrenAttributesEqual(ActualBranchChildrenAttribute[2], ExpectedBranchChildrenAttribute[2]));
	UTEST_TRUE("BranchChildren3", AreBranchChildrenAttributesEqual(ActualBranchChildrenAttribute[3], ExpectedBranchChildrenAttribute[3]));

	return true;
}

PV_IMPORTER_TEST(FindRootBranch)
{
	const TArray<TArray<int32>>& Branches = PVMockTree::BranchPointIndices;
	const int32 RootBranchIndex = PV::ImportHelper::FindRootBranch(PVMockTree::Points, Branches);

	UTEST_NOT_EQUAL("ValidRootBranch", RootBranchIndex, int32(INDEX_NONE));

	const TArray<int32> RootBranchIndices = Branches[RootBranchIndex];
	const FVector3f RootVertex = PVMockTree::Points[RootBranchIndices[0]];

	UTEST_NEARLY_EQUAL("CorrectRootBranchLocation", FVector(RootVertex), FVector::ZeroVector, UE_KINDA_SMALL_NUMBER);

	return true;
}

PV_IMPORTER_TEST(EstimateBranchHierarchy)
{
	// Test standard MockTree
	{
		TArray<TArray<int32>> Branches = {
			PVMockTree::Branch0_PointIndices,
			PVMockTree::Branch1_PointIndices,
			PVMockTree::Branch2_PointIndices,
			PVMockTree::Branch3_PointIndices
		};

		TArray<int32> ExpectedBranchHierarchy = { -1, 0, 1, 0 };

		// Naive testing of hierarchy
		{
			const int32 RootBranchIndex = PV::ImportHelper::FindRootBranch(PVMockTree::Points, Branches);
			const TArray<int32> BranchHierarchy = PV::ImportHelper::EstimateBranchHierarchy(PVMockTree::Points, Branches, RootBranchIndex);

			UTEST_EQUAL("CorrectNrOfBranches", BranchHierarchy.Num(), ExpectedBranchHierarchy.Num());
			for (int32 i = 0; i < Branches.Num(); ++i)
			{
				UTEST_EQUAL(FString::Printf(TEXT("Branch%d_CorrectParent"), i), BranchHierarchy[i], ExpectedBranchHierarchy[i]);
			}
		}

		// Test with branch list reversed
		{
			Algo::Reverse(Branches);
			Algo::Reverse(ExpectedBranchHierarchy);
			for (int32& Hierarchy : ExpectedBranchHierarchy)
			{
				if (Hierarchy != -1)
				{
					Hierarchy = Branches.Num() - 1 - Hierarchy;
				}
			}

			const int32 RootBranchIndex = PV::ImportHelper::FindRootBranch(PVMockTree::Points, Branches);
			const TArray<int32> BranchHierarchy = PV::ImportHelper::EstimateBranchHierarchy(PVMockTree::Points, Branches, RootBranchIndex);
			
			UTEST_EQUAL("CorrectNrOfBranches_Reverse", BranchHierarchy.Num(), ExpectedBranchHierarchy.Num());
			for (int32 i = 0; i < Branches.Num(); ++i)
			{
				UTEST_EQUAL(FString::Printf(TEXT("Branch%d_CorrectParent_Reverse"), i), BranchHierarchy[i], ExpectedBranchHierarchy[i]);
			}
		}
	}

	// Add additional branches to MockTree and ensure hierarchy stays correct
	{
		// Add a few branches starting from the same point as another as we want to make sure the algorithm can handle multiple 
		// branches being at the same distance from the start point of another.
		TArray<FVector3f> PointsCpy = PVMockTree::Points;
		const int32 Point2_1_Index = PointsCpy.Add(PVMockTree::Point2 + FVector3f(-10, 0, 0));
		const int32 Point2_2_Index = PointsCpy.Add(PVMockTree::Point2 + FVector3f(10, 0, 0));
		const int32 Point2_3_Index = PointsCpy.Add(PVMockTree::Point2 + FVector3f(-10, 10, 0));
		const int32 Point2_4_Index = PointsCpy.Add(PVMockTree::Point2 + FVector3f(10, 10, 0));
		const int32 Point2_5_Index = PointsCpy.Add(PVMockTree::Point2 + FVector3f(-10, 10, -10));
		const int32 Point2_6_Index = PointsCpy.Add(PVMockTree::Point2 + FVector3f(10, 8, 1));

		const int32 Point8_1_Index = PointsCpy.Add(PVMockTree::Point8 + FVector3f(10, 0, 0));
		const int32 Point8_2_Index = PointsCpy.Add(PVMockTree::Point8 + FVector3f(-10, 0, 0));
		const int32 Point8_3_Index = PointsCpy.Add(PVMockTree::Point8 + FVector3f(10, 10, 0));
		const int32 Point8_4_Index = PointsCpy.Add(PVMockTree::Point8 + FVector3f(-10, 10, 0));
		const int32 Point8_5_Index = PointsCpy.Add(PVMockTree::Point8 + FVector3f(10, 10, 10));
		const int32 Point8_6_Index = PointsCpy.Add(PVMockTree::Point8 + FVector3f(-10, 10, 10));

		const int32 Point3_0_Index = PointsCpy.Add(PVMockTree::Point3 + FVector3f(10, 0, 0));
		const int32 Point3_1_Index = PointsCpy.Add(PVMockTree::Point3 + FVector3f(-10, 0, 0));

		TArray<TArray<int32>> Branches = {
			PVMockTree::Branch0_PointIndices,
			PVMockTree::Branch1_PointIndices,
			PVMockTree::Branch2_PointIndices,
			PVMockTree::Branch3_PointIndices,
			{ 2, Point2_1_Index },
			{ 2, Point2_2_Index },
			{ 2, Point2_3_Index },
			{ 2, Point2_4_Index },
			{ 2, Point2_5_Index },
			{ 2, Point2_6_Index },

			{ 8, Point8_1_Index },
			{ 8, Point8_2_Index },
			{ 8, Point8_3_Index },
			{ 8, Point8_4_Index },
			{ 8, Point8_5_Index },
			{ 8, Point8_6_Index },

			{ 3, Point3_0_Index },
			{ 3, Point3_1_Index },
		};

		TArray<int32> ExpectedBranchHierarchy = { 
			-1, 0, 1, 0, 
			0, 0, 0, 0, 0, 0,
			1, 1, 1, 1, 1, 1,
			0, 0
		};

		// Naive testing of hierarchy
		{
			const int32 RootBranchIndex = PV::ImportHelper::FindRootBranch(PointsCpy, Branches);
			const TArray<int32> BranchHierarchy = PV::ImportHelper::EstimateBranchHierarchy(PointsCpy, Branches, RootBranchIndex);

			UTEST_EQUAL("CorrectNrOfBranches", BranchHierarchy.Num(), ExpectedBranchHierarchy.Num());
			for (int32 i = 0; i < Branches.Num(); ++i)
			{
				UTEST_EQUAL(FString::Printf(TEXT("Branch%d_CorrectParent"), i), BranchHierarchy[i], ExpectedBranchHierarchy[i]);
			}
		}

		// Test with branch list reversed
		{
			Algo::Reverse(Branches);
			Algo::Reverse(ExpectedBranchHierarchy);
			for (int32& Hierarchy : ExpectedBranchHierarchy)
			{
				if (Hierarchy != -1)
				{
					Hierarchy = Branches.Num() - 1 - Hierarchy;
				}
			}

			const int32 RootBranchIndex = PV::ImportHelper::FindRootBranch(PointsCpy, Branches);
			const TArray<int32> BranchHierarchy = PV::ImportHelper::EstimateBranchHierarchy(PointsCpy, Branches, RootBranchIndex);

			UTEST_EQUAL("CorrectNrOfBranches_Reverse", BranchHierarchy.Num(), ExpectedBranchHierarchy.Num());
			for (int32 i = 0; i < Branches.Num(); ++i)
			{
				UTEST_EQUAL(FString::Printf(TEXT("Branch%d_CorrectParent_Reverse"), i), BranchHierarchy[i], ExpectedBranchHierarchy[i]);
			}
		}
	}

	return true;
}

PV_IMPORTER_TEST(CreateBranchHierarchyFromEdges)
{
	FPVBranchHierarchyDescription ExpectedBranchHierarchy;
	ExpectedBranchHierarchy.RootBranchIndex = 0;
	ExpectedBranchHierarchy.Points = PVMockTree::Points;

	FPVBranchDescription& B0 = ExpectedBranchHierarchy.Branches.AddDefaulted_GetRef();
	B0.BranchIndex = 0;
	B0.PointIndices = PVMockTree::Branch0_PointIndices;
	B0.ParentBranchIndex = -1;
	B0.ChildBranchIndices = { 1, 3 };

	FPVBranchDescription& B1 = ExpectedBranchHierarchy.Branches.AddDefaulted_GetRef();
	B1.BranchIndex = 1;
	B1.PointIndices = PVMockTree::Branch1_PointIndices;
	B1.ParentBranchIndex = 0;
	B1.ChildBranchIndices = { 2 };

	FPVBranchDescription& B2 = ExpectedBranchHierarchy.Branches.AddDefaulted_GetRef();
	B2.BranchIndex = 2;
	B2.PointIndices = PVMockTree::Branch2_PointIndices;
	B2.ParentBranchIndex = 1;
	B2.ChildBranchIndices = { };

	FPVBranchDescription& B3 = ExpectedBranchHierarchy.Branches.AddDefaulted_GetRef();
	B3.BranchIndex = 3;
	B3.PointIndices = PVMockTree::Branch3_PointIndices;
	B3.ParentBranchIndex = 0;
	B3.ChildBranchIndices = { };

	const FPVBranchHierarchyDescription BranchHierarchy = PVMockTree::CreateBranchHierarchyDescription();

	UTEST_EQUAL("CorrectRootIndex", BranchHierarchy.RootBranchIndex, ExpectedBranchHierarchy.RootBranchIndex);
	UTEST_EQUAL("CorrectNrOfBranches", BranchHierarchy.Branches.Num(), ExpectedBranchHierarchy.Branches.Num());

	for (int32 i = 0; i < BranchHierarchy.Branches.Num(); ++i)
	{
		const FPVBranchDescription& Branch = BranchHierarchy.Branches[i];
		const FPVBranchDescription* ExpectedBranchPtr = ExpectedBranchHierarchy.Branches.FindByPredicate([&](const FPVBranchDescription& InBranch) { return InBranch.BranchIndex == Branch.BranchIndex; });

		UTEST_NOT_NULL("ContainsBranchIndex", ExpectedBranchPtr);

		const FPVBranchDescription& ExpectedBranch = *ExpectedBranchPtr;

		const FString BranchName = FString::Printf(TEXT("Branch%d: "), i);

		UTEST_EQUAL(BranchName + "_CorrectBranchIndex", Branch.BranchIndex, ExpectedBranch.BranchIndex);
		UTEST_EQUAL(BranchName + "_CorrectParent", Branch.ParentBranchIndex, ExpectedBranch.ParentBranchIndex);
		UTEST_EQUAL(BranchName + "_CorrectNrOfChildren", Branch.ChildBranchIndices.Num(), ExpectedBranch.ChildBranchIndices.Num());
		UTEST_EQUAL(BranchName + "_CorrectNrOfPoints", Branch.PointIndices.Num(), ExpectedBranch.PointIndices.Num());

		for (int32 j = 0; j < Branch.PointIndices.Num(); ++j)
		{
			const int32 PointIndex = Branch.PointIndices[j];
			const int32 ExpectedPointIndex = ExpectedBranch.PointIndices[j];
			UTEST_EQUAL(BranchName + FString::Printf(TEXT("_CorrectPointIndex %d"), j), PointIndex, ExpectedPointIndex);

			const FVector Point = FVector(BranchHierarchy.Points[PointIndex]);
			const FVector ExpectedPoint = FVector(ExpectedBranchHierarchy.Points[ExpectedPointIndex]);
			UTEST_NEARLY_EQUAL(BranchName + FString::Printf(TEXT("_CorrectPoint %d"), j), Point, ExpectedPoint, UE_KINDA_SMALL_NUMBER);
		}

		for (int32 j = 0; j < Branch.ChildBranchIndices.Num(); ++j)
		{
			UTEST_TRUE(BranchName + FString::Printf(TEXT("_CorrectChildIndex %d"), j), ExpectedBranch.ChildBranchIndices.Contains(Branch.ChildBranchIndices[j]));
		}
	}

	return true;
}

#endif