// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVMockTree.h"
#include "Helpers/PVImportHelpers.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"

const TCHAR* PVMockTreeCollection::CompareResultToString(PVMockTreeCollection::CompareResult InValue)
{
	switch (InValue)
	{
	case CompareResult::Success:
		return TEXT("Success");
	case CompareResult::InvalidCollection:
		return TEXT("InvalidCollection");
	case CompareResult::MismatchNrOfPoints:
		return TEXT("MismatchNrOfPoints");
	case CompareResult::MismatchNrOfBranches:
		return TEXT("MismatchNrOfBranches");
	case CompareResult::MismatchPointPositions:
		return TEXT("MismatchPointPositions");
	default:
		ensureMsgf(false, TEXT("Unknown result type"));
		return TEXT("Unknown");
	}
}

PVMockTreeCollection::CompareResult PVMockTreeCollection::Compare(const FManagedArrayCollection& Collection, bool bCheckPointPositions)
{
	const PV::Facades::FPointFacade InputPointFacade(Collection);
	const PV::Facades::FBranchFacade InputBranchFacade(Collection);
	if (!InputPointFacade.IsValid() || !InputBranchFacade.IsValid())
	{
		return CompareResult::InvalidCollection;
	}

	const TSharedRef<FManagedArrayCollection> BaselineCollection = PVMockTreeCollection::CreateCollection();

	const PV::Facades::FPointFacade BaselinePointFacade(*BaselineCollection);
	const PV::Facades::FBranchFacade BaselineBranchFacade(*BaselineCollection);

	if (InputPointFacade.GetPositions().Num() != BaselinePointFacade.GetPositions().Num())
	{
		return CompareResult::MismatchNrOfPoints;
	}

	if (InputBranchFacade.GetBranchNumbers().Num() != BaselineBranchFacade.GetBranchNumbers().Num())
	{
		return CompareResult::MismatchNrOfBranches;
	}

	if (bCheckPointPositions)
	{
		for (int32 i = 0; i < InputPointFacade.GetPositions().Num(); ++i)
		{
			const FVector3f& PointA = InputPointFacade.GetPosition(i);
			const FVector3f& PointB = BaselinePointFacade.GetPosition(i);

			if (!PointA.Equals(PointB))
			{
				return CompareResult::MismatchPointPositions;
			}
		}
	}

	return CompareResult::Success;
}

TSharedRef<FManagedArrayCollection> PVMockTreeCollection::CreateCollection()
{
	const FPVBranchHierarchyDescription BranchHierarchy = PV::ImportHelper::CreateBranchHierarchyFromPoints(PVMockTree::Points, PVMockTree::PointsRadii, PVMockTree::BranchPointIndices);

	TSharedRef<FManagedArrayCollection> OutCollection = MakeShared<FManagedArrayCollection>();
	const bool bResult = PV::ImportHelper::GenerateGrowthDataFromBranchHierarchy(*OutCollection, BranchHierarchy);
	check(bResult != false); // Mock tree is staticly defined to always be valid, so it should always return valid growth data

	return OutCollection;
}

#endif // WITH_DEV_AUTOMATION_TESTS
