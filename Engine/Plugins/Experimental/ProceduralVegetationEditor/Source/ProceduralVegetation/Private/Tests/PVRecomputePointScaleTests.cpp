// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVTestsCommon.h"
#include "Implementations/PVRecomputePointScale.h"

#define PV_RECOMPUTEPOINTSCALE_TEST(TestName) PV_SIMPLE_AUTOMATION_TEST(RecomputePointScale, TestName)

PV_RECOMPUTEPOINTSCALE_TEST(ComputePointScales_SmoothTaper)
{
	// Branch 0 (points 0-5) of the mock tree is used to cover all scenarios:
	//   P0 = 100  (base anchor)
	//   P1 = 5    (noisy dip: close to base, ImpliedTaperRate >> 3x ExpectedTaperRate => rejected)
	//   P2 = 200  (too large: > CurrentPointScale => skipped as not a valid anchor)
	//   P3 = 70   (normal taper anchor, should be preserved)
	//   P4 = 0    (zero scale, should be interpolated between P3 and P5)
	//   P5 = 20   (tip anchor, should be preserved)
	const TSharedRef<FManagedArrayCollection> Collection = PVMockTreeCollection::CreateCollection();

	PV::FPointScaleAttributeView PointScaleAttribute = PV::FPointScaleAttribute::FindAttribute(*Collection);
	PV::FPointLengthFromRootAttributeConstView PointLengthFromRootAttribute = PV::FPointLengthFromRootAttribute::FindAttribute(*Collection);
	PV::FBranchPointsAttributeConstView BranchPointsAttribute = PV::FBranchPointsAttribute::FindAttribute(*Collection);
	PV::FBranchParentNumberAttributeConstView BranchParentNumberAttribute = PV::FBranchParentNumberAttribute::FindAttribute(*Collection);
	PV::FBranchChildrenAttributeConstView BranchChildrenAttribute = PV::FBranchChildrenAttribute::FindAttribute(*Collection);
	PV::FBranchNumberAttributeConstView BranchNumberAttribute = PV::FBranchNumberAttribute::FindAttribute(*Collection);

	UTEST_TRUE("HasScaleAttribute", PointScaleAttribute.IsValid());
	UTEST_TRUE("HasLengthFromRootAttribute", PointLengthFromRootAttribute.IsValid());
	UTEST_TRUE("HasBranchPointsAttribute", BranchPointsAttribute.IsValid());

	// Set up test scale values for Branch 0 (point indices 0-5)
	PointScaleAttribute[0] = 100.0f; // base anchor
	PointScaleAttribute[1] =   5.0f; // noisy dip: ImpliedTaperRate ~205/m > 3 * ExpectedTaperRate ~151/m => rejected
	PointScaleAttribute[2] = 200.0f; // too large: > CurrentPointScale, never accepted as anchor
	PointScaleAttribute[3] =  70.0f; // valid taper anchor
	PointScaleAttribute[4] =   0.0f; // zero scale, must be interpolated
	PointScaleAttribute[5] =  20.0f; // tip anchor

	PV::ComputePointScales_SmoothTaper({
		PointScaleAttribute,
		PointLengthFromRootAttribute,
		BranchPointsAttribute,
		BranchParentNumberAttribute,
		BranchChildrenAttribute,
		BranchNumberAttribute
	});

	// P0: base anchor must be preserved unchanged
	UTEST_NEARLY_EQUAL("P0_BaseAnchorPreserved", PointScaleAttribute[0], 100.0f, 0.1f);

	// P1: noisy dip was rejected; interpolated between P0 (100) and the first accepted
	//     anchor P3 (70). At LFR alpha ~0.369: lerp(100, 70, 0.369) ~= 88.9
	UTEST_NEARLY_EQUAL("P1_NoisyDipInterpolated", PointScaleAttribute[1], 88.9f, 0.5f);

	// P2: too-large value was skipped; interpolated between P0 (100) and P3 (70).
	//     At LFR alpha ~0.738: lerp(100, 70, 0.738) ~= 77.9
	UTEST_NEARLY_EQUAL("P2_TooLargeInterpolated", PointScaleAttribute[2], 77.9f, 0.5f);

	// P3: normal taper anchor must be preserved unchanged
	UTEST_NEARLY_EQUAL("P3_NormalTaperPreserved", PointScaleAttribute[3], 70.0f, 0.1f);

	// P4: zero scale was interpolated between P3 (70) and P5 (20).
	//     At the exact midpoint (alpha = 0.5): lerp(70, 20, 0.5) = 45.0
	UTEST_NEARLY_EQUAL("P4_ZeroScaleInterpolated", PointScaleAttribute[4], 45.0f, 0.5f);

	// P5: tip anchor must be preserved unchanged
	UTEST_NEARLY_EQUAL("P5_TipAnchorPreserved", PointScaleAttribute[5], 20.0f, 0.1f);

	// Sanity check: the output must be a monotonically non-increasing taper
	UTEST_TRUE("MonotonicTaper_P0_P1", PointScaleAttribute[1] <= PointScaleAttribute[0]);
	UTEST_TRUE("MonotonicTaper_P1_P2", PointScaleAttribute[2] <= PointScaleAttribute[1]);
	UTEST_TRUE("MonotonicTaper_P2_P3", PointScaleAttribute[3] <= PointScaleAttribute[2]);
	UTEST_TRUE("MonotonicTaper_P3_P4", PointScaleAttribute[4] <= PointScaleAttribute[3]);
	UTEST_TRUE("MonotonicTaper_P4_P5", PointScaleAttribute[5] <= PointScaleAttribute[4]);

	// --- Fused-point regression ---
	// We are using fused points so BranchPoints[0] of a child branch is the same point as
	// the parent branch's attachment point. bShouldSkipFirstPoint / StartIndex=1 is supposed
	// to protect it. The bug was that the initial backfill loop (triggered when the first own
	// point of the child has scale 0) started at j=0 instead of j=StartIndex, so it would
	// overwrite BranchPoints[0] — the fused/shared point — with the child branch's scale.
	//
	// Mock-tree fused topology (bSplitPoints=false):
	//   Branch 0 (trunk): {0, 1, 2, 3, 4, 5}
	//   Branch 1:         {2, 7, 8, 9}   — point 2 fused with branch 0
	//   Branch 2:         {8, 10, 11}    — point 8 fused with branch 1
	//   Branch 3:         {3, 6}         — point 3 fused with branch 0
	//
	// Setting point 7 (Branch1's BranchPoints[1]) to 0 forces the backfill path.
	// With the old bug the backfill would start at j=0 and clobber point 2 with point 8's scale.
	{
		const TSharedRef<FManagedArrayCollection> FusedCollection = PVMockTreeCollection::CreateCollection();

		PV::FPointScaleAttributeView FusedScaleAttr = PV::FPointScaleAttribute::FindAttribute(*FusedCollection);
		PV::FPointLengthFromRootAttributeConstView FusedLFRAttr = PV::FPointLengthFromRootAttribute::FindAttribute(*FusedCollection);
		PV::FBranchPointsAttributeConstView FusedBranchPointsAttr = PV::FBranchPointsAttribute::FindAttribute(*FusedCollection);
		PV::FBranchParentNumberAttributeConstView FusedParentNumberAttr = PV::FBranchParentNumberAttribute::FindAttribute(*FusedCollection);
		PV::FBranchChildrenAttributeConstView FusedBranchChildrenAttr = PV::FBranchChildrenAttribute::FindAttribute(*FusedCollection);
		PV::FBranchNumberAttributeConstView FusedBranchNumberAttr = PV::FBranchNumberAttribute::FindAttribute(*FusedCollection);

		UTEST_TRUE("FusedCollection_HasScaleAttribute", FusedScaleAttr.IsValid());

		// Branch 0 (trunk): monotonically decreasing — all points kept as-is after processing.
		// Point 2 = 80 is the fused attachment point for branch 1.
		FusedScaleAttr[0]  = 100.0f;
		FusedScaleAttr[1]  =  90.0f;
		FusedScaleAttr[2]  =  80.0f; // fused: shared by branch 0 (index 2) and branch 1 (index 0)
		FusedScaleAttr[3]  =  70.0f;
		FusedScaleAttr[4]  =  60.0f;
		FusedScaleAttr[5]  =  50.0f;
		FusedScaleAttr[6]  =  40.0f; // branch 3 own point
		FusedScaleAttr[7]  =   0.0f; // KEY: branch 1 BranchPoints[1] — triggers the backfill path
		FusedScaleAttr[8]  =  30.0f; // branch 1 BranchPoints[2] — first valid value for backfill
		FusedScaleAttr[9]  =  10.0f;
		FusedScaleAttr[10] =  20.0f;
		FusedScaleAttr[11] =   5.0f;

		PV::ComputePointScales_SmoothTaper({
			FusedScaleAttr,
			FusedLFRAttr,
			FusedBranchPointsAttr,
			FusedParentNumberAttr,
			FusedBranchChildrenAttr,
			FusedBranchNumberAttr
		});

		// Point 2 is the fused point. Branch 1's backfill must not overwrite it;
		// it must keep the value set by branch 0 (80).
		UTEST_NEARLY_EQUAL("FusedPoint_P2_NotOverwrittenByChildBackfill", FusedScaleAttr[2], 80.0f, 0.1f);

		// Point 7 is branch 1's first own point; it was 0 and must have been backfilled to
		// the value of the first valid successor (point 8 = 30).
		UTEST_NEARLY_EQUAL("FusedPoint_P7_BackfilledToP8Scale", FusedScaleAttr[7], 30.0f, 0.1f);
	}

	return true;
}

#endif