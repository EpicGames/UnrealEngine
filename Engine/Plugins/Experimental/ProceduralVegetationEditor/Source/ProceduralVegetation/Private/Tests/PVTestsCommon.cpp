// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Tests/PVTestsCommon.h"

DEFINE_LOG_CATEGORY(LogPVTest);

#endif //WITH_DEV_AUTOMATION_TESTS