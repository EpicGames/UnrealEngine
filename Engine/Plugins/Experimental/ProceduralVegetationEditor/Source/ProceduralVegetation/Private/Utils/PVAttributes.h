// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

#include "Facades/PVAttributesNames.h"

/*
This file defines the TAttributeDefinition type which allows for compile-time type checking of attributes.

- Defining a new attribute:
	To define a new attribute, use the PV_DEFINE_ATTRIBUTE as follows:

	inline constexpr TCHAR MyPropertyNameStr[] = TEXT("MyPropertyName");
	inline constexpr TCHAR MyPropertyGroupStr[] = TEXT("MyPropertyGroup");
    PV_DEFINE_ATTRIBUTE(MyProperty, FVector3f, MyPropertyNameStr, MyPropertyGroupStr);
	PV_DEFINE_STRUCTURED_ATTRIBUTE can be used when custom element accessor views are needed.

- Usage in code: 
	PV_DEFINE_ATTRIBUTE defines a new attribute type, the example above defines the type FMyPropertyAttribute.
	This type can be used with the FManagedArrayCollection to get, find, and add attributes using `FMyPropertyAttribute::FindAttribute(Collection)`, 
	`FMyPropertyAttribute::GetAttribute(Collection)`, or `FMyPropertyAttribute::AddAttribute(Collection)`.

	PV_DEFINE_ATTRIBUTE also defines a view and a const view for the attribute which can be used to access elements in the attribute.
	Usage example:
	void MultiplyMyProperty(FMyPropertyAttributeView MyPropertyAttribute, float Multiplier)
	{
		for (FVector3f& Vec : MyPropertyAttribute)
		{
			Vec *= Multiplier;
		}
	}
	
	// At call site:
	MultiplyMyProperty(FMyPropertyAttribute::GetAttribute(Collection), 10.f);
*/

namespace PV
{
template<
	typename InElementType,
	typename InElementViewType,
	typename InElementConstViewType,
	const TCHAR* InAttributeNameStr, 
	const TCHAR* InGroupNameStr
>
class TAttributeView
{
public:
	using ElementType = std::remove_const_t<InElementType>;
	
	using ManagedArrayType =
		std::conditional_t<std::is_const_v<InElementType>,
		const TManagedArray<ElementType>,
		TManagedArray<ElementType>>;
	
	using ElementViewType = 
		std::conditional_t<std::is_const_v<InElementType>,
		InElementConstViewType,
		InElementViewType>;

	using MutableAttributeViewType = TAttributeView<ElementType, InElementViewType, InElementConstViewType, InAttributeNameStr, InGroupNameStr>;
	using ConstAttributeViewType = TAttributeView<const ElementType, InElementViewType, InElementConstViewType, InAttributeNameStr, InGroupNameStr>;

	using SizeType = TArray<ElementType>::SizeType;

	inline static const FName AttributeName = FName(InAttributeNameStr);
	inline static const FName GroupName = FName(InGroupNameStr);

	ManagedArrayType* ManagedArray;

	TAttributeView()
		: ManagedArray(nullptr)
	{}

	TAttributeView(ManagedArrayType* InManagedArray)
		: ManagedArray(InManagedArray)
	{}

	TAttributeView(const MutableAttributeViewType& Other)
		: ManagedArray(Other.ManagedArray)
	{}

	template<typename U = InElementType,
		typename = std::enable_if_t<std::is_const_v<U>>>
	TAttributeView(const ConstAttributeViewType& Other)
		: ManagedArray(Other.ManagedArray)
	{
	}

	bool IsValid() const
	{
		return ManagedArray != nullptr;
	}

	ElementViewType operator[](SizeType Index) const
	{
		check(IsValid() && ManagedArray->IsValidIndex(Index));
		return ElementViewType((*ManagedArray)[Index]);
	}

	operator bool() const 
	{
		return IsValid();
	}

	bool operator!() const
	{
		return !IsValid();
	}

	/**
	* return true if index is in array range.
	*
	* @param Index Index to check.
	*/
	bool IsValidIndex(SizeType Index) const
	{
		return ManagedArray && ManagedArray->IsValidIndex(Index);
	}

	/**
	* Find first index of the element
	*/
	int32 Find(const ElementType& Item) const
	{
		return ManagedArray ? ManagedArray->Find(Item) : INDEX_NONE;
	}

	SizeType Num() const
	{
		return ManagedArray ? ManagedArray->Num() : 0;
	}

	using RangedForIteratorType      = ManagedArrayType::RangedForIteratorType;
	using RangedForConstIteratorType = ManagedArrayType::RangedForConstIteratorType;

	using AttributeRangedForIteratorType =
		std::conditional_t<std::is_const_v<InElementType>,
		RangedForConstIteratorType,
		RangedForIteratorType>;

	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	FORCEINLINE AttributeRangedForIteratorType begin() const { check(ManagedArray); return ManagedArray->begin(); }
	FORCEINLINE AttributeRangedForIteratorType end() const { check(ManagedArray); return ManagedArray->end(); }
};

template<
	typename InElementType,
	typename InElementViewType,
	typename InElementConstViewType,
	const TCHAR* InAttributeNameStr,
	const TCHAR* InGroupNameStr
>
class TAttributeAccessor : public TManagedArrayAccessor<InElementType>
{
	static_assert(!std::is_const_v<InElementType>, "InElementType cannot be const");

public:
	using ElementType = InElementType;
	using ManagedArrayType = TManagedArray<ElementType>;
	using AttributeView = TAttributeView<ElementType, InElementViewType, InElementConstViewType, InAttributeNameStr, InGroupNameStr>;
	using AttributeConstView = TAttributeView<const ElementType, InElementViewType, InElementConstViewType, InAttributeNameStr, InGroupNameStr>;
	using Super = TManagedArrayAccessor<ElementType>;
	
	inline static const FName AttributeName = FName(InAttributeNameStr);
	inline static const FName GroupName = FName(InGroupNameStr);

	TAttributeAccessor() = delete;

	TAttributeAccessor(FManagedArrayCollection& InCollection)
		: Super(InCollection, AttributeName, GroupName)
	{}

	TAttributeAccessor(const FManagedArrayCollection& InCollection)
		: Super(InCollection, AttributeName, GroupName)
	{}

	AttributeView GetView()
	{
		return Super::IsValid() ? AttributeView(&Super::Get()) : AttributeView(nullptr);
	}

	AttributeConstView GetConstView() const
	{
		return Super::IsValid() ? AttributeConstView(&Super::Get()) : AttributeConstView(nullptr);
	}

	InElementViewType ModifyAt(int32 Index)
	{
		ManagedArrayType& MutableArray = Super::Modify();
		return InElementViewType(MutableArray[Index]);
	}

	InElementConstViewType operator[](int32 Index) const
	{
		return InElementConstViewType(Super::operator[](Index));
	}
};

template<
	typename InAttributeType, 
	typename InElementViewType,
	typename InElementConstViewType,
	const TCHAR* InAttributeNameStr, 
	const TCHAR* InGroupNameStr
>
class TAttributeDefinition
{
	static_assert(!std::is_const_v<InAttributeType>, "InAttributeType cannot be const");

public:
	using AttributeType = InAttributeType;
	using AttributeAccessor = TAttributeAccessor<AttributeType, InElementViewType, InElementConstViewType, InAttributeNameStr, InGroupNameStr>;
	using AttributeView = AttributeAccessor::AttributeView;
	using AttributeConstView = AttributeAccessor::AttributeConstView;

	inline static const FName AttributeName = FName(InAttributeNameStr);
	inline static const FName GroupName = FName(InGroupNameStr);
	inline static const FManagedArrayCollection::FManagedType ManagedType = FManagedArrayCollection::TManagedType<AttributeType>(AttributeName, GroupName);

	/**
	* Tries to find the attribute in the supplied collection. Returns a null view if attribute is not in collection.
	* @return A const view of the found attribute.
	*/
	static AttributeConstView FindAttribute(const FManagedArrayCollection& InCollection)
	{
		return AttributeConstView(InCollection.FindAttributeTyped<AttributeType>(AttributeName, GroupName));
	}
	
	/**
	* Tries to find the attribute in the supplied collection. Returns a null view if attribute is not in collection.
	* @return A mutable view of the found attribute.
	*/
	static AttributeView FindAttribute(FManagedArrayCollection& InCollection)
	{
		return AttributeView(InCollection.FindAttributeTyped<AttributeType>(AttributeName, GroupName));
	}

	/**
	* Checked version of FindAttribute. Will assert if attribute is missing from collection.
	* @return A const view of the found attribute.
	*/
	static AttributeConstView GetAttribute(const FManagedArrayCollection& InCollection)
	{
		AttributeConstView AttributeView = FindAttribute(InCollection);
		check(AttributeView.IsValid());
		return AttributeView;
	}

	/**
	* Checked version of FindAttribute. Will assert if attribute is missing from collection.
	* @return A mutable view of the found attribute.
	*/ 
	static AttributeView GetAttribute(FManagedArrayCollection& InCollection)
	{
		AttributeView AttributeView = FindAttribute(InCollection);
		check(AttributeView.IsValid());
		return AttributeView;
	}

	/**
	* Add the attribute to the collection and returns a view to the added collection. Returns existing attribute in the collection if already present.
	* @return A mutable view of the found attribute.
	*/
	static AttributeView AddAttribute(FManagedArrayCollection& InCollection)
	{
		return AttributeView(&InCollection.AddAttribute<AttributeType>(AttributeName, GroupName));
	}

	/**
	* Add the attribute to the collection and returns a view to the added collection. Returns existing attribute in the collection if already present.
	* @param DefaultValue - Initialize all elements in the added attribute with this value (only if the attribute is not already present)
	* @return A mutable view of the found attribute.
	*/
	static AttributeView AddAttribute(FManagedArrayCollection& InCollection, const AttributeType& DefaultValue)
	{
		AttributeView AttributeView = FindAttribute(InCollection);
		if (AttributeView.IsValid())
		{
			return AttributeView;
		}

		AttributeView = AddAttribute(InCollection);
		for (int32 i = 0; i < AttributeView.Num(); ++i)
		{
			AttributeView[i] = DefaultValue;
		}
		return AttributeView;
	}

	/**
	* Check for the existence of the attribute in the collection.
	*/
	static bool HasAttribute(const FManagedArrayCollection& InCollection)
	{
		return InCollection.HasAttribute(AttributeName, GroupName);
	}

	/**
	* Copy the attribute from one collection to another. Will perform an implicit group sync.
	*/
	static void CopyAttribute(const FManagedArrayCollection& SourceCollection, FManagedArrayCollection& DestCollection)
	{
		if (!SourceCollection.HasAttribute(AttributeName, GroupName))
		{
			return;
		}

		DestCollection.CopyAttribute(SourceCollection, AttributeName, GroupName);
	}
};

// Helper class to allow for initialization, and selecton of, a fixed size array based of off default input array.
template<typename InElementType>
struct TFixedSizeArrayHelper
{
	using ElementType = std::remove_const_t<InElementType>;
	using MutableArrayType = TArray<ElementType>;
	using ConstArrayType = const TArray<ElementType>;
	using ArrayType = std::conditional_t<std::is_const_v<InElementType>, ConstArrayType, MutableArrayType>;

	TFixedSizeArrayHelper() = delete;

	static MutableArrayType& GetFixedArray(MutableArrayType& InMutableArray, const ConstArrayType& InDefaultData)
	{
		if (InMutableArray.Num() < InDefaultData.Num())
		{
			const int32 FirstIndex = InMutableArray.Num();
			InMutableArray.SetNum(InDefaultData.Num());
			for (int32 i = FirstIndex; i < InDefaultData.Num(); ++i)
			{
				InMutableArray[i] = InDefaultData[i];
			}
		}

		return InMutableArray;
	}

	static ConstArrayType& GetFixedArray(ConstArrayType& InConstArray, const ConstArrayType& InDefaultData)
	{
		if (InConstArray.Num() < InDefaultData.Num())
		{
			return InDefaultData;
		}
		return InConstArray;
	}
};

// Returns true if all attributes are valid, non-empty, and attributes sharing a group name have the same element count.
template<typename... TAttributeViews>
inline bool ValidateAttributeCollection(TAttributeViews... Attributes)
{
	TMap<FName, int32> GroupSizes;
	GroupSizes.Reserve(sizeof...(TAttributeViews));

	bool bIsValid = true;
	auto AddEntry = [&GroupSizes, &bIsValid](auto Attr)
	{
		if (!Attr.IsValid())
		{
			bIsValid = false;
			return;
		}

		if (Attr.Num() == 0)
		{
			bIsValid = false;
			return;
		}

		const FName GroupName = std::remove_reference_t<decltype(Attr)>::GroupName;
		const int32 Size = Attr.Num();
		if (const int32* ExistingSize = GroupSizes.Find(GroupName))
		{
			bIsValid &= (*ExistingSize == Size);
		}
		else
		{
			GroupSizes.Add(GroupName, Size);
		}
	};

	(AddEntry(Attributes), ...);

	return bIsValid;
}

#define PV_DEFINE_STRUCTURED_ATTRIBUTE(Name, Type, ElementViewType, ElementConstViewType, AttributeNameStr, AttributeGroupStr)          \
	using F##Name##Attribute = TAttributeDefinition<Type, ElementViewType, ElementConstViewType, AttributeNameStr, AttributeGroupStr>;  \
	using F##Name##AttributeView = F##Name##Attribute::AttributeView;                                                                   \
	using F##Name##AttributeConstView = F##Name##Attribute::AttributeConstView;                                                         \
	using F##Name##AttributeAccessor = F##Name##Attribute::AttributeAccessor;

#define PV_DEFINE_ATTRIBUTE(Name, Type, AttributeNameStr, AttributeGroupStr) \
	PV_DEFINE_STRUCTURED_ATTRIBUTE(Name, Type, Type&, const Type&, AttributeNameStr, AttributeGroupStr)

// Branch Attributes
PV_DEFINE_ATTRIBUTE(BranchParents, TArray<int32>, AttributeNames::BranchParentsStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchChildren, TArray<int32>, AttributeNames::BranchChildrenStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchPoints, TArray<int32>, AttributeNames::BranchPointsStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchNumber, int32, AttributeNames::BranchNumberStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchSourceBudNumber, int32, AttributeNames::BranchSourceBudNumberStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchFoliageIDs, TArray<int32>, AttributeNames::BranchFoliageIDsStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchUVMaterial, int32, AttributeNames::BranchUVMaterialStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchHierarchyNumber, int32, AttributeNames::BranchHierarchyNumberStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchSimulationGroupIndex, int32, AttributeNames::BranchSimulationGroupIndexStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchPlantNumber, int32, AttributeNames::PlantNumberStr, GroupNames::BranchGroupStr);
PV_DEFINE_ATTRIBUTE(BranchParentNumber, int32, AttributeNames::BranchParentNumberStr, GroupNames::BranchGroupStr);

// Point Attributes
PV_DEFINE_ATTRIBUTE(PointPosition, FVector3f, AttributeNames::PointPositionStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointLengthFromRoot, float, AttributeNames::LengthFromRootStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointLengthFromSeed, float, AttributeNames::LengthFromSeedStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointScale, float, AttributeNames::PointScaleStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointTextureCoordV, float, AttributeNames::TextureCoordVStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointTextureCoordUOffset, float, AttributeNames::TextureCoordUOffsetStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointURange, FVector2f, AttributeNames::URangeStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointHullGradient, float, AttributeNames::HullGradientStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointMainTrunkGradient, float, AttributeNames::MainTrunkGradientStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointGroundGradient, float, AttributeNames::GroundGradientStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointScaleGradient, float, AttributeNames::PointScaleGradientStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointPlantGradient, float, AttributeNames::PlantGradientStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointBudNumber, int32, AttributeNames::BudNumberStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointNjordPixelIndex, float, AttributeNames::NjordPixelIndexStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointSeedPScale, float, AttributeNames::SeedPScaleStr, GroupNames::PointGroupStr);
PV_DEFINE_ATTRIBUTE(PointSeedPScaleRatio, float, AttributeNames::SeedPScaleRatioStr, GroupNames::PointGroupStr);

template<typename TFloatType>
struct TBudLightDetectedView
{
	using ArrayType = TFixedSizeArrayHelper<TFloatType>::ArrayType;

	inline const static TArray<float> DefaultData = { 0, 0, 0, 0 };

	ArrayType& Array;
	TFloatType& Availible;
	TFloatType& Resource;
	TFloatType& Branch;
	TFloatType& Collision;

	TBudLightDetectedView() = delete;
	TBudLightDetectedView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TFloatType>::GetFixedArray(InData, DefaultData))
		, Availible(Array[0])
		, Resource(Array[1])
		, Branch(Array[2])
		, Collision(Array[3])
	{
	}
};
using FBudLightDetectedView = TBudLightDetectedView<float>;
using FBudLightDetectedConstView = TBudLightDetectedView<const float>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(BudLightDetected, TArray<float>, FBudLightDetectedView, FBudLightDetectedConstView, AttributeNames::BudLightDetectedStr, GroupNames::PointGroupStr);

template<typename TFloatType>
struct TBudLateralMeristemView
{
	using ArrayType = TFixedSizeArrayHelper<TFloatType>::ArrayType;
	
	inline const static TArray<float> DefaultData = { 0, 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TFloatType& LateralMeristem;
	TFloatType& Multiplier;
	TFloatType& Inactive;
	TFloatType& Davinci;
	TFloatType& ParentDot;
	TFloatType& RootDistance;
	TFloatType& Degredation;

	TBudLateralMeristemView() = delete;
	TBudLateralMeristemView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TFloatType>::GetFixedArray(InData, DefaultData))
		, LateralMeristem(Array[0])
		, Multiplier(Array[1])
		, Inactive(Array[2])
		, Davinci(Array[3])
		, ParentDot(Array[4])
		, RootDistance(Array[5])
		, Degredation(Array[6])
	{
	}
};
using FBudLateralMeristemView = TBudLateralMeristemView<float>;
using FBudLateralMeristemConstView = TBudLateralMeristemView<const float>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(BudLateralMeristem, TArray<float>, FBudLateralMeristemView, FBudLateralMeristemConstView, AttributeNames::BudLateralMeristemStr, GroupNames::PointGroupStr);

template<typename TFloatType>
struct TBudHormoneLevelsView
{
	using ArrayType = TFixedSizeArrayHelper<TFloatType>::ArrayType;

	inline const static TArray<float> DefaultData = { 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TFloatType& Apical;
	TFloatType& Axillary;
	TFloatType& AxillaryInhibition;
	TFloatType& Radical;
	TFloatType& Ethylene;
	TFloatType& Cytokinin;

	TBudHormoneLevelsView() = delete;
	TBudHormoneLevelsView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TFloatType>::GetFixedArray(InData, DefaultData))
		, Apical(Array[0])
		, Axillary(Array[1])
		, AxillaryInhibition(Array[2])
		, Radical(Array[3])
		, Ethylene(Array[4])
		, Cytokinin(Array[5])
	{
	}
};
using FBudHormoneLevelsView = TBudHormoneLevelsView<float>;
using FBudHormoneLevelsConstView = TBudHormoneLevelsView<const float>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(BudHormoneLevels, TArray<float>, FBudHormoneLevelsView, FBudHormoneLevelsConstView, AttributeNames::BudHormoneLevelsStr, GroupNames::PointGroupStr);

template<typename TVec3Type>
struct TBudDirectionView
{
	using ArrayType = TFixedSizeArrayHelper<TVec3Type>::ArrayType;

	inline const static TArray<FVector3f> DefaultData = { FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector };

	ArrayType& Array;
	TVec3Type& Apical;
	TVec3Type& Axillary;
	TVec3Type& LightOptimal;
	TVec3Type& LightSubOptimal;
	TVec3Type& GuideCurve;
	TVec3Type& UpVector;

	TBudDirectionView() = delete;
	TBudDirectionView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TVec3Type>::GetFixedArray(InData, DefaultData))
		, Apical(Array[0])
		, Axillary(Array[1])
		, LightOptimal(Array[2])
		, LightSubOptimal(Array[3])
		, GuideCurve(Array[4])
		, UpVector(Array[5])
	{
	}
};
using FBudDirectionView = TBudDirectionView<FVector3f>;
using FBudDirectionConstView = TBudDirectionView<const FVector3f>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(BudDirection, TArray<FVector3f>, FBudDirectionView, FBudDirectionConstView, AttributeNames::BudDirectionStr, GroupNames::PointGroupStr);

template<typename TIntType>
struct TBudStatusView
{
	using ArrayType = TFixedSizeArrayHelper<TIntType>::ArrayType;
	
	inline const static TArray<int32> DefaultData = { 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TIntType& ApicalMeristem;
	TIntType& Codominant;
	TIntType& Axillary;
	TIntType& Seed;
	TIntType& Dormant;
	TIntType& Triggered;
	TIntType& NumTriggered;
	TIntType& Inactive;
	TIntType& BrokenTip;
	TIntType& Broken;

	TBudStatusView() = delete;
	TBudStatusView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TIntType>::GetFixedArray(InData, DefaultData))
		, ApicalMeristem(Array[0])
		, Codominant(Array[1])
		, Axillary(Array[2])
		, Seed(Array[3])
		, Dormant(Array[4])
		, Triggered(Array[5])
		, NumTriggered(Array[6])
		, Inactive(Array[7])
		, BrokenTip(Array[8])
		, Broken(Array[9])
	{
	}
};
using FBudStatusView = TBudStatusView<int32>;
using FBudStatusConstView = TBudStatusView<const int32>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(BudStatus, TArray<int32>, FBudStatusView, FBudStatusConstView, AttributeNames::BudStatusStr, GroupNames::PointGroupStr);

template<typename TIntType>
struct TBudDevelopmentView
{
	using ArrayType = TFixedSizeArrayHelper<TIntType>::ArrayType;

	inline const static TArray<int32> DefaultData = { 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TIntType& Generation;
	TIntType& BudAge;
	TIntType& BranchAge;
	TIntType& AgeSenescense;
	TIntType& LightSenescense;
	TIntType& RelativeBudAge;

	TBudDevelopmentView() = delete;
	TBudDevelopmentView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TIntType>::GetFixedArray(InData, DefaultData))
		, Generation(Array[0])
		, BudAge(Array[1])
		, BranchAge(Array[2])
		, AgeSenescense(Array[3])
		, LightSenescense(Array[4])
		, RelativeBudAge(Array[5])
	{
	}
};
using FBudDevelopmentView = TBudDevelopmentView<int32>;
using FBudDevelopmentConstView = TBudDevelopmentView<const int32>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(BudDevelopment, TArray<int32>, FBudDevelopmentView, FBudDevelopmentConstView, AttributeNames::BudDevelopmentStr, GroupNames::PointGroupStr);

// Foliage Attributes
PV_DEFINE_ATTRIBUTE(FoliageNameID, int32, AttributeNames::FoliageNameIDStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliageBranchID, int32, AttributeNames::FoliageBranchIDStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliagePivotPoint, FVector3f, AttributeNames::FoliagePivotPointStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliageUpVector, FVector3f, AttributeNames::FoliageUPVectorStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliageNormalVector, FVector3f, AttributeNames::FoliageNormalVectorStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliageScale, float, AttributeNames::FoliageScaleStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliageLengthFromRoot, float, AttributeNames::FoliageLengthFromRootStr, GroupNames::FoliageGroupStr);
PV_DEFINE_ATTRIBUTE(FoliageParentBoneID, int32, AttributeNames::FoliageParentBoneIDStr, GroupNames::FoliageGroupStr);

// FoliageNames Attributes
PV_DEFINE_ATTRIBUTE(FoliageName, FString, AttributeNames::FoliageNameStr, GroupNames::FoliageNamesGroupStr);

// Details Attributes
PV_DEFINE_ATTRIBUTE(DetailLeafPhyllotaxy, TArray<float>, AttributeNames::LeafPhyllotaxyStr, GroupNames::DetailsGroupStr);
PV_DEFINE_ATTRIBUTE(DetailFoliagePath, FString, AttributeNames::FoliagePathStr, GroupNames::DetailsGroupStr);
PV_DEFINE_ATTRIBUTE(DetailGuid, FGuid, AttributeNames::GuidStr, GroupNames::DetailsGroupStr);

template<typename TFloatType>
struct TDetailLeafGrowthView
{
	using ArrayType = TFixedSizeArrayHelper<TFloatType>::ArrayType;

	inline const static TArray<float> DefaultData = { 0, 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TFloatType& Ethylene;
	TFloatType& StartScale;
	TFloatType& EndScale;
	TFloatType& Density;
	TFloatType& AbscisicAcid;
	TFloatType& EthyleneThreshold;
	TFloatType& LeafAuxinRetention;

	TDetailLeafGrowthView() = delete;
	TDetailLeafGrowthView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TFloatType>::GetFixedArray(InData, DefaultData))
		, Ethylene(Array[0])
		, StartScale(Array[1])
		, EndScale(Array[2])
		, Density(Array[3])
		, AbscisicAcid(Array[4])
		, EthyleneThreshold(Array[5])
		, LeafAuxinRetention(Array[6])
	{
	}
};
using FDetailLeafGrowthView = TDetailLeafGrowthView<float>;
using FDetailLeafGrowthConstView = TDetailLeafGrowthView<const float>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(DetailLeafGrowth, TArray<float>, FDetailLeafGrowthView, FDetailLeafGrowthConstView, AttributeNames::LeafGrowthStr, GroupNames::DetailsGroupStr);

template<typename TFloatType>
struct TDetailAbscissionSenescenseView
{
	using ArrayType = TFixedSizeArrayHelper<TFloatType>::ArrayType;

	inline const static TArray<float> DefaultData = { 0, 0, 0, 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TFloatType& GravitationalAbscissionFactor;
	TFloatType& AgeAbscissionThreshold;
	TFloatType& AgeAbscissionMin;
	TFloatType& AgeAbscissionMax;
	TFloatType& AgeScaleRetention;
	TFloatType& LightAbscissionThreshold;
	TFloatType& LightAbscissionMin;
	TFloatType& LightAbscissionMax;
	TFloatType& LightScaleRetention;

	TDetailAbscissionSenescenseView() = delete;
	TDetailAbscissionSenescenseView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TFloatType>::GetFixedArray(InData, DefaultData))
		, GravitationalAbscissionFactor(Array[0])
		, AgeAbscissionThreshold(Array[1])
		, AgeAbscissionMin(Array[2])
		, AgeAbscissionMax(Array[3])
		, AgeScaleRetention(Array[4])
		, LightAbscissionThreshold(Array[5])
		, LightAbscissionMin(Array[6])
		, LightAbscissionMax(Array[7])
		, LightScaleRetention(Array[8])
	{
	}
};
using FDetailAbscissionSenescenseView = TDetailAbscissionSenescenseView<float>;
using FDetailAbscissionSenescenseConstView = TDetailAbscissionSenescenseView<const float>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(DetailAbscissionSenescense, TArray<float>, FDetailAbscissionSenescenseView, FDetailAbscissionSenescenseConstView, AttributeNames::AbscissionSenescenseStr, GroupNames::DetailsGroupStr);

template<typename TFloatType>
struct TDetailLateralElongationView
{
	using ArrayType = TFixedSizeArrayHelper<TFloatType>::ArrayType;

	inline const static TArray<float> DefaultData = { 0, 0, 0, 0, 0, 0, 0, 0, 0 };

	ArrayType& Array;
	TFloatType& Elongation;
	TFloatType& SeedScaleEffect;
	TFloatType& WhorledImpact;
	TFloatType& CellDensityMin;
	TFloatType& CellDensityMax;
	TFloatType& CellDensityCore;
	TFloatType& CellDevelopmentTime;
	TFloatType& LeafWeight;
	TFloatType& CellDensityInput;

	TDetailLateralElongationView() = delete;
	TDetailLateralElongationView(ArrayType& InData)
		: Array(TFixedSizeArrayHelper<TFloatType>::GetFixedArray(InData, DefaultData))
		, Elongation(Array[0])
		, SeedScaleEffect(Array[1])
		, WhorledImpact(Array[2])
		, CellDensityMin(Array[3])
		, CellDensityMax(Array[4])
		, CellDensityCore(Array[5])
		, CellDevelopmentTime(Array[6])
		, LeafWeight(Array[7])
		, CellDensityInput(Array[8])
	{
	}
};
using FDetailLateralElongationView = TDetailLateralElongationView<float>;
using FDetailLateralElongationConstView = TDetailLateralElongationView<const float>;
PV_DEFINE_STRUCTURED_ATTRIBUTE(DetailLateralElongation, TArray<float>, FDetailLateralElongationView, FDetailLateralElongationConstView, AttributeNames::LateralElongationStr, GroupNames::DetailsGroupStr);
PV_DEFINE_ATTRIBUTE(DetailTrunkURange, TArray<FVector2f>, AttributeNames::TrunkURangeStr, GroupNames::DetailsGroupStr);
PV_DEFINE_ATTRIBUTE(DetailTrunkMaterialPath, FString, AttributeNames::TrunkMaterialPathStr, GroupNames::DetailsGroupStr);

// PlantProfiles Attributes
PV_DEFINE_ATTRIBUTE(ProfilePoints, TArray<float>, AttributeNames::ProfilePointsStr, GroupNames::PlantProfilesGroupStr);

// Bone Attributes
PV_DEFINE_ATTRIBUTE(BoneName, FString, AttributeNames::BoneNameStr, GroupNames::BonesGroupStr);
PV_DEFINE_ATTRIBUTE(BoneParentIndex, int32, AttributeNames::BoneParentIndexStr, GroupNames::BonesGroupStr);
PV_DEFINE_ATTRIBUTE(BonePose, FTransform, AttributeNames::BonePoseStr, GroupNames::BonesGroupStr);
PV_DEFINE_ATTRIBUTE(BonePointIndex, int32, AttributeNames::BonePointIndexStr, GroupNames::BonesGroupStr);
PV_DEFINE_ATTRIBUTE(BoneAbsolutePosition, FVector3f, AttributeNames::BoneAbsolutePositionStr, GroupNames::BonesGroupStr);
PV_DEFINE_ATTRIBUTE(BoneBranchIndex, int32, AttributeNames::BoneBranchIndexStr, GroupNames::BonesGroupStr);
PV_DEFINE_ATTRIBUTE(BoneId, int32, AttributeNames::BoneIdStr, GroupNames::BonesGroupStr);

// VerticesGroup
PV_DEFINE_ATTRIBUTE(VerticesPointIds, int32, AttributeNames::VertexPointIdsStr, GroupNames::VerticesGroupStr);

// Leaf Meta Attributes (one element per plant; carries the leaf static mesh soft-object path)
PV_DEFINE_ATTRIBUTE(LeafMeshPath, FString, AttributeNames::LeafMeshPathStr, GroupNames::LeafProxyMetaGroupStr);

// Leaves Attributes (one element per leaf instance; stores the final-cycle spawn transform)
PV_DEFINE_ATTRIBUTE(LeafPosition, FVector3f, AttributeNames::PointPositionStr, GroupNames::LeafProxyGroupStr);
PV_DEFINE_ATTRIBUTE(LeafRotation, FVector4f, AttributeNames::LeafRotationStr, GroupNames::LeafProxyGroupStr);
PV_DEFINE_ATTRIBUTE(LeafScale, float, AttributeNames::PointScaleStr, GroupNames::LeafProxyGroupStr);

};