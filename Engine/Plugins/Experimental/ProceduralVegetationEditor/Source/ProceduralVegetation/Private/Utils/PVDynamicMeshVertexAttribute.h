// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicVertexAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "GeometryCollection/GeometryCollection.h" // Needed for CopyAttributeToCollection

namespace PV
{
template<typename AttribValueType>
class TDynamicMeshVertexAttributeExt 
	: public UE::Geometry::TDynamicMeshVertexAttribute<AttribValueType, 1>
{
public:
	using Super = UE::Geometry::TDynamicMeshVertexAttribute<AttribValueType, 1>;

	TDynamicMeshVertexAttributeExt() : Super() {}
	TDynamicMeshVertexAttributeExt(UE::Geometry::FDynamicMesh3* ParentIn, bool bAutoInit = true) : Super(ParentIn, bAutoInit) {}

	inline AttribValueType GetValue(int32 VertexID) const
	{
		AttribValueType Value = (AttribValueType)0;
		Super::GetValue(VertexID, &Value);
		return Value;
	}

	inline void SetValue(int32 VertexID, const AttribValueType& NewValue)
	{
		Super::SetValue(VertexID, &NewValue);
	}

	template <typename Predicate, typename Iteratable>
	TArray<int32> FindAllByPredicate(Predicate Pred, const Iteratable& InIteratable) const
	{
		TArray<int32> Result;

		if (Super::Parent)
		{
			Result.Reserve(Super::Parent->VertexCount());

			for (int32 VertexID : InIteratable)
			{
				check(Super::Parent->IsVertex(VertexID));

				const AttribValueType Value = GetValue(VertexID);
				if (Pred(Value))
				{
					Result.Add(VertexID);
				}
			}
		}

		return Result;
	}

	TArray<int32> FindAllNonZero() const
	{
		if (!Super::Parent)
		{
			return TArray<int32>();
		}

		return FindAllByPredicate(
			[](const auto& Value) { return Value != (AttribValueType)0; },
			Super::Parent->VertexIndicesItr()
		);
	}

	TArray<int32> FindAllZero() const
	{
		if (!Super::Parent)
		{
			return TArray<int32>();
		}

		return FindAllByPredicate(
			[](const auto& Value) { return Value == (AttribValueType)0; },
			Super::Parent->VertexIndicesItr()
		);
	}

	TArray<int32> FindAllNonZero(const TArray<int32>& InVertexIDsToSearch) const
	{
		return FindAllByPredicate(
			[](const auto& Value) { return Value != (AttribValueType)0; },
			InVertexIDsToSearch
		);
	}

	TArray<int32> FindAllZero(const TArray<int32>& InVertexIDsToSearch) const
	{
		return FindAllByPredicate(
			[](const auto& Value) { return Value == (AttribValueType)0; },
			InVertexIDsToSearch
		);
	}
};

template<typename T>
class TDynamicMeshVertexAttributeDefinition
{
public:
	using FAttributeType = TDynamicMeshVertexAttributeExt<T>;

	const FName AttributeName;

	TDynamicMeshVertexAttributeDefinition(FName InAttributeName)
		: AttributeName(InAttributeName)
	{
	}

	const FAttributeType* GetAttribute(const UE::Geometry::FDynamicMesh3& DynamicMesh) const
	{
		if (DynamicMesh.HasAttributes() && DynamicMesh.Attributes()->HasAttachedAttribute(AttributeName))
		{
			return static_cast<const FAttributeType*>(DynamicMesh.Attributes()->GetAttachedAttribute(AttributeName));
		}
		return nullptr;
	}

	FAttributeType* GetAttribute(UE::Geometry::FDynamicMesh3& DynamicMesh) const
	{
		if (DynamicMesh.HasAttributes() && DynamicMesh.Attributes()->HasAttachedAttribute(AttributeName))
		{
			return static_cast<FAttributeType*>(DynamicMesh.Attributes()->GetAttachedAttribute(AttributeName));
		}
		return nullptr;
	}

	const FAttributeType& GetAttributeChecked(const UE::Geometry::FDynamicMesh3& DynamicMesh) const
	{
		const FAttributeType* Attribute = GetAttribute(DynamicMesh);
		check(Attribute != nullptr);
		return *Attribute;
	}

	FAttributeType& GetAttributeChecked(UE::Geometry::FDynamicMesh3& DynamicMesh) const
	{
		FAttributeType* Attribute = GetAttribute(DynamicMesh);
		check(Attribute != nullptr);
		return *Attribute;
	}

	FAttributeType& GetOrAttachAttribute(UE::Geometry::FDynamicMesh3& DynamicMesh) const
	{
		FAttributeType* Attribute = GetAttribute(DynamicMesh);
		if (Attribute == nullptr)
		{
			AttachAttribute(DynamicMesh);
		}

		return GetAttributeChecked(DynamicMesh);
	}

	void AttachAttribute(UE::Geometry::FDynamicMesh3& DynamicMesh) const
	{
		if (!DynamicMesh.HasAttributes())
		{
			DynamicMesh.EnableAttributes();
		}

		if (!DynamicMesh.Attributes()->HasAttachedAttribute(AttributeName))
		{
			auto* Attribute = new FAttributeType(&DynamicMesh);
			Attribute->Initialize();
			DynamicMesh.Attributes()->AttachAttribute(AttributeName, Attribute);
		}
	}

	void CopyAttributeToCollection(const UE::Geometry::FDynamicMesh3& DynamicMesh, FManagedArrayCollection& OutCollection) const
	{
		const FAttributeType* AttachedAttribute = GetAttribute(DynamicMesh);
		if (!AttachedAttribute)
		{
			return;
		}

		const static FName VertexAttributesGroupName = TEXT("VertexAttributes");

		if (!OutCollection.HasGroup(VertexAttributesGroupName))
		{
			OutCollection.AddGroup(VertexAttributesGroupName);
			OutCollection.AddElements(DynamicMesh.MaxVertexID(), VertexAttributesGroupName);

			auto& DebugVertexPositionAttribute = OutCollection.AddAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, VertexAttributesGroupName);
			for (int32 VertexIndex : DynamicMesh.VertexIndicesItr())
			{
				DebugVertexPositionAttribute[VertexIndex] = FVector3f(DynamicMesh.GetVertex(VertexIndex));
			}
		}

		TManagedArray<T>& CollectionAttribute = OutCollection.AddAttribute<T>(AttributeName, VertexAttributesGroupName);
		check(CollectionAttribute.Num() >= DynamicMesh.MaxVertexID());

		for (int32 VertexIndex : DynamicMesh.VertexIndicesItr())
		{
			CollectionAttribute[VertexIndex] = AttachedAttribute->GetValue(VertexIndex);
		}
	}
};
}