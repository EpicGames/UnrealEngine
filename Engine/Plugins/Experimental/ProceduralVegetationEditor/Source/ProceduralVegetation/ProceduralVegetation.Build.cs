// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProceduralVegetation : ModuleRules
	{
		public ProceduralVegetation(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"DynamicWind"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Chaos",
					"ChaosCore",
					"Core",
					"CoreUObject",
					"Engine",
					"GeometryCore",
					"GeometryFramework",
					"Json",
					"JsonUtilities",
					"MeshDescription",
					"MeshConversionEngineTypes",
					"ModelingComponents",
					"PCG",
					"RenderCore",
					"StaticMeshDescription",
					"RHI",
					"Projects",
					"ImageCore"
				}
			);
		}
	}
}