// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"

#include "Data/PCGSpatialData.h"
#include "Engine/EngineTypes.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "PVData.generated.h"

struct FPCGContext;
class FGeometryCollection;
class FTransformCollection;

UENUM()
enum class EPVRenderType : uint16
{
	None 				= 0,
	PointData 			= 1 << 0,
	Mesh 				= 1 << 1,
	Foliage 			= 1 << 2,
	Bones 				= 1 << 3,
	FoliageGrid 		= 1 << 4,
	FoliageAttachments	= 1 << 5,
	Seed				= 1 << 6,
	Textures			= 1 << 7,
	Leaf 				= 1 << 8,
	GrafterGrid			= 1 << 9,
	PlantProfile		= 1 << 10,
	BoundingBoxOnly		= 1 << 11,

	// Add new renter type above
	Count
};

ENUM_CLASS_FLAGS(EPVRenderType)

UENUM()
enum class EPVDebugType
{
	Point,
	Branches,
	Foliage,
	Custom
};

UENUM()
enum class EPVDebugValueVisualizationMode
{
	Text,
	Direction,
	Sphere,
	Vector,
	Point,
	Curve,
	ColorRamp
};

USTRUCT()
struct FPVVisualizationSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	EPVDebugType DebugType = EPVDebugType::Point;

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	bool bShow = true;

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	bool bShowAnchorPoints = true;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "VisualizationMode==EPVDebugValueVisualizationMode::Point || bShowAnchorPoints"))
	bool bDrawPointAsMesh = true;
	
	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "VisualizationMode==EPVDebugValueVisualizationMode::Sphere"))
	bool bDrawSphereAsMesh = false;

	// If true, then the AttributeToFilter will be used as point scale and the CustomPivotPositionAttributeName will be used for the point position. If false then AttributeToFilter is used for position and pivot attribute is ignored. 
	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "DebugType==EPVDebugType::Custom && VisualizationMode==EPVDebugValueVisualizationMode::Point"))
	bool bUsePivotAsPosition = false;

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	FName AttributeToFilter;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "DebugType==EPVDebugType::Custom"))
	FName CustomGroupToFilter = TEXT("Custom");

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "DebugType==EPVDebugType::Custom && VisualizationMode != EPVDebugValueVisualizationMode::Curve && (VisualizationMode != EPVDebugValueVisualizationMode::Point || bUsePivotAsPosition)"))
	FName CustomPivotPositionAttributeName = TEXT("Position");

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "DebugType==EPVDebugType::Custom && VisualizationMode != EPVDebugValueVisualizationMode::Curve && (VisualizationMode != EPVDebugValueVisualizationMode::Point || bUsePivotAsPosition)"))
	FName CustomPivotScaleAttributeName = TEXT("Scale");

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	EPVDebugValueVisualizationMode VisualizationMode = EPVDebugValueVisualizationMode::Text;

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	bool bRandomizeColors = false;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "!bRandomizeColors"))
	FColor Color = FColor::Black;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "bShowAnchorPoints"))
	float AnchorPointsScale = 1.f;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "VisualizationMode==EPVDebugValueVisualizationMode::Text"))
	float TextScale = 2.5f;

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta = (EditCondition = "VisualizationMode==EPVDebugValueVisualizationMode::Text"))
	FVector3f TextOffset = FVector3f::Zero();

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	float GizmoScale = 1.f;

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriorityGroup = ESceneDepthPriorityGroup::SDPG_World;
	
	FName GetGroupToFilter() const
	{
		switch (DebugType)
		{
		case EPVDebugType::Point:
			return "Points";

		case EPVDebugType::Branches:
			return "Primitives";

		case EPVDebugType::Foliage:
			return "Foliage";

		case EPVDebugType::Custom:
			return CustomGroupToFilter;
		}
		return NAME_None;
	}
};

USTRUCT()
struct FPVParamDebugData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = DebugData)
	FVector Pivot = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = DebugData)
	TOptional<FVector> Direction;

	UPROPERTY(EditAnywhere, Category = DebugData)
	TOptional<FText> TextData;
};

USTRUCT()
struct FPVParamDebuggerSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = LoopDebugSettings)
	EPVDebugValueVisualizationMode VisualizationMode = EPVDebugValueVisualizationMode::Text;

	UPROPERTY(EditAnywhere, Category = LoopDebugSettings)
	FText ParamName;

	UPROPERTY(EditAnywhere, Category = LoopDebugSettings)
	FPVParamDebugData Data;
};

USTRUCT()
struct FPVDebugSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DebugSettings)
	TArray<FPVVisualizationSettings> VisualizationSettings;

	UPROPERTY()
	TArray<FPVParamDebuggerSettings> ParamDebugVisualizationSettings;
	
	bool bAutoFocusLoopDebug = false;
};

struct FPVVisualizationCollection
{
	FManagedArrayCollection Collection;
	TArray<FPVVisualizationSettings> VisualizationSettings;
};

USTRUCT()
struct FPVSettingsVisualization
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DebugSettings, meta=(EditConditionHides))
	FPVDebugSettings DebugSettings;

	UPROPERTY()
	TArray<EPVRenderType> CurrentRenderType;
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class PROCEDURALVEGETATION_API UPVData : public UPCGSpatialData
{
	GENERATED_BODY()

public:
	UPVData(const FObjectInitializer& ObjectInitializer);
	
	void Initialize(FManagedArrayCollection&& InCollection);
	void Initialize(FGeometryCollection&&) = delete;
	void Initialize(FTransformCollection&&) = delete;
	
	// ~Begin UPCGData interface
	virtual EPCGDataType GetDataType() const override { return EPCGDataType::Other; }
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	virtual int GetDimension() const override { return 3; }
	virtual FBox GetBounds() const override;
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	//~End UPCGSpatialData interface

	const FManagedArrayCollection& GetCollection() const { return *Collection; }
	FManagedArrayCollection& GetCollection() { return *Collection; }
	TSharedPtr<FManagedArrayCollection>& GetSharedCollection() { return Collection; }
	const TSharedPtr<FManagedArrayCollection>& GetSharedCollection() const { return Collection; }

	void AddManagedResource(const UObject* InResource) 
	{ 
		if (InResource)
		{
			ManagedResources.AddUnique(InResource);
		}
	}

#if WITH_EDITORONLY_DATA
	void AddVisualizationCollection(FManagedArrayCollection InCollection, TArray<FPVVisualizationSettings> InVisualizationSettings = {});
	void SetDebugSettings(const FPVDebugSettings& InSettings) const;
	void SetDebugSettings(FPVDebugSettings&& InSettings) const;
	FPVDebugSettings GetDebugSettings() const;
	const TArray<FPVVisualizationCollection>& GetVisualizationCollections() const { return VisualizationCollections; }
#endif
	
protected:
	
	// ~Begin UPCGData interface
	virtual bool SupportsFullDataCrc() const override { return true; }
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
public:
	virtual const UPCGPointData* ToPointData(FPCGContext* Context, const FBox& InBounds) const override;
	virtual const UPCGPointArrayData* ToPointArrayData(FPCGContext* Context, const FBox& InBounds) const override;
	//~End UPCGSpatialData interface
	
private:

	const UPCGBasePointData* ToBasePointData(FPCGContext* Context, const FBox& InBounds, TSubclassOf<UPCGBasePointData> PointDataClass) const;

protected:
	
	TSharedPtr<FManagedArrayCollection> Collection;

	UPROPERTY(Transient)
	TArray<TObjectPtr<const UObject>> ManagedResources;

#if WITH_EDITOR
	mutable FCriticalSection DebugSettingsMutex;
	mutable FPVDebugSettings DebugSettings;

	TArray<FPVVisualizationCollection> VisualizationCollections;
#endif
};
