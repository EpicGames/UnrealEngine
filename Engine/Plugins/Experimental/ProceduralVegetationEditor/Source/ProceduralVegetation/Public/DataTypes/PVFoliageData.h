// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PVData.h"
#include "PVFoliageData.generated.h"

USTRUCT()
struct FPVDataTypeInfoFoliage : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVFoliageData : public UPVData
{
	GENERATED_BODY()

public:
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoFoliage)
	//~ End UPCGData interface
};