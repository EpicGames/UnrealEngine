// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/SoftObjectPtr.h"

#include "PVFoliageInfo.generated.h"

USTRUCT()
struct FPVDistributionConditions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Info", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preferred light exposure at the placement point. 0 = shaded, 1 = fully lit."))
	float Light = 0.0f;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preferred normalized branch scale at the placement point. 0 = thin/small, 1 = thick/large."))
	float Scale = 0.0f;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preferred alignment of the branch with world up. 0 = horizontal, 1 = vertical."))
	float UpAlignment = 0.0f;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preferred branch health at the placement point. 0 = unhealthy, 1 = healthy."))
	float Health = 0.0f;

	UPROPERTY(EditAnywhere, Category="Info", meta=(Tooltip="When enabled, this entry is only placed on branch tips."))
	bool Tip = false;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preferred normalized height along the plant. 0 = base, 1 = top."))
	float Height = 0.0f;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ClampMin=0.0f, ClampMax=1.0f, UIMin=0.0f, UIMax=1.0f, Tooltip="Preferred branch generation (sub-branch depth) at the placement point. 0 = trunk, 1 = furthest sub-branches."))
	float Generation = 0.0f;

	FString ToString() const
	{
		return FString::Printf(
			TEXT("Light = %f, Scale = %f, UpAlignment = %f, Health = %f, Tip = %s, Height = %f, Generation = %f"),
			Light,
			Scale,
			UpAlignment,
			Health,
			Tip ? TEXT("True") : TEXT("False"),
			Height,
			Generation
		);
	}
};

USTRUCT()
struct FPVFoliageInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Info", meta=(Tooltip="When enabled, this entry acts as a placement mask only — no mesh is spawned. Use to reserve slots driven by distribution conditions."))
	bool bUseAsMask = false;
	
	UPROPERTY(EditAnywhere, Category = "Info", meta = (EditCondition="!bUseAsMask", EditConditionHides, AllowedClasses = "/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh", Tooltip="Static Mesh or Skeletal Mesh placed at points matched by this entry's Attributes."))
	TSoftObjectPtr<UObject> Mesh;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ShowInnerProperties, FullyExpand="true", Tooltip="Target values (0–1) used by the Foliage Distributor to pick this entry: the distributor selects the foliage whose Attributes are closest to each point's sampled values."))
	FPVDistributionConditions Attributes;

	bool IsValid() const
	{
		// Asset is loaded and pointer is valid
		// Path is valid, but asset is NOT loaded yet
		return Mesh.IsValid() || Mesh.ToSoftObjectPath().IsValid();
	}
};
