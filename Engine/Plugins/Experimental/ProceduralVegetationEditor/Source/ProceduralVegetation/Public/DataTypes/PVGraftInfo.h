// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVFoliageInfo.h"

#include "PVGraftInfo.generated.h"

USTRUCT()
struct PROCEDURALVEGETATION_API FPVGraftInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Info", meta=(Tooltip="When enabled, this entry acts as a placement mask only — no graft is spawned and its input pin is hidden. Use to block attachment points matching these conditions without spawning a graft"))
	bool bUseAsMask = false;

	UPROPERTY(EditAnywhere, Category="Info", meta=(ShowInnerProperties, FullyExpand="true", Tooltip="Target values (0–1) used by the Graft Distributor to pick this entry: the distributor selects the entry whose Attributes are closest to each attachment point's sampled values."))
	FPVDistributionConditions Attributes;
};
