// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PVData.h"
#include "PVGrowthData.h"
#include "PVGrafterPaletteData.generated.h"

USTRUCT()
struct FPVDataTypeInfoGrafterPalette : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrafterPaletteData : public UPVData
{
	GENERATED_BODY()

public:
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrafterPalette)
	//~ End UPCGData interface

	void Initialize(TArray<TObjectPtr<UPVGrowthData>>&& InGrowthDataElements);
	// Prevent callers from using the base initializer, which would leave GrowthDataElements unpopulated.
	void Initialize(FManagedArrayCollection&&) = delete;
	
	// ~Begin UPCGData interface
	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	// ~End UPCGData interface
	
	const TArray<TObjectPtr<UPVGrowthData>>& GetGrowthDataElements() const { return GrowthDataElements; }
	TArray<TObjectPtr<UPVGrowthData>>& GetGrowthDataElements() { return GrowthDataElements; }
	
protected:
	// ~Begin UPCGData interface
	virtual bool SupportsFullDataCrc() const override { return true; }
	// ~End UPCGData interface

	// ~Begin UPCGSpatialData interface
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	// ~End UPCGSpatialData interface
	
	UPROPERTY()
	TArray<TObjectPtr<UPVGrowthData>> GrowthDataElements;
};