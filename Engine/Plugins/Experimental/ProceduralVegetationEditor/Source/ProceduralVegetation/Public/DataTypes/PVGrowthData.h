// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PVData.h"
#include "Nodes/PVObjectInteractionSettings.h"
#include "Serialization/ArchiveCrc32.h"
#include "PVGrowthData.generated.h"

USTRUCT()
struct FPVDataTypeInfoGrowth : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVGrowthData : public UPVData
{
	GENERATED_BODY()

public:
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoGrowth)
	//~ End UPCGData interface
};