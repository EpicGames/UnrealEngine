// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PVData.h"
#include "PVTrunkTextureSetupData.generated.h"

class UMaterialInterface;
class UTexture;

UENUM()
enum class EPVTextureChannel : uint8
{
	BaseColor,
	AO,
	Normal,
	Displacement,
	Bump,
	Cavity,
	Gloss,
	Roughness,
	Specular,
	Custom,
};

USTRUCT()
struct FPVGenerationUVRange
{
	GENERATED_BODY()
	
	UPROPERTY()
	float OffsetXStart = 0.0f;

	UPROPERTY()
	float OffsetXEnd = 1.0f;

	UPROPERTY()
	float DilationFactor = 0.0f;

	inline float GetOverflowValue() const {return (OffsetXEnd + DilationFactor) - 1.0;}
};

USTRUCT()
struct FPVTrunkTextureSetupInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, TObjectPtr<UTexture>> Channels;

	UPROPERTY()
	TArray<FPVGenerationUVRange> GenerationUVs;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY()
	int Resolution = 1024;

	UPROPERTY()
	FString PreviewChannelName;

	bool IsOverflowing() const;
};

USTRUCT()
struct FPVDataTypeInfoTrunkTextureSetup : public FPCGDataTypeInfo
{
	GENERATED_BODY()

	PCG_DECLARE_TYPE_INFO(PROCEDURALVEGETATION_API);

#if WITH_EDITOR
	// If the type should be hidden to the user.
	virtual bool Hidden() const override { return true; };
#endif // WITH_EDITOR
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVTrunkTextureSetupData : public UPVData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FPVTrunkTextureSetupInfo TrunkTextureSetupInfo;
	//~ Begin UPCGData interface
	PCG_ASSIGN_DEFAULT_TYPE_INFO(FPVDataTypeInfoTrunkTextureSetup)

	virtual void AddToCrc(FArchiveCrc32& Ar, bool bFullDataCrc) const override;
	
	//~ End UPCGData interface
};