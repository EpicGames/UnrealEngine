// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace PV::Facades
{
	/**
	 * Any Facade can implement IShrinkable to provide capability of removing arbitrary elements from the underlying
	 * Collection/Group in an optimal way.
	 */
	class PROCEDURALVEGETATION_API IShrinkable
	{
	public:
		virtual ~IShrinkable() = default;

		virtual int32 GetElementCount() const = 0;

		virtual void CopyEntry(int32 FromIndex, int32 ToIndex) = 0;

		virtual void RemoveEntries(int32 NumEntries, int32 StartIndex) = 0;

		static TMap<int32, int32> RemoveEntries(IShrinkable& FacadeOut, TArray<bool>& EntriesToRemove)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PV::Facades::IShrinkable::RemoveEntries);

			check(FacadeOut.GetElementCount() == EntriesToRemove.Num());

			TMap<int32, int32> OldIDsToNewIDs;
			int32 ToRemoveIndex = 0;
			int32 ToCopyIndex;

			do
			{
				// Find a slot to remove/fill
				while (ToRemoveIndex < EntriesToRemove.Num() && !EntriesToRemove[ToRemoveIndex]) { ToRemoveIndex++; }

				// Find the next slot to fill the empty one with
				ToCopyIndex = ToRemoveIndex;
				while (ToCopyIndex < EntriesToRemove.Num() && EntriesToRemove[ToCopyIndex]) { ToCopyIndex++; }

				if (ToCopyIndex < EntriesToRemove.Num())
				{
					FacadeOut.CopyEntry(ToCopyIndex, ToRemoveIndex);
					EntriesToRemove[ToCopyIndex] = true;

					OldIDsToNewIDs.Emplace(ToCopyIndex, ToRemoveIndex);
					ToRemoveIndex++;
				}
			} while (ToRemoveIndex < EntriesToRemove.Num() && ToCopyIndex < EntriesToRemove.Num());

			// In case there was nothing to remove
			if (ToRemoveIndex == EntriesToRemove.Num())
			{
				return OldIDsToNewIDs;
			}

			FacadeOut.RemoveEntries(FacadeOut.GetElementCount() - ToRemoveIndex, ToRemoveIndex);

			return OldIDsToNewIDs;
		}
	};
};
