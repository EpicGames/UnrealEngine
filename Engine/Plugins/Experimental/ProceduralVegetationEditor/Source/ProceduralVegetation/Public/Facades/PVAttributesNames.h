// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace PV::GroupNames
{
	//Group Names
	inline constexpr TCHAR BranchGroupStr[] = TEXT("Primitives");
	inline constexpr TCHAR PointGroupStr[] = TEXT("Points");
	inline constexpr TCHAR FoliageGroupStr[] = TEXT("Foliage");
	inline constexpr TCHAR FoliageNamesGroupStr[] = TEXT("FoliageNames");
	inline constexpr TCHAR DetailsGroupStr[] = TEXT("Details");
	inline constexpr TCHAR PlantProfilesGroupStr[] = TEXT("PlantProfiles");
	inline constexpr TCHAR BonesGroupStr[] = TEXT("Bones");
	inline constexpr TCHAR VerticesGroupStr[] = TEXT("Vertices");

	inline static const FName BranchGroup = FName(BranchGroupStr);
	inline static const FName PointGroup = FName(PointGroupStr);
	inline static const FName FoliageGroup = FName(FoliageGroupStr);
	inline static const FName FoliageNamesGroup = FName(FoliageNamesGroupStr);
	inline static const FName DetailsGroup = FName(DetailsGroupStr);
	inline static const FName PlantProfilesGroup = FName(PlantProfilesGroupStr);
	inline static const FName BonesGroup = FName(BonesGroupStr);
	inline static const FName VerticesGroup = FName(VerticesGroupStr);

	// Leaf instance groups (final-cycle spawn data written by FPVGrower::FillCollection)
	inline constexpr TCHAR LeafProxyMetaGroupStr[] = TEXT("LeafProxyMeta");
	inline constexpr TCHAR LeafProxyGroupStr[] = TEXT("LeafProxy");
	inline static const FName LeafMetaGroup = FName(LeafProxyMetaGroupStr);
	inline static const FName LeavesGroup = FName(LeafProxyGroupStr);
}

namespace PV::AttributeNames
{
	//Branch Attributes
	inline constexpr TCHAR BranchParentsStr[] = TEXT("parents");
	inline constexpr TCHAR BranchChildrenStr[] = TEXT("children");
	inline constexpr TCHAR BranchPointsStr[] = TEXT("Points");
	inline constexpr TCHAR BranchNumberStr[] = TEXT("branchNumber");
	inline constexpr TCHAR BranchSourceBudNumberStr[] = TEXT("branchSourceBudNumber");
	inline constexpr TCHAR BranchFoliageIDsStr[] = TEXT("FoliageID");
	inline constexpr TCHAR BranchUVMaterialStr[] = TEXT("branchUVMat");
	inline constexpr TCHAR TrunkMaterialPathStr[] = TEXT("TrunkMaterialPath");
	inline constexpr TCHAR TrunkURangeStr[] = TEXT("TrunkURange");
	inline constexpr TCHAR BranchHierarchyNumberStr[] = TEXT("branchHierarchyNumber");
	inline constexpr TCHAR BranchSimulationGroupIndexStr[] = TEXT("branchSimulationGroupIndex");
	inline constexpr TCHAR PlantNumberStr[] = TEXT("plantNumber");
	inline constexpr TCHAR BranchParentNumberStr[] = TEXT("branchParentNumber");

	inline static const FName BranchParents = FName(BranchParentsStr);
	inline static const FName BranchChildren = FName(BranchChildrenStr);
	inline static const FName BranchPoints = FName(BranchPointsStr);
	inline static const FName BranchNumber = FName(BranchNumberStr);
	inline static const FName BranchSourceBudNumber = FName(BranchSourceBudNumberStr);
	inline static const FName BranchFoliageIDs = FName(BranchFoliageIDsStr);
	inline static const FName BranchUVMaterial = FName(BranchUVMaterialStr);
	inline static const FName TrunkMaterialPath = FName(TrunkMaterialPathStr);
	inline static const FName TrunkURange = FName(TrunkURangeStr);
	inline static const FName BranchHierarchyNumber = FName(BranchHierarchyNumberStr);
	inline static const FName BranchSimulationGroupIndex = FName(BranchSimulationGroupIndexStr);
	inline static const FName PlantNumber = FName(PlantNumberStr);
	inline static const FName BranchParentNumber = FName(BranchParentNumberStr);

	//Point Attributes
	inline constexpr TCHAR PointPositionStr[] = TEXT("Position");
	inline constexpr TCHAR LengthFromRootStr[] = TEXT("lengthFromRoot");
	inline constexpr TCHAR PointScaleGradientStr[] = TEXT("LOD_totalPscaleGradient");
	inline constexpr TCHAR PointScaleStr[] = TEXT("Scale");
	inline constexpr TCHAR LengthFromSeedStr[] = TEXT("lengthFromSeed");
	inline constexpr TCHAR SeedPScaleStr[] = TEXT("seedPScale");
	inline constexpr TCHAR SeedPScaleRatioStr[] = TEXT("seedPScaleRatio");
	inline constexpr TCHAR BudLightDetectedStr[] = TEXT("budLightDetected");
	inline constexpr TCHAR BudDevelopmentStr[] = TEXT("budDevelopment");
	inline constexpr TCHAR BudLateralMeristemStr[] = TEXT("budLateralMeristem");
	inline constexpr TCHAR TextureCoordVStr[] = TEXT("uv_v");
	inline constexpr TCHAR TextureCoordUOffsetStr[] = TEXT("uv_uOffset");
	inline constexpr TCHAR URangeStr[] = TEXT("uv_uRange");
	inline constexpr TCHAR HullGradientStr[] = TEXT("LOD_hullGradient");
	inline constexpr TCHAR MainTrunkGradientStr[] = TEXT("LOD_mainTrunkGradient");
	inline constexpr TCHAR GroundGradientStr[] = TEXT("LOD_groundGradient");
	inline constexpr TCHAR BudNumberStr[] = TEXT("budNumber");
	inline constexpr TCHAR BudHormoneLevelsStr[] = TEXT("budHormoneLevels");
	inline constexpr TCHAR BudDirectionStr[] = TEXT("budDirection");
	inline constexpr TCHAR BudStatusStr[] = TEXT("budStatus");
	inline constexpr TCHAR PlantGradientStr[] = TEXT("plantGradient");
	inline constexpr TCHAR NjordPixelIndexStr[] = TEXT("njord_pixelIdx");

	inline static const FName PointPosition = FName(PointPositionStr);
	inline static const FName LengthFromRoot = FName(LengthFromRootStr);
	inline static const FName PointScaleGradient = FName(PointScaleGradientStr);
	inline static const FName PointScale = FName(PointScaleStr);
	inline static const FName LengthFromSeed = FName(LengthFromSeedStr);
	inline static const FName BudLightDetected = FName(BudLightDetectedStr);
	inline static const FName BudDevelopment = FName(BudDevelopmentStr);
	inline static const FName BudLateralMeristem = FName(BudLateralMeristemStr);
	inline static const FName TextureCoordV = FName(TextureCoordVStr);
	inline static const FName TextureCoordUOffset = FName(TextureCoordUOffsetStr);
	inline static const FName URange = FName(URangeStr);
	inline static const FName HullGradient = FName(HullGradientStr);
	inline static const FName MainTrunkGradient = FName(MainTrunkGradientStr);
	inline static const FName GroundGradient = FName(GroundGradientStr);
	inline static const FName BudNumber = FName(BudNumberStr);
	inline static const FName BudHormoneLevels = FName(BudHormoneLevelsStr);
	inline static const FName BudDirection = FName(BudDirectionStr);
	inline static const FName BudStatus = FName(BudStatusStr);
	inline static const FName PlantGradient = FName(PlantGradientStr);
	inline static const FName NjordPixelIndex = FName(NjordPixelIndexStr);

	inline static const FName SeedPScale = FName(SeedPScaleStr);
	inline static const FName SeedPScaleRatio = FName(SeedPScaleRatioStr);
	

	//Foliage Attributes
	inline constexpr TCHAR FoliageNameIDStr[] = TEXT("FoliageNameID");
	inline constexpr TCHAR FoliageBranchIDStr[] = TEXT("FoliageBranchID");
	inline constexpr TCHAR FoliagePivotPointStr[] = TEXT("FoliagePivotPoint");
	inline constexpr TCHAR FoliageUPVectorStr[] = TEXT("FoliageUPVector");
	inline constexpr TCHAR FoliageNormalVectorStr[] = TEXT("FoliageNormalVector");
	inline constexpr TCHAR FoliageScaleStr[] = TEXT("FoliageScale");
	inline constexpr TCHAR FoliageLengthFromRootStr[] = TEXT("LengthFromRoot");
	inline constexpr TCHAR FoliageParentBoneIDStr[] = TEXT("FoliageParentBoneID");

	inline static const FName FoliageNameID = FName(FoliageNameIDStr);
	inline static const FName FoliageBranchID = FName(FoliageBranchIDStr);
	inline static const FName FoliagePivotPoint = FName(FoliagePivotPointStr);
	inline static const FName FoliageUPVector = FName(FoliageUPVectorStr);
	inline static const FName FoliageNormalVector = FName(FoliageNormalVectorStr);
	inline static const FName FoliageScale = FName(FoliageScaleStr);
	inline static const FName FoliageLengthFromRoot = FName(FoliageLengthFromRootStr);
	inline static const FName FoliageParentBoneID = FName(FoliageParentBoneIDStr);

	//FoliageNames Attributes
	inline constexpr TCHAR FoliageNameStr[] = TEXT("Name");

	inline static const FName FoliageName = FName(FoliageNameStr);

	//Details Attributes
	inline constexpr TCHAR LeafPhyllotaxyStr[] = TEXT("phyllotaxyLeaf");
	inline constexpr TCHAR FoliagePathStr[] = TEXT("FoliagePath");
	inline constexpr TCHAR GuidStr[] = TEXT("Guid");
	inline constexpr TCHAR LeafGrowthStr[] = TEXT("leafGrowth");
	inline constexpr TCHAR AbscissionSenescenseStr[] = TEXT("abscissionSenescense");
	inline constexpr TCHAR LateralElongationStr[] = TEXT("lateralElongation");
	inline constexpr TCHAR GraftUseAsMaskStr[] = TEXT("GraftUseAsMask");
	inline constexpr TCHAR GraftLightStr[] = TEXT("GraftLight");
	inline constexpr TCHAR GraftScaleStr[] = TEXT("GraftScale");
	inline constexpr TCHAR GraftTipStr[] = TEXT("GraftTip");
	inline constexpr TCHAR GraftUpAlignmentStr[] = TEXT("GraftUpAlignment");
	inline constexpr TCHAR GraftHealthStr[] = TEXT("GraftHealth");
	inline constexpr TCHAR GraftHeightStr[] = TEXT("GraftHeight");
	inline constexpr TCHAR GraftGenerationStr[] = TEXT("GraftGeneration");

	inline static const FName LeafPhyllotaxy = FName(LeafPhyllotaxyStr);
	inline static const FName FoliagePath = FName(FoliagePathStr);
	inline static const FName Guid = FName(GuidStr);
	inline static const FName LeafGrowth = FName(LeafGrowthStr);
	inline static const FName AbscissionSenescense = FName(AbscissionSenescenseStr);
	inline static const FName LateralElongation = FName(LateralElongationStr);
	inline static const FName GraftUseAsMask = FName(GraftUseAsMaskStr);
	inline static const FName GraftLight = FName(GraftLightStr);
	inline static const FName GraftScale = FName(GraftScaleStr);
	inline static const FName GraftTip = FName(GraftTipStr);
	inline static const FName GraftUpAlignment = FName(GraftUpAlignmentStr);
	inline static const FName GraftHealth = FName(GraftHealthStr);
	inline static const FName GraftHeight = FName(GraftHeightStr);
	inline static const FName GraftGeneration = FName(GraftGenerationStr);

	//PlantProfiles Attributes
	inline constexpr TCHAR ProfilePointsStr[] = TEXT("ProfilePoints");

	inline static const FName ProfilePoints = FName(ProfilePointsStr);

	//Bone Attributes
	inline constexpr TCHAR BoneNameStr[] = TEXT("BoneName");
	inline constexpr TCHAR BoneParentIndexStr[] = TEXT("BoneParentIndex");
	inline constexpr TCHAR BonePoseStr[] = TEXT("BonePose");
	inline constexpr TCHAR BonePointIndexStr[] = TEXT("BonePointIndex");
	inline constexpr TCHAR BoneAbsolutePositionStr[] = TEXT("BoneAbsolutePosition");
	inline constexpr TCHAR BoneBranchIndexStr[] = TEXT("BoneBranchIndex");
	inline constexpr TCHAR NjordPixelIdStr[] = TEXT("NjordPixelId");
	inline constexpr TCHAR VertexPointIdsStr[] = TEXT("VertexPointIds");
	inline constexpr TCHAR BoneIdStr[] = TEXT("BoneId");

	inline static const FName BoneName = FName(BoneNameStr);
	inline static const FName BoneParentIndex = FName(BoneParentIndexStr);
	inline static const FName BonePose = FName(BonePoseStr);
	inline static const FName BonePointIndex = FName(BonePointIndexStr);
	inline static const FName BoneAbsolutePosition = FName(BoneAbsolutePositionStr);
	inline static const FName BoneBranchIndex = FName(BoneBranchIndexStr);
	inline static const FName NjordPixelId = FName(NjordPixelIdStr);
	inline static const FName VertexPointIds = FName(VertexPointIdsStr);
	inline static const FName BoneId = FName(BoneIdStr);

	// Leaf Meta Attributes
	inline constexpr TCHAR LeafMeshPathStr[] = TEXT("MeshPath");
	inline static const FName LeafMeshPath = FName(LeafMeshPathStr);

	// Leaves Attributes (per-leaf transform; Position reuses PointPositionStr, Scale reuses PointScaleStr)
	inline constexpr TCHAR LeafRotationStr[] = TEXT("Rotation");
	inline static const FName LeafRotation = FName(LeafRotationStr);
}

