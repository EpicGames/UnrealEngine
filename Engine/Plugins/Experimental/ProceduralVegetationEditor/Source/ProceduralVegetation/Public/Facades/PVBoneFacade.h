// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
	/**
	 * FBoneNode contains the bone data and linked point and branch ids
	 */
	struct PROCEDURALVEGETATION_API FBoneNode
	{
		FName BoneName = NAME_None;
		int32 BoneIndex = INDEX_NONE;
		int32 ParentBoneIndex = INDEX_NONE;
		int32 PointIndex = INDEX_NONE;
		int32 BranchIndex = INDEX_NONE;
		float NjordPixelID = 0.0;

		FTransform BoneTransform{};
		FVector AbsolutePosition = FVector::ZeroVector;
	};
	
	/**
	 * FBoneFacade is used to access and manipulate the Bone data from Bones Group with in the ProceduralVegetation's FManagedArrayCollection.
	 * NjordPixel id here is stored in vertex Group and is used for bone assignment
	 */
	class PROCEDURALVEGETATION_API FBoneFacade
	{
	public:
		FBoneFacade(FManagedArrayCollection& InCollection);
		
		FBoneFacade(const FManagedArrayCollection& InCollection);

		static void DefineSchema(FManagedArrayCollection& InCollection);
		
		bool IsValid() const;

		void GetPointIds(TArray<int32>& OutVertexPointIds) const;
		
		const TManagedArray<FString>& GetBoneNames() const;
		
		const TManagedArray<int32>& GetBoneParentIndices() const;
		
		const TManagedArray<FTransform>& GetBonePoses() const;

		const TManagedArray<int32>& GetVertexPointIds() const;
		
		TManagedArray<int32>& ModifyVertexPointIds();

		void SetWindSimulationData(FManagedArrayCollection& Collection);
		
		bool CreateBoneData(TArray<FBoneNode>& Bones,const float ReductionStrength,bool bSetWindData = true);

		static FBoneNode* FindClosestBone(const FManagedArrayCollection& Collection, TArray<FBoneNode>& Bones, const int32 PointID);

		TArray<FBoneNode> GetBoneDataFromCollection() const;

		void SetBoneDataToCollection(TArray<FBoneNode>& Bones);
		
	private:
		TManagedArrayAccessor<FString> BoneName;
		TManagedArrayAccessor<int32> BoneParentIndex;
		TManagedArrayAccessor<int32> BoneId;
		TManagedArrayAccessor<FTransform> BonePose;
		TManagedArrayAccessor<int32> BonePointIndex;
		TManagedArrayAccessor<FVector3f> BoneAbsolutePosition;
		TManagedArrayAccessor<int32> BoneBranchIndex;
		TManagedArrayAccessor<int32> VertexPointIds;
	};
}
