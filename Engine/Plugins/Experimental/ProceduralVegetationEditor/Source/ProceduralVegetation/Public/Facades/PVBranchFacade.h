// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IShrinkableFacade.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
	/**
	 * FBranchFacade is used to access and manipulate the data from Branch Group with in the ProceduralVegetation's FManagedArrayCollection
	 * A branch is made up of multiple points. Each Branch will know about its parents and children, along with other attributes
	 * Only add the frequently used branch attributes and their access to this facade, for the specific access write a new facade
	 */
	class PROCEDURALVEGETATION_API FBranchFacade final : public IShrinkable
	{
	public:
		FBranchFacade(FManagedArrayCollection& InCollection);
		FBranchFacade(const FManagedArrayCollection& InCollection);

		bool IsConst() const { return Collection == nullptr; }

		bool IsValid() const;

		virtual int32 GetElementCount() const override;

		const TArray<int32>& GetPoints(int32 Index) const;

		int32 GetRootPoint(int32 Index) const;

		void SetPoints(int32 Index, TArray<int32> InPoints);

		const TArray<int32>& GetChildren(int32 Index) const;

		TArray<int32> GetImmediateChildren(int32 Index) const;

		void SetChildren(int32 Index, TArray<int32> InChildren);

		const TArray<int32>& GetParents(int32 Index) const;

		int32 GetParentIndex(int32 BranchIndex) const;

		int32 GetParentBranchNumber(int32 BranchIndex) const;
		
		void SetParentBranchNumber(int32 BranchIndex, int32 InParentBranchNumber);
		
		const TManagedArray<int32>& GetParentBranchNumbers() const;

		void SetParents(int32 Index, TArray<int32> InParents);

		int32 GetBranchNumber(int32 Index) const;

		void SetBranchNumber(int32 Index, int32 InBranchNumber);
		
		const TArray<int32>& GetBranchFoliageIDs(int32 Index) const;
		
		void SetBranchFoliageIDs(int32 Index, TArray<int32> InBranchFoliageIDs);

		int32 GetBranchPlantNumber(int32 Index) const;

		void SetBranchPlantNumber(int32 Index, int32 InPlantNumber);

		int32 GetBranchSourceBudNumber(int32 Index) const;
		
		void SetBranchSourceBudNumber(int32 Index, int32 InSourceBudNumber);

		TArray<int32> GetParentBranchIndices(int BranchIndex) const;
		
		int32 GetParentBranchIndex(int32 BranchIndex) const;

		int32 GetHierarchyGenerationNumber(int32 Index) const;

		const TManagedArray<int32>& GetBranchNumbers() const;

		int32 GetBranchUVMaterial(int32 Index) const;

		void SetBranchUVMaterial(int32 Index, int32 InMaterial);
		
		int32 GetBranchHierarchyNumber(int32 Index) const;
		
		const TManagedArray<int32>& GetBranchHierarchyNumbers() const;

		void SetBranchHierarchyNumber(int32 Index, int32 InBranchHierarchyNumber);

		int32 GetBranchSimulationGroupIndex(int32 Index) const;
		
		void SetBranchSimulationGroupIndex(int32 Index, int32 InLogicalDepth);

		void SetTrunkMaterialPath(const FString& InPath);

		FString GetTrunkMaterialPath() const;

		void SetTrunkURange(TArray<FVector2f> InURange);

		const TArray<FVector2f>& GetTrunkURange() const;

		virtual void CopyEntry(int32 FromIndex, int32 ToIndex) override;

		virtual void RemoveEntries(int32 NumEntries, int32 StartIndex) override;

		void GetSortedBranchIndicesByHierarchy(TArray<int32>& OutSortedIndices) const;
		
		int32 GetBranchIndexFromPointIndex(int32 PointIndex) const;
		
		int32 AddElements(int NumElements);

		void FillParents(const TArray<TArray<int32>>& InputArray);
		
		void RecomputeBranchChildren();

		static TArray<int32> ComputeDescendantsForBranchNumber(
			const int32 BranchNumber,
			const TMap<int32, TArray<int32>>& ImmediateChildren,
			TMap<int32, TArray<int32>>& Memo,
			TSet<int32>& VisitingSet);

		static TArray<int32> ComputeParentsForBranchNumber(const int32 BranchNumber, const TMap<int32, int32>& BranchNumbersToBranchParentNumbers,
		                                                   TMap<int32, TArray<int32>>& Memo, TSet<int32>& VisitedParents);

		void RecomputeBranchParents();

		void FillChildren(const TArray<TArray<int32>>& InputArray);
		
		void FillBranchPoints(const TArray<TArray<int32>>& InputArray);

		void FillBranchNumbers(const TArray<int32>& InputArray);
		
		void FillBranchSourceBudNumber(const TArray<int32>& InputArray);
		
		void FillBranchHierarchyNumber(const TArray<int32>& InputArray);
		
		void FillBranchParentNumbers(const TArray<int32>& InputArray);
		
		void FillPlantNumbers(const TArray<int32>& InputArray);
		
		void SetBranchNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);
		
		void SetParentBranchNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);
		
		void SetBranchHierarchyNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);

		template<typename T>
		FORCEINLINE bool GetValueFromAccessor(TManagedArrayAccessor<T> Accessor, T& OutValue, int32 Index, const T DefaultValue) const
		{
			if ((Accessor).IsValid() && (Accessor).IsValidIndex(Index))
			{ 
				OutValue = (Accessor)[Index]; 
				return true;; 
			}

			OutValue = DefaultValue;
			return false;
		}
		
		bool IsTrunk(int32 Index) const;

		void GetParents(int32 Index, TArray<int32>& OutValue) const;

		void GetChildren(int32 Index, TArray<int32>& OutValue) const;

		void GetBranchPoints(int32 Index, TArray<int32>& OutValue) const;
		
		void GetBranchNumber(int32 Index, int32& OutValue) const;
		
		void GetBranchSourceBudNumber(int32 Index, int32& OutValue) const;
		
		const TManagedArray<int32>& GetBranchSourceBudNumbers() const;
		
		void SetBranchSourceBudNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);
		
		void GetBranchHierarchyNumber(int32 Index, int32& OutValue) const;
		
		void GetBranchParentNumber(int32 Index, int32& OutValue) const;
		
		void GetPlantNumber(int32 Index, int32& OutValue) const;

	private:
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<TArray<int32>> Parents;
		TManagedArrayAccessor<TArray<int32>> Children;
		TManagedArrayAccessor<TArray<int32>> BranchPoints;
		TManagedArrayAccessor<int32> BranchNumbers;
		TManagedArrayAccessor<int32> BranchSourceBudNumber;
		TManagedArrayAccessor<TArray<int32>> BranchFoliageIDs;
		TManagedArrayAccessor<int32> BranchUVMaterial;
		TManagedArrayAccessor<int32> BranchHierarchyNumber;
		TManagedArrayAccessor<int32> BranchSimulationGroupIndex;
		TManagedArrayAccessor<FString> TrunkMaterialPathAttribute;
		TManagedArrayAccessor<TArray<FVector2f>> TrunkURangeAttribute;
		TManagedArrayAccessor<int32> BranchParentNumbers;
		TManagedArrayAccessor<int32> PlantNumbers;
	};
}
