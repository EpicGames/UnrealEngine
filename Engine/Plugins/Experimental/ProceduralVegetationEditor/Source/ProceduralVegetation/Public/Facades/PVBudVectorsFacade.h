// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
	constexpr int BudDirectionsApical = 0;
	constexpr int BudDirectionsAxillary = 1;
	constexpr int BudDirectionsLightOptimalIndex = 2;
	constexpr int BudDirectionsLightSubOptimal = 3;
	constexpr int BudDirectionsGuideCurve = 4;
	constexpr int BudDirectionsUpVector = 5;
	
	constexpr int BudLightDetectedAvailableIndex = 0;
	
	/**
	 * FBudVectorsFacade is used to access and manipulate the BudVectors data from Point Group with in the ProceduralVegetation's FManagedArrayCollection
	 * BudVectors represent different growth related vectors in plant data and are associated with each point. 
	 */
	class FBudVectorsFacade
	{
	public:
		FBudVectorsFacade(FManagedArrayCollection& InCollection);
		FBudVectorsFacade(const FManagedArrayCollection& InCollection);

		bool IsValid() const;

		const TArray<FVector3f>& GetBudDirection(int32 Index) const;

		TManagedArray<TArray<FVector3f>>& ModifyBudDirections();

		const TManagedArray<TArray<FVector3f>>& GetBudDirections() const;
		
	private:
		TManagedArrayAccessor<TArray<FVector3f>> BudDirection;
	};
}
