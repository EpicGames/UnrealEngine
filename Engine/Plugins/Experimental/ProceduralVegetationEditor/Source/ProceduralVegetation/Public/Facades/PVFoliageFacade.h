// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "CoreMinimal.h"
#include "IShrinkableFacade.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "DataTypes/PVFoliageInfo.h"
 
namespace PV::Facades
{
	struct PROCEDURALVEGETATION_API FFoliageEntryData
	{
		int32 NameId;
		int32 BranchId;
		FVector3f PivotPoint;
		FVector3f UpVector;
		FVector3f NormalVector;
		float Scale;
		float LengthFromRoot;
		float ConditionUpAlignment;
		float ConditionTip;
		float ConditionLight;
		float ConditionScale;
		float ConditionHealth;
		float ConditionHeight;
		float ConditionGeneration;
		int32 ParentBoneID = INDEX_NONE;
	};
 
	/**
	 * FFoliageFacade is used to access and manipulate Foliage related data for a ProceduralVegetation's FManagedArrayCollection
	 * It presents access to foliage Mesh names, their respective Branch ids, and instancing transform data (Pivot point, Up Vector, and Scale)
	 * It lays out data in two Groups: FoliageNames and Foliage where Foliage references FoliageNames and Branch/Primitives with Ids
	 * It also adds Foliage entry Ids to the Branch/Primitives group. 
	 */
	class PROCEDURALVEGETATION_API FFoliageFacade final : public IShrinkable
	{
	public:
		FFoliageFacade(FManagedArrayCollection& InCollection, int32 InitialSize = 0);
		FFoliageFacade(const FManagedArrayCollection& InCollection);
 
		bool IsConst() const { return Collection == nullptr; }
 
		bool IsValid() const;
 
		int32 NumFoliageEntries() const { return NameIdsAttribute.Num(); }
 
		FFoliageEntryData GetFoliageEntry(const int32 Index) const;
 
		float GetLengthFromRoot(int32 Index) const;
 
		void SetLengthFromRoot(int32 Index, float Input);
 
		int32 GetParentBoneID(int32 Index) const;
 
		void SetParentBoneID(int32 Index, int32 Input);
 
		const TArray<int32>& GetFoliageEntryIdsForBranch(int32 Index) const;
 
		int32 AddFoliageEntry(const FFoliageEntryData& InputData);
 
		void SetFoliageEntry(int32 Index, const FFoliageEntryData& InputData);
 
		void SetFoliageIdsArray(int32 Index, TArray<int32> InputIds);
 
		int32 GetFoliageBranchId(int32 Index) const;
 
		int32 NumFoliageNames() const { return FoliageInfoNamesAttribute.Num(); };
		
		void SetFoliageBranchId(int32 Index, int32 InputId);

		bool IsMaskFoliageEntry(int32 Index) const;
		
		void SetMaskFoliageEntry(int32 Index, bool InUseAsMask);
		
		const FString GetFoliageName(int32 Index) const;
 
		TArray<FString> GetFoliageNames() const;
		
		void SetFoliageNamesFromIndex(const TArray<FString>& InputArray, const int32 StartIndex);
 
		int32 NumFoliageInfo() const { return FoliageInfoNamesAttribute.Num(); };
 
		FPVFoliageInfo GetFoliageInfo(const int32 Index) const;
		
		const FVector3f& GetPivotPoint(int32 Index) const;
 
		TArray<FPVFoliageInfo> GetFoliageInfos() const;
		
		const FVector3f& GetUpVector(int32 Index) const;
 
		const FVector3f& GetNormalVector(int32 Index) const;
 
		void SetPivotPoint(int32 Index, const FVector3f& Input);
 
		void SetUpVector(int32 Index, const FVector3f& Input);
 
		void SetNormalVector(int32 Index, const FVector3f& Input);
 
		void SetScale(int32 Index, float Input);
 
		void SetFoliageName(int32 Index, const FString& Input);
 
		void SetFoliageNames(const TArray<FString>& InputNames);
 
		void SetFoliageInfo(const int32 Index, const FString& Input);
 
		void SetFoliageInfos(const TArray<FPVFoliageInfo>& Infos);
 
		int32 AppendFoliageInfos(const TArray<FPVFoliageInfo>& Infos);

		void SetFoliagePath(FString InPath);
 
		FString GetFoliagePath() const;
 
		virtual int32 GetElementCount() const override;
 
		virtual void CopyEntry(int32 FromIndex, int32 ToIndex) override;
 
		virtual void RemoveEntries(int32 NumEntries, int32 StartIndex) override;
		
		FTransform GetFoliageTransform(int32 Id) const;
 
		const TManagedArray<FVector3f>& GetPivotPositions() const;
		
		void SetPivotPositionsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex);
		
		TManagedArray<FVector3f>& ModifyPivotPositions();
		
		const TManagedArray<int32>& GetFoliageNameIDs() const;
		
		void SetFoliageNameIDsFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);
		
		const TManagedArray<int32>& GetFoliageBranchIDs() const;
		
		void SetFoliageBranchIDsFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);
		
		const TManagedArray<FVector3f>& GetUpVectors() const;
		
		void SetUpVectorsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex);
		
		const TManagedArray<FVector3f>& GetNormalVectors() const;
		
		void SetNormalVectorsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex);
		
		const TManagedArray<float>& GetScales() const;
		
		void SetScalesFromIndex(const TArray<float>& InputArray, const int32 StartIndex);
		
		const TManagedArray<float>& GetLFRs() const;
		
		void SetLFRsFromIndex(const TArray<float>& InputArray, const int32 StartIndex);
 
	protected:
		void DefineSchema(int32 InitialSize = 0);
 
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
 
		TManagedArrayAccessor<bool> FoliageInfoUseAsMaskAttribute;
		TManagedArrayAccessor<FString> FoliageInfoNamesAttribute;
		TManagedArrayAccessor<float> FoliageInfoLightAttribute;
		TManagedArrayAccessor<float> FoliageInfoScaleAttribute;
		TManagedArrayAccessor<float> FoliageInfoTipAttribute;
		TManagedArrayAccessor<float> FoliageInfoUpAlignmentAttribute;
		TManagedArrayAccessor<float> FoliageInfoHealthAttribute;
		TManagedArrayAccessor<float> FoliageInfoHeightAttribute;
		TManagedArrayAccessor<float> FoliageInfoGenerationAttribute;
		
		TManagedArrayAccessor<int32> NameIdsAttribute;
		TManagedArrayAccessor<int32> BranchIdsAttribute;
		TManagedArrayAccessor<FVector3f> PivotPointsAttribute;
		TManagedArrayAccessor<FVector3f> UpVectorsAttribute;
		TManagedArrayAccessor<FVector3f> NormalVectorsAttribute;
		TManagedArrayAccessor<float> ScalesAttribute;
		TManagedArrayAccessor<float> LengthFromRootAttribute;
		TManagedArrayAccessor<float> ConditionLightAttribute;
		TManagedArrayAccessor<float> ConditionScaleAttribute;
		TManagedArrayAccessor<float> ConditionTipAttribute;
		TManagedArrayAccessor<float> ConditionUpAlignmentAttribute;
		TManagedArrayAccessor<float> ConditionHealthAttribute;
		TManagedArrayAccessor<float> ConditionHeightAttribute;
		TManagedArrayAccessor<float> ConditionGenerationAttribute;
		TManagedArrayAccessor<int32> ParentBoneIdsAttribute;
		TManagedArrayAccessor<TArray<int32>> FoliageIdsAttribute;
		TManagedArrayAccessor<FString> FoliagePathAttribute;
	};
}