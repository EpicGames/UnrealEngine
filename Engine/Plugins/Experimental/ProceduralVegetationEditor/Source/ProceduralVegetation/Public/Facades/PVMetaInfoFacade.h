// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

struct FPVGraftInfo;

namespace PV::Facades
{
	class PROCEDURALVEGETATION_API FMetaInfoFacade
	{
	public:
		FMetaInfoFacade(FManagedArrayCollection& InCollection, const int32 InitialSize = 0);
		FMetaInfoFacade(const FManagedArrayCollection& InCollection);

		bool IsConst() const { return Collection == nullptr; }
		bool IsValid() const;

		void CreateGuid(const FString& InPath);
		FGuid GetGuid() const;
		
		void SetGraftAttributes(const FPVGraftInfo& InGraftData);
		FPVGraftInfo GetGraftAttributes() const;
		
		bool GraftEntryUseAsMask() const;
		void SetGraftUseAsMask(const bool InUseAsMask);
		
		void SetLeafGrowth(const TArray<float>& InLeafGrowth);
		void SetAbscissionSenescense(const TArray<float>& InAbscissionSenescense);
		void SetLateralElongation(const TArray<float>& InLateralElongation);

		const TArray<float>& GetLeafGrowth() const;
		const TArray<float>& GetAbscissionSenescense() const;
		const TArray<float>& GetLateralElongation() const;

	protected:
		void DefineSchema(const int32 InitialSize = 0);

		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
		
		TManagedArrayAccessor<FGuid> GuidAttribute;
		TManagedArrayAccessor<TArray<float>> LeafGrowthAttribute;
		TManagedArrayAccessor<TArray<float>> AbscissionSenescenseAttribute;
		TManagedArrayAccessor<TArray<float>> LateralElongationAttribute;
		TManagedArrayAccessor<bool> GraftUseAsMask;
		TManagedArrayAccessor<float> GraftLightAttribute;
		TManagedArrayAccessor<float> GraftUpAlignmentAttribute;
		TManagedArrayAccessor<float> GraftHealthAttribute;
		TManagedArrayAccessor<float> GraftTipAttribute;
		TManagedArrayAccessor<float> GraftHeightAttribute;
		TManagedArrayAccessor<float> GraftGenerationAttribute;
		TManagedArrayAccessor<float> GraftScaleAttribute;
	};
}
