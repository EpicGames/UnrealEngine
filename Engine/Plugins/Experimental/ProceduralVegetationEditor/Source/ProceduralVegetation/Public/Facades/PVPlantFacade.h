// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IShrinkableFacade.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
	/**
	 * FPlantFacade is used to access and manipulate the data from Plants Group with in the ProceduralVegetation's FManagedArrayCollection
	 * A ProceduralVegetation asset can consist of multiple plants. Each entry in the group references that plant's associated branch
	 * numbers and indices.
	 */
	class PROCEDURALVEGETATION_API FPlantFacade
	{
	public:
		FPlantFacade(FManagedArrayCollection& InCollection);
		FPlantFacade(const FManagedArrayCollection& InCollection);

		bool IsConst() const { return Collection == nullptr; }

		bool IsValid() const;

		int32 GetElementCount() const;

		TArray<int32> GetBranchNumbers(const int32 PlantNumber) const;

		TArray<int32> GetBranchIndices(const int32 PlantNumber) const;

		TArray<int32> GetBranchIndicesSortedByHierarchyNumber(const int32 PlantNumber) const;

		TArray<int32> GetPlantNumbers() const;

		TArray<int32> GetTrunkIndices() const;

		TMap<int32, int32> GetPlantNumbersToTrunkIndicesMap() const;

		int32 GetTrunkIndex(const int32 PlantNumber) const;

		bool IsTrunkIndex(const int32 BranchIndex) const;

	private:
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<int32> PlantNumbers;
		TManagedArrayAccessor<int32> BranchNumbers;
		TManagedArrayAccessor<int32> BranchParentNumbers;
		TManagedArrayAccessor<int32> BranchHierarchyNumbers;
	};
}
