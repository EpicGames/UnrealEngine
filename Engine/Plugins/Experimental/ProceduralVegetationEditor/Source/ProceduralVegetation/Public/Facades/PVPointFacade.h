// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IShrinkableFacade.h"
#include "VisualizeTexture.h"
#include "Distributions/Distribution.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
#define FILL_COLLECTION_ATTRIBUTE(Accessor, InputArray) \
if (!Accessor.IsValid()) \
{ \
Accessor.Add(); \
} \
if (Accessor.Num() < InputArray.Num()) \
{ \
Accessor.AddElements(InputArray.Num() - Accessor.Num()); \
} \
for (int32 i = 0; i < InputArray.Num(); i++) \
{ \
Accessor.ModifyAt(i, InputArray[i]); \
}
	/**
	 * FPointFacade is used to access and manipulate the Point Group data from the ProceduralVegetation's FManagedArrayCollection
	 * Only add the frequently used Point attributes and their access to this facade, for the specific access write a new facade
	 */
	class PROCEDURALVEGETATION_API FPointFacade final : public IShrinkable
	{
	public:
		FPointFacade(FManagedArrayCollection& InCollection);
		FPointFacade(const FManagedArrayCollection& InCollection);
		
		bool IsConst() const { return Collection == nullptr; }

		bool IsValid() const;

		virtual int32 GetElementCount() const override;

		const FVector3f& GetPosition(int32 Index) const;
		
		void SetPosition(int32 Index, const FVector3f& InPosition);

		float GetLengthFromRoot(int32 Index) const;
		
		void SetLengthFromRoot(int32 Index, float InLength);

		float GetLengthFromSeed(int32 Index) const;
		
		void SetLengthFromSeed(int32 Index, float InLength);

		float GetPointScale(int32 Index) const;

		void SetPointScale(int32 Index, float InPointScale);
		
		void SetSeedPScale(int32 Index, float InSeedPScale);
		
		void SetSeedPScaleRatio(int32 Index, float InSeedPScaleRatio);

		int32 GetBudNumber(int32 Index) const;

		void SetBudNumber(int32 Index, int32 InBudNumber);

		const TArray<float>& GetBudLightDetected(int32 Index) const;

		void SetBudLightDetected(int32 Index, const TArray<float>& InBudLightDetected);

		const TArray<int>& GetBudDevelopment(int32 Index) const;
		
		void SetBudDevelopment(int32 Index, const TArray<int>& InBudDevelopment);

		int32 GetBudGeneratation(int32 Index) const;

		int32 GetBudAge(int32 Index) const;
		
		int32 GetMaxBudAge() const;
		
		void AddToBudAgeForAllPoints(const int32 Age);

		int32 GetBudBranchAge(int32 Index) const;

		const TArray<FVector3f>& GetBudDirection(int32 Index) const;

		void SetBudDirections(int32 Index, const TArray<FVector3f>& InBudDirections);

		const TArray<float>& GetBudHormoneLevels(int32 Index) const;

		void SetBudHormoneLevels(int32 Index, const TArray<float>& InBudHormoneLevels);

		const TArray<float>& GetBudLateralMeristem(int32 Index) const;
		
		void SetBudLateralMeristem(int32 Index, const TArray<float>& InBudLateralMeristem);

		float GetTextureCoordV(int32 Index) const;
		
		void SetTextureCoordV(int32 Index, float InTextureCoordV);
		
		float GetTextureCoordUOffset(int32 Index) const;
		
		void SetTextureCoordUOffset(int32 Index, float InTextureCoordUOffset);
		
		const FVector2f& GetURange(int32 Index) const;
		
		void SetURange(int32 Index, FVector2f InURange);

		bool IsFusedPoint(const int Generation, const int SourceBudNumber, const int PointIndex) const;

		TManagedArray<FVector3f>& ModifyPositions();
		
		TManagedArray<float>& ModifyPointScales();

		TManagedArray<float>& ModifyLengthFromRoots();
		
		TManagedArray<float>& ModifyLengthFromSeeds();
		
		const TManagedArray<FVector3f>& GetPositions() const;
		
		const TManagedArray<float>& GetPointScales() const;

		const TManagedArray<float>& GetLengthFromRootsArray() const;

		virtual void CopyEntry(int32 FromIndex, int32 ToIndex) override;

		virtual void RemoveEntries(int32 NumEntries, int32 StartIndex) override;
		
		int32 AddElements(int NumElements);

		void FillPositions(const TArray<FVector3f>& InputArray);

		void FillLFR(const TArray<float>& InputArray);
		
		void FillPointScales(const TArray<float>& InputArray);

		void FillBudDirections(const TArray<TArray<FVector3f>>& InputArray);
		
		void FillBudHormoneLevels(const TArray<TArray<float>>& InputArray);
		
		void FillLengthFromSeeds(const TArray<float>& InputArray);
		
		void FillBudNumbers(const TArray<int32>& InputArray);
		
		void FillBudLightDetected(const TArray<TArray<float>>& InputArray);
		
		void FillBudDevelopment(const TArray<TArray<int>>& InputArray);

		void SetPositionsFromIndex(const TArray<FVector3f>& InputArray, const int32 StartIndex);
		
		void SetPointScalesFromIndex(const TArray<float>& InputArray, const int32 StartIndex);
		
		void SetSeedPScalesFromIndex(const TArray<float>& InputArray, const int32 StartIndex);
		
		void SetSeedPScaleRatiosFromIndex(const TArray<float>& InputArray, const int32 StartIndex);
		
		void SetLFRsFromIndex(const TArray<float>& InputArray, const int32 StartIndex);
		
		const TManagedArray<float>& GetLengthFromSeedsArray() const;
		
		void SetLengthFromSeedsArrayFromIndex(const TArray<float>& InputArray, const int32 StartIndex);

		void FillBudStatus(const TArray<TArray<int>>& InputArray);

		void FillBudLateralMeristem(const TArray<TArray<float>>& InputArray);

		void FillSeedPScale(const TArray<float>& InputArray);

		void FillSeedPScaleRatio(const TArray<float>& InputArray);

		template<typename T>
		FORCEINLINE bool GetValueFromAccessor(TManagedArrayAccessor<T> Accessor, T& OutValue, int32 Index, const T DefaultValue) const
		{
			if ((Accessor).IsValid() && (Accessor).IsValidIndex(Index))
			{ 
				OutValue = (Accessor)[Index]; 
				return true;; 
			}

			OutValue = DefaultValue;
			return false;
		}
		
		void GetLFR(int32 Index, float& LFR) const;

		void GetPointScales(int32 Index, float& OutValue) const;

		void GetBudDirections(int32 Index, TArray<FVector3f>& OutValue) const;

		void GetBudHormoneLevels(int32 Index, TArray<float>& OutValue) const;

		void GetBudNumbers(int32 Index, int32& OutValue) const;
		
		const TManagedArray<int32>& GetBudNumbersAttribute() const;
		
		void SetBudNumbersFromIndex(const TArray<int32>& InputArray, const int32 StartIndex);

		void GetBudLightDetected(int32 Index, TArray<float>& OutValue) const;
		
		const TManagedArray<TArray<float>>& GetBudLightDetectedArrays() const;
		
		const TManagedArray<TArray<float>>& GetBudLateralMeristemArrays() const;
		
		void SetBudLightDetectedArraysFromIndex(const TArray<TArray<float>>& InputArray, const int32 StartIndex);

		void GetBudDevelopment(int32 Index, TArray<int>& OutValue) const;
		
		void GetBudStatus(int32 Index, TArray<int>& OutValue) const;
		
		void SetBudStatus(int32 Index, const TArray<int>& InValue);
		
		void GetBudLateralMeristem(int32 Index, TArray<float>& OutValue) const;
		
		void GetSeedPScale(int32 Index, float& OutValue) const;
		
		void GetSeedPScaleRatio(int32 Index, float& OutValue) const;

	private:
		FManagedArrayCollection* Collection = nullptr;
		TManagedArrayAccessor<TArray<int32>> BranchPoints;
		TManagedArrayAccessor<FVector3f> Positions;
		TManagedArrayAccessor<float> LengthFromRoot;
		TManagedArrayAccessor<float> PointScaleGradient;
		TManagedArrayAccessor<float> HullGradient;
		TManagedArrayAccessor<float> MainTrunkGradient;
		TManagedArrayAccessor<float> GroundGradient;
		TManagedArrayAccessor<float> PointScale;
		TManagedArrayAccessor<TArray<FVector3f>> BudDirections;
		TManagedArrayAccessor<TArray<int>> BudStatus;
		TManagedArrayAccessor<TArray<float>> BudLateralMeristem;
		TManagedArrayAccessor<TArray<float>> BudHormoneLevels;
		TManagedArrayAccessor<float> PlantGradients;
		TManagedArrayAccessor<float> LengthFromSeed;
		TManagedArrayAccessor<float> SeedPScale;
		TManagedArrayAccessor<float> SeedPScaleRatio;
		TManagedArrayAccessor<int32> BudNumber;
		TManagedArrayAccessor<float> NjordPixelIndex;
		TManagedArrayAccessor<TArray<float>> BudLightDetected;
		TManagedArrayAccessor<TArray<int>> BudDevelopment;
		TManagedArrayAccessor<float> TextureCoordV;
		TManagedArrayAccessor<float> TextureCoordUOffset;
		TManagedArrayAccessor<FVector2f> URange;
	};
}
