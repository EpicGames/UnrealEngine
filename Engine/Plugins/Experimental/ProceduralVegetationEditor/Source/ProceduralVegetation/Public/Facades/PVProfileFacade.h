// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace PV::Facades
{
	/**
	 * FTrunkProfileFacade is used to access and manipulate Trunk profile data for a ProceduralVegetation's FManagedArrayCollection
	 */
	class PROCEDURALVEGETATION_API FPlantProfileFacade
	{
	public:
		FPlantProfileFacade(FManagedArrayCollection& InCollection);
		FPlantProfileFacade(const FManagedArrayCollection& InCollection);

		bool IsConst() const { return Collection == nullptr; }

		bool IsValid() const;

		int32 NumProfileEntries() const { return ProfilePointsAttribute.Num(); };

		int32 AddProfileEntry(const TArray<float>& InProfilePoints);

		const TArray<float>& GetProfilePoints(const int32 Index) const;

	protected:
		void DefineSchema();

		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<TArray<float>> ProfilePointsAttribute;
	};
}
