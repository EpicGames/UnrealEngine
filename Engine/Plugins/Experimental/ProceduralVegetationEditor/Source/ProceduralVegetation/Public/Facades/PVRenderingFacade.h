// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "PVFoliageFacade.h"

namespace PV::Facades
{
	/**
	 * FPVRenderingFacade is used to build a collection that contains the render data required to render Procedural Vegetation
	 * Add functions to access and modify render data for Procedural Vegetation
	 */
	class PROCEDURALVEGETATION_API FPVRenderingFacade : public GeometryCollection::Facades::FRenderingFacade
	{
	public:
		FPVRenderingFacade(FManagedArrayCollection& InSelf);
		FPVRenderingFacade(const FManagedArrayCollection& InSelf);

		void DefineSchema();

		const FFoliageFacade& GetFoliageFacade() const;

		void CopyFrom(const FManagedArrayCollection& Other);

	protected:
		const FManagedArrayCollection& ConstProceduralVegetationCollection;
		FManagedArrayCollection* ProceduralVegetationCollection = nullptr;

		FFoliageFacade FoliageFacade;
	};
}
