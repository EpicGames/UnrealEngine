// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"

#include "PVTreeFacade.generated.h"

struct FManagedArrayCollection;

UENUM()
enum class ESkeletonVisualizationModes : uint8
{
	None UMETA(Category=""),
	GravitationalStress UMETA(Category="Gravity"),
	CellDensity UMETA(Category="Gravity"),
	CellWeight UMETA(Category="Gravity"),
	LightSenescence UMETA(Category="Light"),
	BranchLight UMETA(Category="Light"),
	LightAbscissionRetention UMETA(Category="Light"),
	AgeSenescence UMETA(Category="Age"),
	AgeAbscissionRetention UMETA(Category="Age"),
	Apical UMETA(Category="Auxin"),
	Auxin UMETA(Category="Auxin"),
	Radical UMETA(Category="Auxin"),
	Ethylene UMETA(Category="Ethlyene"),
	AuxinRetention UMETA(Category="Ethlyene"),
	Cytokinin UMETA(Category="Cytokinin"),
	BranchGeneration UMETA(Category="Debug")
};

namespace PV::Facades
{
	struct FRemoveEntriesResult
	{
		TMap<int32, int32> PointsOldIDsToNewIDs;
		TMap<int32, int32> BranchesOldIDsToNewIDs;
		TMap<int32, int32> FoliageInstancesOldIDsToNewIDs;
	};

	class FBranchFacade;

	/**
	 * FTreeFacade is used to remove branches, points, and foliage instances from a tree collection.
	 * It does not validate whether the input data is consistent i.e. the points actually belong to the
	 * same branches that are asking to be removed etc.
	 */
	class PROCEDURALVEGETATION_API FTreeFacade
	{
	public:
		static FRemoveEntriesResult RemoveEntriesAndReIndexAttributes(FManagedArrayCollection& OutCollection, TArray<bool>& PointsToRemove,
		                                                              TArray<bool>& BranchesToRemove, TArray<bool>& FoliageInstancesToRemove);

		static void RemoveBranches(const FBranchFacade& InBranchFacade, TArray<int>& InBranchesToRemove, FManagedArrayCollection& OutCollection);
		
		static int32 GetBranchGenerationNumber(const FManagedArrayCollection& Collection, int32 BranchIndex);
		
		static TArray<float> GetVisualizationValues(const FManagedArrayCollection& Collection, ESkeletonVisualizationModes VisualizationMode);
	private:
		static void GatherChildBranches(const FBranchFacade& InBranchFacade, const int InParentIndex, TArray<int32>& OutBranchesToRemove);
	};
}
