// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FManagedArrayCollection;
struct FPVExportParams;
class FProperty;
struct FAnalyticsEventAttribute;

namespace PV::Analytics
{
	void PROCEDURALVEGETATION_API SendSessionStartedEvent();
	void PROCEDURALVEGETATION_API SendSessionEndedEvent(const double InTimeInSeconds);
	void SendNodeAddedEvent(const FString& InNodeType);
	void SendNodeTweakedEvent(const FString& InNodeType, const FProperty* InProperty);
	void SendMaterialChangeEvent(FString InMatPath);
	void SendFoliageMeshChangeEvent(FString InFoliageMeshPath);
	void SendWindSettingsChangeEvent(FString InPath);
	TArray<FAnalyticsEventAttribute> PROCEDURALVEGETATION_API GatherExportCommonAttributes(const FManagedArrayCollection& InCollection, const FPVExportParams& InExportParam);
	void PROCEDURALVEGETATION_API SendExportStartEvent(const TArray<FAnalyticsEventAttribute>& InCommonAttributes);

	enum class EExportResult
	{
		Failed,
		Success,
		Skipped
	};
	void PROCEDURALVEGETATION_API SendExportFinishedEvent(TArray<FAnalyticsEventAttribute> InCommonAttributes, float InTotalTimeInSeconds, EExportResult ExportResult);
}
