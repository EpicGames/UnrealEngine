// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataTypes/PVData.h"

#include "Facades/PVRenderingFacade.h"

#include "PVUtilities.generated.h"

class UInstancedStaticMeshComponent;
class UMeshComponent;
class FFoliageFacade;
class AActor;
struct FAssetData;
struct FPVExportParams;

namespace PV::Utilities
{
	bool PROCEDURALVEGETATION_API DebugModeEnabled();

	DECLARE_DELEGATE_OneParam(FFoliageComponentCreatedCallback, UMeshComponent*);

	bool PROCEDURALVEGETATION_API ValidateAssetPathAndName(const FString& MeshName, const FString& Path, UClass* InClass, FString& OutError);

	bool IsFileNameValid(FName FileName, FText& Reason);
	bool PROCEDURALVEGETATION_API DoesConflictingPackageExist(const FString& PackageName, UClass* InClass);
	bool PROCEDURALVEGETATION_API PackageExists(const FString& LongPackageName, UClass* AssetClass);
	int32 PROCEDURALVEGETATION_API GetMeshTriangles(const FString InMeshPath);
	FLinearColor PROCEDURALVEGETATION_API GetRandomHueColor(float Alpha);

	bool PROCEDURALVEGETATION_API IsValidGrowthData(const FManagedArrayCollection& Collection);

	template <typename T>
	TArray<T> LerpArrayElements(const TArray<T>& InArray1, const TArray<T>& InArray2, const float Alpha);

	int32 AddInterpolatedPointToCollection(FManagedArrayCollection& OutCollection, int32 BranchIndex, const int32 CurrentPointIndex, const int32 NextPointIndex,
	                                  const float Alpha, const int32 BudNumber, bool CalculatePixelIndex = false);
	
	bool DoesAssetExist(const FSoftObjectPath& InPath);
}

USTRUCT()
struct FLoopDebugStepper
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Loop Debug", meta=(InlineEditConditionToggle))
	bool bDebug = false;

	UPROPERTY(EditAnywhere, Category="Loop Debug")
	mutable bool bDebugFocus = false;

	UPROPERTY(EditAnywhere, Category="Loop Debug", meta=(EditCondition="bDebug"))
	uint32 MaxSteps = 1;

	void IncrementStep()
	{
		if (MaxSteps != UINT32_MAX) MaxSteps++;
	}

	void DecrementStep()
	{
		if (MaxSteps > 1) MaxSteps--;
	}
};

struct FPVParamDebuggerState
{
	UPVData* Data = nullptr;
	
#if WITH_EDITORONLY_DATA
	FPVDebugSettings DebugSettings;
#endif
};

struct FPVLoopDebuggerState
{
	uint32 NumSteps = 0;
	const FLoopDebugStepper* LoopDebugStepper = nullptr;
	UPVData* Data = nullptr;

	bool bComplete = false;

	bool IsLastLoopIteration() const
	{
		check(LoopDebugStepper);
		return NumSteps >= LoopDebugStepper->MaxSteps;
	}

	bool IterationsCompleted() const
	{
		check(LoopDebugStepper);
		return NumSteps == LoopDebugStepper->MaxSteps;
	}
	
#if WITH_EDITORONLY_DATA
	FPVDebugSettings GetDebugSettings() const
	{
		check(Data);
		return CopyTemp(Data->GetDebugSettings());
	}

	void SetDebugSettings(FPVDebugSettings&& InSettings) const
	{
		check(Data);
		Data->SetDebugSettings(MoveTemp(InSettings));
	}
#endif
	
};


#if WITH_EDITORONLY_DATA

inline FCriticalSection GlobalPVDebuggerLock;
inline TUniquePtr<FPVParamDebuggerState> GlobalPVDebuggerState;

#define PVE_DEBUG_LOCK()																									\
	{																														\
		FScopeLock Lock(&GlobalPVDebuggerLock);
#define PVE_DEBUG_UNLOCK()																									\
	}

#define PVE_PARAM_DEBUG_INIT(InputSettings, PVData)																			\
	if (InputSettings->bDebug)																								\
	{																														\
		GlobalPVDebuggerState = MakeUnique<FPVParamDebuggerState>(PVData);													\
		GlobalPVDebuggerState->DebugSettings = PVData->GetDebugSettings();													\
	}

#define PVE_PARAM_DEBUG_POINT_PARAM(InParamName, InPivot)																	\
	if (GlobalPVDebuggerState)																								\
		GlobalPVDebuggerState->DebugSettings.ParamDebugVisualizationSettings.Emplace(										\
			EPVDebugValueVisualizationMode::Point,																			\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), NullOpt, NullOpt)															\
		);

#define PVE_PARAM_DEBUG_DIRECTION_PARAM(InParamName, InPivot, InDirection)													\
	if (GlobalPVDebuggerState)																								\
		GlobalPVDebuggerState->DebugSettings.ParamDebugVisualizationSettings.Emplace(										\
			EPVDebugValueVisualizationMode::Direction,																		\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), FVector(InDirection), NullOpt)												\
		);

#define PVE_PARAM_DEBUG_VECTOR_PARAM(InParamName, InPivot, InVector)														\
	if (GlobalPVDebuggerState)																								\
		GlobalPVDebuggerState->DebugSettings.ParamDebugVisualizationSettings.Emplace(										\
			EPVDebugValueVisualizationMode::Vector,																			\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), FVector(InVector), NullOpt)													\
		);

#define PVE_PARAM_DEBUG_TEXT_PARAM(InParamName, InPivot, InText)															\
	if (GlobalPVDebuggerState)																								\
		GlobalPVDebuggerState->DebugSettings.ParamDebugVisualizationSettings.Emplace(										\
			EPVDebugValueVisualizationMode::Text,																			\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), NullOpt, InText)															\
		);

#define PVE_PARAM_DEBUG_END()																								\
	if (GlobalPVDebuggerState)																								\
	{																														\
		GlobalPVDebuggerState->Data->SetDebugSettings(GlobalPVDebuggerState->DebugSettings);								\
		GlobalPVDebuggerState.Reset();																						\
	}

inline TUniquePtr<FPVLoopDebuggerState> GlobalPVLoopDebuggerState;

#define PVE_LOOP_DEBUG_INIT(InputSettings, LoopDebugStepperName, PVData)													\
	if (InputSettings->bDebug && InputSettings->LoopDebugStepperName.bDebug)												\
	{																														\
		GlobalPVLoopDebuggerState = MakeUnique<FPVLoopDebuggerState>(0, &InputSettings->LoopDebugStepperName, PVData);		\
	}

#define PVE_LOOP_DEBUG_PARAMS_START()																						\
	if (																													\
		GlobalPVLoopDebuggerState &&																						\
		GlobalPVLoopDebuggerState->IsLastLoopIteration() &&																	\
		!GlobalPVLoopDebuggerState->bComplete																				\
	)																														\
	{																														\
		FPVDebugSettings __DebugSettings = GlobalPVLoopDebuggerState->GetDebugSettings();									\
		__DebugSettings.bAutoFocusLoopDebug = GlobalPVLoopDebuggerState->LoopDebugStepper->bDebugFocus;

#define PVE_LOOP_DEBUG_POINT_PARAM(InParamName, InPivot)																	\
		__DebugSettings.ParamDebugVisualizationSettings.Emplace(															\
			EPVDebugValueVisualizationMode::Point,																			\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), NullOpt, NullOpt)															\
		);

#define PVE_LOOP_DEBUG_DIRECTION_PARAM(InParamName, InPivot, InDirection)													\
		__DebugSettings.ParamDebugVisualizationSettings.Emplace(															\
			EPVDebugValueVisualizationMode::Direction,																		\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), FVector(InDirection), NullOpt)												\
		);

#define PVE_LOOP_DEBUG_VECTOR_PARAM(InParamName, InPivot, InVector)															\
		__DebugSettings.ParamDebugVisualizationSettings.Emplace(															\
			EPVDebugValueVisualizationMode::Vector,																			\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), FVector(InVector), NullOpt)													\
		);

#define PVE_LOOP_DEBUG_TEXT_PARAM(InParamName, InPivot, InText)																\
		__DebugSettings.ParamDebugVisualizationSettings.Emplace(															\
			EPVDebugValueVisualizationMode::Text,																			\
			FText::FromString(TEXT(InParamName)),																			\
			FPVParamDebugData(FVector(InPivot), NullOpt, InText)															\
		);

#define PVE_LOOP_DEBUG_PARAMS_END()																							\
		if (GlobalPVLoopDebuggerState)																						\
			GlobalPVLoopDebuggerState->SetDebugSettings(MoveTemp(__DebugSettings));											\
	}
#define PVE_OUTER_LOOP_DEBUG_CHECK(ExitMethod)																				\
	if (																													\
		GlobalPVLoopDebuggerState &&																						\
		GlobalPVLoopDebuggerState->bComplete																				\
	)																														\
	{																														\
		ExitMethod;																											\
	}

#define PVE_LOOP_DEBUG_CHECK(ExitMethod)																					\
	if (																													\
		GlobalPVLoopDebuggerState &&																						\
		GlobalPVLoopDebuggerState->IsLastLoopIteration()																	\
	)																														\
	{																														\
		GlobalPVLoopDebuggerState->NumSteps++;																				\
		GlobalPVLoopDebuggerState->bComplete = true;																		\
		ExitMethod;																											\
	}

#define PVE_LOOP_DEBUG_STEP(ExitMethod)																						\
	if (GlobalPVLoopDebuggerState)																							\
	{																														\
		PVE_LOOP_DEBUG_CHECK(ExitMethod)																					\
		GlobalPVLoopDebuggerState->NumSteps++;																				\
	}

#define PVE_LOOP_DEBUG_END()																								\
	GlobalPVLoopDebuggerState.Reset();

#else

#define PVE_DEBUG_LOCK()
#define PVE_DEBUG_UNLOCK()

#define PVE_PARAM_DEBUG_INIT(InputSettings, PVData)

#define PVE_PARAM_DEBUG_POINT_PARAM(InParamName, InPivot)
#define PVE_PARAM_DEBUG_DIRECTION_PARAM(InParamName, InPivot, InDirection)
#define PVE_PARAM_DEBUG_VECTOR_PARAM(InParamName, InPivot, InVector)
#define PVE_PARAM_DEBUG_TEXT_PARAM(InParamName, InPivot, InText)

#define PVE_PARAM_DEBUG_END()

#define PVE_LOOP_DEBUG_INIT(InputSettings, LoopDebugStepperName, PVData)

#define PVE_LOOP_DEBUG_PARAMS_START()
#define PVE_LOOP_DEBUG_POINT_PARAM(InParamName, InPivot)
#define PVE_LOOP_DEBUG_DIRECTION_PARAM(InParamName, InPivot, InDirection)
#define PVE_LOOP_DEBUG_VECTOR_PARAM(InParamName, InPivot, InVector)
#define PVE_LOOP_DEBUG_TEXT_PARAM(InParamName, InPivot, InText)
#define PVE_LOOP_DEBUG_PARAMS_END()

#define PVE_OUTER_LOOP_DEBUG_CHECK(ExitMethod)
#define PVE_LOOP_DEBUG_CHECK(ExitMethod)
#define PVE_LOOP_DEBUG_STEP(ExitMethod)

#define PVE_LOOP_DEBUG_END()

#endif
