// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PCGSettings.h"
#include "PVRenderSettings.h"
#include "DataTypes/PVData.h"
#include "Helpers/PVUtilities.h"
#include "PVBaseSettings.generated.h"

UCLASS(BlueprintType, Abstract, HideCategories=(Debug, AssetInfo), ClassGroup = (Procedural))
class UPVBaseSettings : public UPCGSettings, public IPVRenderSettings
{
	GENERATED_BODY()

public:
	UPVBaseSettings();
	
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }

	/**
	 * Free-form category override for the PV action menu.
	 * Return an empty FText to fall back to the EPCGSettingsType enum-derived name.
	 */
	virtual FText GetCategoryOverride() const { return FText::GetEmpty(); }

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray<EPVRenderType>(); }
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return GetDefaultRenderType(); }
	virtual TArray<EPVRenderType> GetCurrentRenderTypes() const override { return Visualization.CurrentRenderType; }
	virtual TMap<UObject*, FTransform> GetViewportObjects() const override { return TMap<UObject*, FTransform>(); }
	virtual void SetCurrentRenderType (TArray<EPVRenderType> InRenderTypes) override;
	
	virtual const FPVDebugSettings& GetDebugVisualizationSettings() const override { return Visualization.DebugSettings; }
	virtual bool IsDebug() const override { return bDebug; }
	virtual bool IsCollectionRenderingEnabled() const override { return true; }
	virtual bool IsVisualizationCollectionsRenderingEnabled() const override { return true; }
	virtual ELevelViewportType GetPreferredViewportType() const { return ELevelViewportType::LVT_None; }

	virtual bool IsInspectionLocked() const override { return bLockInspection; }
	virtual void SetInspectionLocked(bool bInLocked) override { bLockInspection = bInLocked; }

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	virtual EPCGNodeToolStartBehavior GetNodeToolStartBehaviour() const override { return EPCGNodeToolStartBehavior::OnInspect; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	virtual bool HasExecutionDependencyPin() const override { return false; }

	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AssetInfo, AssetRegistrySearchable, meta = (PCGNoHash))
	bool bOnlyExposeInDebugMode = false;
#endif

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = DebugSettings, meta=(EditCondition = "bDebug", EditConditionHides))
	FPVSettingsVisualization Visualization;
#endif

private:
#if WITH_EDITOR
	bool bLockInspection = false;
#endif
};

class FPVBaseElement : public IPCGElement
{
protected:
	virtual void PostExecuteInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
};