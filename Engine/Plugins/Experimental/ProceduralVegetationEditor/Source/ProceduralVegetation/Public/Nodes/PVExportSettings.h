// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Params/PVExportParams.h"
#include "Nodes/PVBaseSettings.h"
#include "PVExportSettings.generated.h"

UCLASS(ClassGroup = (Procedural))
class UPVExportSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVExportSettings();

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return ExportSettings.MeshName.IsNone() ? "Export" : ExportSettings.MeshName; }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::InputOutput; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{EPVRenderType::PointData, EPVRenderType::Foliage, EPVRenderType::FoliageGrid, EPVRenderType::FoliageAttachments, EPVRenderType::Mesh, EPVRenderType::Bones}; }
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{EPVRenderType::Foliage, EPVRenderType::Mesh}; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

	virtual FPCGElementPtr CreateElement() const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EPCGChangeType GetChangeTypeForProperty(FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif
	
public:
	UPROPERTY(EditAnywhere, Category = ExportSettings, meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FPVExportParams ExportSettings;
};

class FPVExportElement : public FPVBaseElement
{
	virtual bool ExecuteInternal(FPCGContext* Context) const override;

	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
};