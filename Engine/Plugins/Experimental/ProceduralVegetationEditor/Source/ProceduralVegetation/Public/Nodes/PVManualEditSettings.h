// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PVBaseSettings.h"

#include "PVManualEditSettings.generated.h"

UENUM()
enum class EPointSelectionSmoothnessMethod: uint8
{
	Linear,
	Smooth,
	Sphere,
	Root,
	Sharp,
	Sine,
	Constant,
};

UENUM()
enum class ESkeletonSelectionMode: uint8
{
	SelectByTreeDistance,
	SelectByNeighbours,
	SelectByEuclideanDistance,
};

UENUM()
enum class EBranchRemovalMode: uint8
{
	RemoveBranch,
	TrimBranch,
};

UENUM()
enum class EManualEditMode: uint8
{
	TRS,
	BranchRemoval,
};

USTRUCT()
struct FPVSkeletonSelectionParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Selection", meta=(Tooltip="How selection weight falls off from the picked point.\n\nDetermines how strongly nearby points are co-selected. Linear / Smooth / Sphere / Root / Sharp / Sine / Constant."))
	EPointSelectionSmoothnessMethod SelectionFalloff = EPointSelectionSmoothnessMethod::Smooth;

	UPROPERTY(EditAnywhere, Category = "Selection", meta=(Tooltip="How to measure distance for selection: tree-distance, neighbours, or 3D distance.\n\nSelectByTreeDistance: walks along branch graph from the picked point. SelectByNeighbours: N hops along skeleton connections. SelectByEuclideanDistance: standard radial selection."))
	ESkeletonSelectionMode SkeletonSelectionMode = ESkeletonSelectionMode::SelectByNeighbours;

	UPROPERTY(EditAnywhere, Category = "Selection",
		meta = (UIMin=0, ClampMin=0, EditCondition="SkeletonSelectionMode == ESkeletonSelectionMode::SelectByNeighbours", EditConditionHides, Tooltip="Number of neighbouring points to include in the selection.\n\nWalks this many connections from the picked point. 0 = only the picked point itself. Higher = larger soft-selection."))
	int32 NumPointsSelection = 0;

	UPROPERTY(EditAnywhere, Category = "Selection",
		meta = (UIMin=0, ClampMin=0, EditCondition="SkeletonSelectionMode != ESkeletonSelectionMode::SelectByNeighbours", EditConditionHides, Tooltip="Selection radius (in skeleton or world units).\n\nFor SelectByTreeDistance: distance along the skeleton graph. For SelectByEuclideanDistance: straight-line 3D distance in cm. Higher = more points caught."))
	float SelectionDistance = 0.0f;
};

USTRUCT()
struct FPVBranchRemovalParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Branch Removal", meta=(Tooltip="Remove the entire branch or just trim at the click point.\n\nRemoveBranch: deletes the branch and all its children. TrimBranch: cuts at the picked point."))
	EBranchRemovalMode BranchRemovalMode = EBranchRemovalMode::TrimBranch;
};

USTRUCT()
struct FPVManualEditParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Transient, Category = "Tool", meta=(Tooltip="Active editing mode: TRS (transform) or Branch Removal.\n\nTRS: shows a translate/rotate gizmo on the selected points. BranchRemoval: click branches to remove or trim them."))
	EManualEditMode ManualEditMode = EManualEditMode::TRS;

	UPROPERTY(EditAnywhere, Category = "Selection",
		meta=(ShowOnlyInnerProperties, EditCondition="ManualEditMode == EManualEditMode::TRS", EditConditionHides))
	FPVSkeletonSelectionParams SkeletonSelectionParams;

	UPROPERTY(EditAnywhere, Category = "Selection",
		meta=(ShowOnlyInnerProperties, EditCondition="ManualEditMode == EManualEditMode::BranchRemoval", EditConditionHides))
	FPVBranchRemovalParams BranchRemovalParams;

	UPROPERTY()
	TArray<FVector3f> RelativeOffsets;

	UPROPERTY()
	TArray<bool> RemovedPoints;
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVManualEditSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::BoundingBoxOnly }; }
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

public:
#if WITH_EDITOR
	UFUNCTION(CallInEditor, Category = "Edits", DisplayName="Clear Transformations", meta=(Tooltip="Reset all manual transforms.\n\nRemoves every manual translate/rotate/scale edit, restoring the skeleton to its source state. Does not affect branch removals."))
	void ClearTranformations();

	UFUNCTION(CallInEditor, Category = "Edits", DisplayName="Clear Removals", meta=(Tooltip="Restore all removed branches.\n\nUndoes every branch removal, restoring all branches to the source state. Does not affect transforms."))
	void ClearRemovals();
#endif // WITH_EDITOR

	UPROPERTY(EditAnywhere, Category = "Manual Edit", meta = (ShowOnlyInnerProperties))
	FPVManualEditParams ManualEditSettings;
};

class FPVManualEditElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
