// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PVBaseSettings.h"

#include "PVObjectInteractionSettings.generated.h"

UENUM()
enum class EPVCollisionType
{
	AVOID UMETA(DisplayName = "Avoid"),
	TRIM_OUTSIDE UMETA(DisplayName = "Trim Outside"),
	TRIM_INSIDE UMETA(DisplayName = "Trim Inside")
};

USTRUCT()
struct FPVColliderParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Collider", meta=(Tooltip="The static mesh used as the collider shape.\n\nAny static mesh — typically simple primitives like boxes, spheres, or capsules. Complex meshes work but cost more per evaluation."))
	TSoftObjectPtr<UStaticMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = "Collider", meta=(Tooltip="World-space transform of the collider.\n\nPosition, rotation, and scale of the collider relative to the plant's origin."))
	FTransform Transform;

	UPROPERTY(EditAnywhere, Category = "Collider", meta=(Tooltip="How growth responds to this collider.\n\nAvoid: branches bend around the volume. Trim Outside: keep only growth inside the volume. Trim Inside: remove growth inside the volume."))
	EPVCollisionType CollisionType = EPVCollisionType::AVOID;

	UPROPERTY(EditAnywhere, Category = "Collider", meta=(EditCondition = "CollisionType == EPVCollisionType::AVOID", EditConditionHides, UIMin=0, ClampMin=0, Tooltip="Smoothing iterations applied to deflected branches.\n\nHigher values create smoother bends around the collider. 0 = sharp deflection at contact; higher values produce gradual curves over more points."))
	int32 SmoothnessAmount = 0;

	UPROPERTY(EditAnywhere, Category = "Collider", meta=(EditCondition = "CollisionType == EPVCollisionType::AVOID", EditConditionHides, UIMin=0, UIMax=1, ClampMin=0, ClampMax=1, Tooltip="Strength of the smoothing per iteration.\n\n1.0 = full smoothing per iteration. Lower values produce more subtle smoothing."))
	float SmoothnessFactor = 1.0f;
};

USTRUCT()
struct FPVObjectInteractionParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Colliders", meta=(Tooltip="List of collider volumes that affect growth.\n\nEach entry defines one collider's mesh, transform, and interaction mode. The list is evaluated in order."))
	TArray<FPVColliderParams> Colliders;
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVObjectInteractionSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override
	{
		return FName(TEXT("Object Interaction"));
	}

	virtual FText GetNodeTooltipText() const override
	{
		return NSLOCTEXT("PVObjectInteractionSettings", "NodeTooltip",
			"Make the plant respond to collider volumes (avoid, trim inside, or trim outside).\n\n"
			"Defines collision volumes the grown plant must respect. Each collider can avoid (branches grow around it), trim outside (keep only growth inside the volume), or trim inside (keep only growth outside). "
			"Useful for plants growing past obstacles, topiary shaping, or carving clearance volumes for windows/walkways.");
	}

	virtual FLinearColor GetNodeTitleColor() const override;

	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override
	{
		return TArray{EPVRenderType::PointData};
	}
#endif

protected:
	virtual FPCGDataTypeIdentifier GetInputPinTypeIdentifier() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;

	virtual FPCGElementPtr CreateElement() const override;

public:
	UPROPERTY(EditAnywhere, Category = "Object Interaction", meta = (ShowOnlyInnerProperties))
	FPVObjectInteractionParams ObjectInteractionSettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Loop Debug", meta=(EditCondition = "bDebug", EditConditionHides))
	FLoopDebugStepper LoopDebugStepper;
#endif
};

class FPVObjectInteractionElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
