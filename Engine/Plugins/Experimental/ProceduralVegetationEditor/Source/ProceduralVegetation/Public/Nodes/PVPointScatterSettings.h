// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PCGSubgraph.h"
#include "PVPointScatterSettings.generated.h"

UCLASS(BlueprintType, HideCategories=(Debug), ClassGroup = (Procedural))
class UPVPointScatterSettings : public UPCGSubgraphSettings
{
	GENERATED_BODY()
public:
	UPVPointScatterSettings(const FObjectInitializer& InObjectInitializer);
	
	virtual bool IsDynamicGraph() const override { return true; }
	virtual FString GetAdditionalTitleInformation() const override { return FString(""); }
	
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PointScatter")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("UPVPointScatterSettings", "NodeTitle", "Point Scatter"); }
	virtual FText GetNodeTooltipText() const override
	{
		return NSLOCTEXT("UPVPointScatterSettings", "NodeTooltip",
			"Scatter points using a configured PCG graph (for use as seeds).\n\n"
			"Wraps an underlying PCG Graph that emits scattered points. Useful for generating seed positions across an area. "
			"The configuration of the underlying PCG graph determines the scatter behavior — density, area, randomness, etc.");
	}
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
	virtual bool GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const override { return false; }
	// TODO: rendering stuff
	// virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::PointData }; }
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	virtual bool HasExecutionDependencyPin() const override { return false; }
};

class FPVPointScatterElement : public FPCGSubgraphElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
