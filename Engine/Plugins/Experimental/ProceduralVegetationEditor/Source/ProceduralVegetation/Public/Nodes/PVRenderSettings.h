// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#if WITH_EDITOR
#include "Editor/UnrealEdTypes.h"
#endif
#include "PVRenderSettings.generated.h"

struct FPVDebugSettings;
enum class EPVRenderType : uint16;

UINTERFACE(MinimalAPI)
class UPVRenderSettings : public UInterface
{
	GENERATED_BODY()
};

class IPVRenderSettings
{
	GENERATED_BODY()
	
public:

#if WITH_EDITOR
	virtual TArray<EPVRenderType> GetDefaultRenderType() const = 0;
	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const = 0;
	virtual TArray<EPVRenderType> GetCurrentRenderTypes() const = 0;
	virtual TMap<UObject*, FTransform> GetViewportObjects() const = 0; 
	virtual void SetCurrentRenderType(TArray<EPVRenderType> InRenderTypes) = 0;
	virtual const FPVDebugSettings& GetDebugVisualizationSettings() const = 0;
	virtual ELevelViewportType GetPreferredViewportType() const = 0;

	virtual bool IsDebug() const = 0;
	virtual bool IsCollectionRenderingEnabled() const = 0;
	virtual bool IsVisualizationCollectionsRenderingEnabled() const = 0;

	virtual bool IsInspectionLocked() const { return false; }
	virtual void SetInspectionLocked(bool bInLocked) {}
#endif
};
