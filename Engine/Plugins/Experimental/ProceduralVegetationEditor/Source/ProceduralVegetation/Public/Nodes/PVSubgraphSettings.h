// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PCGSubgraph.h"
#include "ProceduralVegetation.h"
#include "Nodes/PVBaseSettings.h"
#include "PVSubgraphSettings.generated.h"

class UProceduralVegetationInstance;
class UProceduralVegetation;

UCLASS(BlueprintType, HideCategories=(Debug, AssetInfo), ClassGroup = (Procedural))
class UPVSubgraphSettings : public UPCGBaseSubgraphSettings, public IPVRenderSettings
{
	GENERATED_BODY()

public:
	UPVSubgraphSettings(const FObjectInitializer& InObjectInitializer);
	
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("ProceduralVegetationSubgraph")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PVSubgraphSettings", "NodeTitle", "Subgraph"); }
	virtual FText GetNodeTooltipText() const override
	{
		return NSLOCTEXT("PVSubgraphSettings", "NodeTooltip",
			"Embed a reusable PVE graph as a single node.\n\n"
			"Wraps a saved Procedural Vegetation graph asset as a node in the current graph. Inputs are forwarded to the subgraph; outputs are forwarded back. "
			"Use to share complex configurations across multiple plants, or to package frequently-used node chains into reusable components.");
	}
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Subgraph; }
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual bool GetPinExtraIcon(const UPCGPin* InPin, FName& OutExtraIcon, FText& OutTooltip) const override { return false; }

	virtual TArray<EPVRenderType> GetSupportedRenderTypes() const override { return TArray{EPVRenderType::PointData, EPVRenderType::Foliage, EPVRenderType::FoliageGrid, EPVRenderType::FoliageAttachments, EPVRenderType::Mesh, EPVRenderType::Bones}; }
	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{EPVRenderType::PointData}; }
	virtual TArray<EPVRenderType> GetCurrentRenderTypes() const override { return Visualization.CurrentRenderType; }
	virtual TMap<UObject*, FTransform> GetViewportObjects() const override { return TMap<UObject*, FTransform>(); }
	virtual void SetCurrentRenderType (TArray<EPVRenderType> InRenderTypes) override;
	
	virtual const FPVDebugSettings& GetDebugVisualizationSettings() const override { return Visualization.DebugSettings; }
	virtual bool IsDebug() const override { return bDebug; }
	virtual bool IsCollectionRenderingEnabled() const override { return true; }
	virtual bool IsVisualizationCollectionsRenderingEnabled() const override { return true; }
	virtual ELevelViewportType GetPreferredViewportType() const { return ELevelViewportType::LVT_Perspective; }

	virtual bool IsInspectionLocked() const override { return bLockInspection; }
	virtual void SetInspectionLocked(bool bInLocked) override { bLockInspection = bInLocked; }
#endif
	
	//~UPCGSettings interface implementation
	virtual UPCGNode* CreateNode() const override;
	virtual FString GetAdditionalTitleInformation() const override;

protected:
	virtual FPCGElementPtr CreateElement() const override;
	virtual void SetSubgraphInternal(UPCGGraphInterface* InGraph) override;
	virtual bool HasExecutionDependencyPin() const override { return false; }
	
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif

private:
	UPROPERTY(VisibleAnywhere, Category = Properties, Instanced, meta = (NoResetToDefault, Tooltip="The Procedural Vegetation graph instance to embed.\n\nPick a UProceduralVegetationInstance asset. The subgraph's nodes execute within this node's context. Edits to the source asset propagate to this instance."))
	TObjectPtr<UProceduralVegetationInstance> SubgraphInstance;
	
public:
	
	virtual UPCGGraphInterface* GetSubgraphInterface() const override;

	virtual UPCGGraphInterface* GetOriginalSubgraphInterface() const override { return SubgraphInstance->GraphInstance; }
	virtual bool IsDynamicGraph() const override { return true;}

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = DebugSettings, meta=(EditCondition = "bDebug", EditConditionHides, Tooltip="Debug visualization overrides for the subgraph."))
	FPVSettingsVisualization Visualization;
#endif

private:
#if WITH_EDITOR
	bool bLockInspection = false;
#endif

};

class FPVSubgraphElement : public FPCGSubgraphElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual void PostExecuteInternal(FPCGContext* InContext) const override;
};