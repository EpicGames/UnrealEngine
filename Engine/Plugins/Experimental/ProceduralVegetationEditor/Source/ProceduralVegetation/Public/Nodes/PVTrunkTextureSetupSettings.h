// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Params/PVTrunkTextureSetupParams.h"
#include "Nodes/PVBaseSettings.h"
#include "PVTrunkTextureSetupSettings.generated.h"

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPVTrunkTextureSetupSettings : public UPVBaseSettings
{
	GENERATED_BODY()

public:
	UPVTrunkTextureSetupSettings();

#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("TrunkTextureSetup")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PVTrunkTextureSetupSettings", "NodeTitle", "Trunk Texture Setup"); }
	virtual FText GetNodeTooltipText() const override
	{
		return NSLOCTEXT("PVTrunkTextureSetupSettings", "NodeTooltip",
			"Bake or load trim sheet textures for plant materials.\n\n"
			"Generates trim sheet textures (or loads previously-baked ones) and outputs the trim sheet UV details that feed into the Mesh Builder's materials. "
			"Use Texture Create mode to bake fresh textures; switch to Texture Load to use baked output.");
	}
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetCategoryOverride() const override;

	virtual TArray<EPVRenderType> GetDefaultRenderType() const override { return TArray{ EPVRenderType::Textures }; }
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	virtual void PostEditUndo() override;
#endif

	PROCEDURALVEGETATION_API bool GetTextureBaked() const { return bTexturesBaked; }

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGDataTypeIdentifier GetOutputPinTypeIdentifier() const override;
	virtual FPCGElementPtr CreateElement() const override;

public:
	UFUNCTION(Category=NoCategory)
	PROCEDURALVEGETATION_API void BakeTextures();
	
	UPROPERTY(EditAnywhere, Category=TrunkTextureSetup, meta=(RelativeToGameDir, ShowOnlyInnerProperties))
	FPVTrunkTextureSetupParams TrunkTextureSetupParams;

	UPROPERTY()
	FPVTrunkTextureSetupInfo TrunkTextureSetupInfo;

	UPROPERTY()
	FString BakingErrors;

private:
	UPROPERTY()
	bool bTexturesBaked = false;
};

class FPVTrunkTextureSetupElement : public FPVBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};
