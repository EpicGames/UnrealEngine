// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicWindSkeletalData.h"
#include "Engine/DataAsset.h"
#include "PVWindSettings.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UPVWindSettings : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="Wind Settings", meta=(Tooltip="Overwrite existing wind data on the target mesh.\n\nWhen on, replaces any wind data already attached to the exported skeletal mesh. Off = leave existing wind setup intact."))
	bool bOverwriteExisting = false;

	UPROPERTY(EditAnywhere, Category="Wind Settings", meta=(Tooltip="List of wind simulation groups (provided by the Dynamic Wind plugin).\n\nEach entry configures one wind simulation group's stiffness, damping, and influence parameters. See the Dynamic Wind plugin docs for per-field details."))
	TArray<FDynamicWindSimulationGroupData> SimulationGroupData;
};