// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/EngineTypes.h"
#include "PVWindSettings.h"

#include "Misc/Paths.h"
#include "PVExportParams.generated.h"

UENUM()
enum class EPVExportMeshType: uint8
{
	StaticMesh,
	SkeletalMesh,
};

UENUM()
enum class EPVAssetReplacePolicy : uint8
{
	/** Create new assets with numbered suffixes */
	Append,

	/** Replaces existing asset with new asset */
	Replace,

	/** Ignores the new asset and keeps the existing asset */
	Ignore,
};

UENUM()
enum class EPVCollisionGeneration : uint8
{
	/** Do not create collision for mesh */
	None,

	/** Creates collision for trunk only */
	TrunkOnly,

	/** Creates collision for all branch generations */
	AllGenerations,
};

USTRUCT()
struct PROCEDURALVEGETATION_API FPVExportParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Asset", Meta = (ContentDir, DisplayName = "Content Browser Folder", Tooltip="Folder under Content/ to save the exported asset(s).\n\nPath relative to the project's Content folder. The exported mesh and any sub-assets (textures, materials) are written here. Must exist or be a writable parent directory."))
	FDirectoryPath ContentBrowserFolder;

	UPROPERTY(EditAnywhere, Category = "Asset", Meta = (DisplayName = "Mesh Name", Tooltip="Filename for the exported mesh asset.\n\nBase name for the output. Combines with `ReplacePolicy` to decide behavior on name collision."))
	FName MeshName;

	UPROPERTY(EditAnywhere, Category = "Asset", Meta = (DisplayName = "Asset Replace Policy", Tooltip="What to do if the target asset already exists: Append, Replace, or Ignore.\n\nAppend: create a numbered version (`MyTree_01`, `MyTree_02`). Replace: overwrite. Ignore: skip the export."))
	EPVAssetReplacePolicy ReplacePolicy = EPVAssetReplacePolicy::Replace;

	UPROPERTY(EditAnywhere, Category = "Mesh", Meta = (DisplayName = "Export Mesh Type", Tooltip="Export as Static Mesh or Skeletal Mesh.\n\nStaticMesh: no wind animation support. SkeletalMesh: includes bones for wind sway."))
	EPVExportMeshType ExportMeshType = EPVExportMeshType::SkeletalMesh;

	UPROPERTY(EditAnywhere, Category = "Mesh | Nanite", Meta = (DisplayName = "Create Nanite Foliage", Tooltip="Add Nanite foliage support for the exported mesh."))
	bool bCreateNaniteFoliage = true;

	UPROPERTY(EditAnywhere, Category = "Mesh | Nanite", Meta = (DisplayName = "Nanite Shape Preservation Method", Tooltip="How Nanite preserves shape during simplification.\n\nVoxelize (default) is the standard Nanite shape-preservation algorithm. See Unreal's Nanite documentation for advanced options."))
	ENaniteShapePreservation NaniteShapePreservation = ENaniteShapePreservation::Voxelize;

	UPROPERTY(EditAnywhere, Category = "Mesh | Collision", Meta = (EditCondition = "ExportMeshType == EPVExportMeshType::StaticMesh", EditConditionHides, Tooltip="Generate collision geometry for the static mesh.\n\nWhen on, builds a simplified collision shape. Useful for plants the player can collide with. Higher build time."))
	bool bCollision = false;

	UPROPERTY(EditAnywhere, Category = "Mesh | Collision", Meta = (EditCondition = "ExportMeshType == EPVExportMeshType::SkeletalMesh", EditConditionHides, Tooltip="Generate a physics asset and pick which branches get collision.\n\nNone: no physics asset. TrunkOnly: collide only with the trunk. AllGenerations: collide with every branch (highest cost)."))
	EPVCollisionGeneration CollisionGeneration = EPVCollisionGeneration::None;

	UPROPERTY(EditAnywhere, Category = "Mesh | Dynamic Wind", Meta = (EditCondition = "ExportMeshType == EPVExportMeshType::SkeletalMesh", EditConditionHides, Tooltip="Wind simulation preset to assign to the skeletal mesh.\n\nContains the Dynamic Wind plugin's simulation group data. Default presets are available for saplings and trees. Required for wind sway."))
	TObjectPtr<UPVWindSettings> WindSettings;

	void Initialize(const FString& InAssetPath, const FString& InName, const FString& InDefaultWindSettingsPath)
	{
		ContentBrowserFolder.Path = FPaths::GetPath(InAssetPath);
		MeshName = FName(InName);
		ReplacePolicy = EPVAssetReplacePolicy::Replace;

		WindSettings = LoadObject<UPVWindSettings>(nullptr, InDefaultWindSettingsPath);
	}

	FString GetOutputObjectPath() const
	{
		const FString AssetName = MeshName.ToString();
		const FString OutputPath = ContentBrowserFolder.Path / AssetName + '.' + AssetName;
		return OutputPath;
	}

	FString GetOutputMeshPackagePath() const
	{
		return FPaths::Combine(ContentBrowserFolder.Path, MeshName.ToString());
	}

	FString GetOutputSkeletonName() const
	{
		return FString::Printf(TEXT("%s_Skeleton"), *MeshName.ToString());
	}

	FString GetOutputSkeletonPackagePath() const
	{
		return FPaths::Combine(ContentBrowserFolder.Path, GetOutputSkeletonName());
	}

	FString GetOutputPhysicsAssetName() const
	{
		return FString::Printf(TEXT("%s_Physics"), *MeshName.ToString());
	}

	FString GetOutputPhysicsAssetPackagePath() const
	{
		return FPaths::Combine(ContentBrowserFolder.Path, GetOutputPhysicsAssetName());
	}
	
	bool IsCollisionEnable() const
	{
		return CollisionGeneration != EPVCollisionGeneration::None;
	}

	UClass* GetMeshClass() const;

	bool Validate(FString& OutError) const;
};