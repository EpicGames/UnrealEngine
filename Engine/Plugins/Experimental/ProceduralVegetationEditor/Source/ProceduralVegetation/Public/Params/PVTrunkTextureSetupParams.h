// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderGraphBuilder.h"
#include "TextureResource.h"
#include "DataTypes/PVTrunkTextureSetupData.h"
#include "Materials/MaterialInstance.h"
#include "PVTrunkTextureSetupParams.generated.h"

UENUM()
enum class EPVTrunkTextureSetupMode : uint8
{
	TextureCreate UMETA(DisplayName="Texture Create"),
	TextureLoad   UMETA(DisplayName="Texture Load"),
};

UENUM()
enum class EPVTextureResolution
{
	Resolution512 = 512 UMETA(DisplayName = "512 x 512"),
	Resolution1024 = 1024 UMETA(DisplayName = "1024 x 1024"),
	Resolution2048 = 2048 UMETA(DisplayName = "2048 x 2048"),
	Resolution4096 = 4096 UMETA(DisplayName = "4096 x 4096"),
	Resolution8192 = 8192 UMETA(DisplayName = "8192 x 8192"),
	Resolution16384 = 16384 UMETA(DisplayName = "16384 x 16384"),
};

USTRUCT()
struct FPVTextureChannelParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Channel, meta=(Tooltip="Channel type (BaseColor, Normal, Roughness, etc.).\n\nPicks which output channel of the bake this texture contributes to."))
	EPVTextureChannel Channel = EPVTextureChannel::BaseColor;

	UPROPERTY(EditAnywhere, Category=Channel, meta=(EditCondition="Channel == EPVTextureChannel::Custom", Tooltip="Custom name for the channel.\n\nRequired when `Channel == Custom`. The string becomes the texture parameter name on the output material."))
	FString ChannelName;

	UPROPERTY(EditAnywhere, Category= Channel, meta = (AllowPrivateAccess = true, FilePathFilter = "Image Files (*.png;*.PNG;*.jpg;*.jpeg;*.exr;)|*.png;*.PNG;*.jpg;*.jpeg;*.exr;", Tooltip="Path to a source image file (PNG / JPG / EXR).\n\nExternal texture loaded from disk for this channel."))
	FFilePath TextureFilePath;

	UPROPERTY(VisibleAnywhere, Category=Channel, meta=(Tooltip="Loaded texture object (auto-populated).\n\nSet automatically after the source file path is loaded. Read-only display."))
	TObjectPtr<UTexture> Texture;

	float WidthRatio = 1.0;

	void LoadTexture();
	
	void ResetTexture();

	FString GetChannelName() const;
};

USTRUCT()
struct FPVTextureGenerationParams
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category= Generation, meta=(Tooltip="Pixel dilation around each generation's region.\n\nAdds padding around the generation's texture region to avoid seams from filtering. Higher = more padding (less risk of seams, less usable area)."))
	float Dilation = 0.1f;

	UPROPERTY(EditAnywhere, Category= Generation, meta=(Tooltip="Horizontal scale of this generation's region.\n\nMultiplier on the region's width within the trim sheet."))
	float XScale = 1.0f;

	UPROPERTY(EditAnywhere, Category= Generation, meta=(Tooltip="Base tile count in X for this generation.\n\nNumber of times the source material tiles within this generation's region."))
	float BaseTileX = 1.0f;

	UPROPERTY(EditAnywhere, Category= Generation, meta=(Tooltip="Base tile count in Y for this generation.\n\nNumber of times the source material tiles within this generation's region."))
	float BaseTileY = 1.0f;

	UPROPERTY( EditAnywhere, Category= Generation, meta=(Tooltip="Source textures for each channel of this generation."))
	TArray<FPVTextureChannelParams> Channels;

	void LoadChannelTexture(int ChannelIndex);

	TArray<FString> GetChannelNames();

	const FPVTextureChannelParams* GetChannel(const FString& Name);

	float GetWidthRatio() const;
	
	bool IsWidthRatioSame() const;
};

USTRUCT()
struct FPVTrunkTextureSetupParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=TrunkTextureSetup, meta=(Tooltip="Bake new textures or load previously-baked ones.\n\nTexture Create: Setup textures and bake to disk. Texture Load: Uses textures already in the bake folder. Use Create when iterating on inputs; switch to Load when iterating on downstream nodes."))
	EPVTrunkTextureSetupMode Mode = EPVTrunkTextureSetupMode::TextureCreate;

	UPROPERTY(EditAnywhere, Category= "Baked Textures", meta=(Tooltip="Prefix added to baked texture asset names.\n\nAll baked textures get this prefix in their filename. Useful for distinguishing multiple Texture Setup nodes' outputs in the same folder."))
	FString AssetNamePrefix;

	UPROPERTY(EditAnywhere, Category= "Baked Textures", meta=(ContentDir, RelativeToGameDir, Tooltip="Folder where baked textures are written / loaded.\n\nPath under Content/. In Texture Create mode, textures are written here. In Texture Load mode, they're read from here. Must exist before baking."))
	FDirectoryPath BakeTextureFolder;

	UPROPERTY(EditAnywhere, Category= "Baked Textures", meta=(EditCondition="Mode == EPVTrunkTextureSetupMode::TextureCreate", EditConditionHides, Tooltip="Enable virtual texture streaming for the baked textures.\n\nGenerates the textures as virtual textures (VTs)."))
	bool bVirtualTexture = false;

	UPROPERTY(EditAnywhere, Category= NoCategory, meta=(Tooltip="Material instance to populate with the baked textures.\n\nAfter baking, the texture parameters on this material instance are set to the baked outputs. The Mesh Builder can then use this material."))
	TObjectPtr<UMaterialInstance> Material;

	UPROPERTY(EditAnywhere, Category=TrunkTextureSetup, meta=(EditCondition="Mode == EPVTrunkTextureSetupMode::TextureCreate", EditConditionHides, Tooltip="Which texture channel to preview in the editor.\n\nBaseColor / AO / Normal / Displacement / Bump / Cavity / Gloss / Roughness / Specular / Custom."))
	EPVTextureChannel PreviewChannel = EPVTextureChannel::BaseColor;

	UPROPERTY(EditAnywhere, Category=TrunkTextureSetup, meta=(EditCondition="Mode == EPVTrunkTextureSetupMode::TextureCreate", EditConditionHides, Tooltip="Resolution of the baked textures.\n\n512 to 16384. Higher = more detail, bigger memory cost."))
	EPVTextureResolution Resolution = EPVTextureResolution::Resolution1024;

	UPROPERTY(EditAnywhere, Category=TrunkTextureSetup, meta=(EditCondition="Mode == EPVTrunkTextureSetupMode::TextureCreate", EditConditionHides, Tooltip="Per-generation trim sheet packing configuration.\n\nEach entry defines how one branch generation's textures are arranged within the trim sheet. Add one entry per generation that should have its own texture region."))
	TArray<FPVTextureGenerationParams> Generations;

	void LoadGenerationChannelTexture(int GenerationIndex, int ChannelIndex);

	float GetWidthRatio(FPVTextureGenerationParams Generation) const;
	
	bool IsWidthRatioSame(FPVTextureGenerationParams Generation) const;
};
