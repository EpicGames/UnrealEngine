// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "PCGGraph.h"
#include "ProceduralVegetation.generated.h"

UCLASS(MinimalAPI)
class UProceduralVegetationGraph : public UPCGGraph
{
	GENERATED_BODY()

public:
	UProceduralVegetationGraph(const FObjectInitializer& ObjectInitializer)
		: UPCGGraph(ObjectInitializer)
	{
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsStandaloneGraph = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bExposeGenerationInAssetExplorer = false;
#endif
		
#if WITH_EDITOR
		SetHiddenFlagInputNode(/*bHidden=*/false);
		SetHiddenFlagOutputNode(/*bHidden=*/false);
#endif
	}

	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual bool ShouldDisplayDebuggingProperties() const override { return false; }
	virtual bool CanToggleStandaloneGraph() const override { return false; };
	virtual bool IsExportToLibraryEnabled() const override { return false; }
	virtual bool ShowGraphCustomization() const override { return false; }
	virtual bool IsTemplatePropertyEnabled() const override { return false; }

	virtual TSubclassOf<UPCGGraph> GetEmbeddedSubgraphClass() const override { return UProceduralVegetationGraph::StaticClass(); }
#endif
	
	virtual bool SupportsEmbeddedSubgraphs() const override { return !IsEmbeddedSubgraph(); }
	UProceduralVegetationGraph* GetEmbeddedParentGraph() const { return GetTypedOuter<UProceduralVegetationGraph>(); }
	bool IsEmbeddedSubgraph() const { return !!GetEmbeddedParentGraph(); }
};

UCLASS(Abstract)
class PROCEDURALVEGETATION_API UProceduralVegetationInterface : public UObject
{
	GENERATED_BODY()

public:
	virtual UPCGGraphInterface* GetGraph() PURE_VIRTUAL(UProceduralVegetationInterface::GetGraph, return nullptr;)
	virtual const UPCGGraphInterface* GetGraph() const PURE_VIRTUAL(UProceduralVegetationInterface::GetGraph, return nullptr;)
};

/**
 * Asset type for procedural plant generation
 */
UCLASS()
class PROCEDURALVEGETATION_API UProceduralVegetation : public UProceduralVegetationInterface
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UProceduralVegetationGraph> Graph;

public:
	virtual UPCGGraphInterface* GetGraph() override { return Graph; }
	virtual const UPCGGraphInterface* GetGraph() const override { return Graph; }

	void SetGraph(UProceduralVegetationGraph* InGraph) { Graph = InGraph; };
	void CreateGraph(const UProceduralVegetationGraph* InGraph = nullptr);
};

UCLASS(MinimalAPI)
class UProceduralVegetationGraphInstance : public UPCGGraphInstance
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Instance)
	TObjectPtr<UProceduralVegetationInterface> ProceduralVegetation;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool ShouldShowGraphProperty() const override { return false; }
	virtual bool CanSaveAsAsset() const override { return false; }
#endif
};

UCLASS()
class PROCEDURALVEGETATION_API UProceduralVegetationInstance : public UProceduralVegetationInterface
{
	GENERATED_BODY()

public:
	UProceduralVegetationInstance(const FObjectInitializer& InObjectInitializer);

	virtual UPCGGraphInterface* GetGraph() override { return GraphInstance ? GraphInstance->GetGraph() : nullptr; }
	virtual const UPCGGraphInterface* GetGraph() const override { return GraphInstance ? GraphInstance->GetGraph() : nullptr; }

	UProceduralVegetationInterface* GetProceduralVegetationAsset() const { return GraphInstance ? GraphInstance->ProceduralVegetation : nullptr; }
	void SetProceduralVegetationAsset(UProceduralVegetationInterface* InAsset) { if (ensure(GraphInstance)) { GraphInstance->ProceduralVegetation = InAsset; } }

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Properties, Instanced, meta = (NoResetToDefault))
	TObjectPtr<UProceduralVegetationGraphInstance> GraphInstance;

public:
	virtual void PostLoad() override;
};
