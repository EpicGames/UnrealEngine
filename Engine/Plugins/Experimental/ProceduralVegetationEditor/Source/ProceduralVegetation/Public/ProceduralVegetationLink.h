// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "ProceduralVegetation.h"
#include "ProceduralVegetationLink.generated.h"

UCLASS(MinimalAPI)
class UProceduralVegetationLink : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = Settings)
	TObjectPtr<UProceduralVegetation> Source;
};
