// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"

#include "GeometryCollection/ManagedArrayCollection.h"

#include "ProceduralVegetationPreset.generated.h"

USTRUCT()
struct FPVPresetVariationInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Variations")
	FName Name;

	UPROPERTY(VisibleAnywhere, Category = "Variations")
	TArray<TSoftObjectPtr<UObject>> FoliageMeshes;

	UPROPERTY(VisibleAnywhere, Category = "Variations")
	TArray<TSoftObjectPtr<UObject>> Materials;

	UPROPERTY(VisibleAnywhere, Category = "Variations")
	TArray<FString> PlantProfiles;
};

UCLASS(BlueprintType, meta=(DisplayName="[DEPRECATED] Procedural Vegetation Preset"))
class PROCEDURALVEGETATION_API UProceduralVegetationPreset : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	FDirectoryPath JsonDirectoryPath;

	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	bool bOverrideFolderPaths = false;

	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	FDirectoryPath FoliageFolder;

	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	FDirectoryPath MaterialsFolder;

	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	FString TrunkMaterialName;

	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	bool bCreateProfileDataAsset = false;

	UPROPERTY(VisibleAnywhere, Category="Preset Data")
	FString PlantProfileName;

	UPROPERTY()
	TMap<FString, FManagedArrayCollection> Variants;

	UPROPERTY(VisibleAnywhere, Category = "Preset Data")
	TArray<FPVPresetVariationInfo> PresetVariations;
};
