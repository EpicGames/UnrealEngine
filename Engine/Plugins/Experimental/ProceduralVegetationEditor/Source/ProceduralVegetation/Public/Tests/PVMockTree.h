// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"

struct FPVBranchHierarchyDescription;
struct FManagedArrayCollection;

namespace PVMockTree
{
	/*
	* Branch Indices:     Branch Numbers:
	*           *B2		            *BN3
	*   *B3 |  / 		    *BN4|  /
	*    \  | / / *B1	     \  | / / *BN2
	*     \ | \/		      \ | \/ 
	*      \| /			       \| / 
	*       |/ 			        |/ 
	*       |			        |
	*       | *B0		        | *BN1
	* 
	* 
	* Point Indices:
	*       5*  11*
	*        |   /     *9
	* *6     |  /     /
	*  \     | *10   /
	*   \   4*  \   /
	*    \   |   \ /
	*     \  |    *8
	*      \ |   /
	*       3*  *7
	*        | /
	*        |/
	*        *2
	*        |
	*        *1
	*        |
	*        *0
	*/

	const FVector3f Point0  = {  00.0000f,  00.0000f, 000.0000f };
	const FVector3f Point1  = {  00.0394f, -00.1783f, 046.2954f };
	const FVector3f Point2  = {  00.0788f, -00.3566f, 092.5909f };
	const FVector3f Point3  = {  01.1927f,  00.9499f, 125.4070f };
	const FVector3f Point4  = {  03.0192f,  01.8644f, 161.3479f };
	const FVector3f Point5  = {  04.8458f,  02.7790f, 197.2887f };
	const FVector3f Point6  = {  19.7899f,  73.3726f, 171.2636f };
	const FVector3f Point7  = { -23.0204f, -29.6581f, 148.4679f };
	const FVector3f Point8  = { -24.5249f, -31.0789f, 156.2391f };
	const FVector3f Point9  = { -30.1656f, -36.4065f, 185.3766f };
	const FVector3f Point10 = { -24.4424f, -21.5258f, 175.3671f };
	const FVector3f Point11 = { -23.0204f, -23.7218f, 201.3998f };

	const float Point0_Radius = 10.f;
	const float Point1_Radius = 0.f;
	const float Point2_Radius = 0.f;
	const float Point3_Radius = 0.f;
	const float Point4_Radius = 0.f;
	const float Point5_Radius = 0.f;
	const float Point6_Radius = 0.f;
	const float Point7_Radius = 0.f;
	const float Point8_Radius = 0.f;
	const float Point9_Radius = 0.f;
	const float Point10_Radius = 0.f;
	const float Point11_Radius = 0.f;

	const float Point0To1_Dist   = 46.2958f;
	const float Point1To2_Dist   = 46.2958f;
	const float Point2To3_Dist   = 32.861f;
	const float Point3To4_Dist   = 35.9989f;
	const float Point4To5_Dist   = 35.9989f;
	const float Point3To6_Dist   = 87.7139f;
	const float Point2To7_Dist   = 67.1893f;
	const float Point7To8_Dist   = 08.0419f;
	const float Point8To9_Dist   = 30.1529f;
	const float Point8To10_Dist  = 21.3811f;
	const float Point10To11_Dist = 26.1638f;

	const float Point0_LengthFromRoot = 0;
	const float Point1_LengthFromRoot = Point0To1_Dist * 0.01f/*cm->m*/;
	const float Point2_LengthFromRoot = Point1_LengthFromRoot + Point1To2_Dist * 0.01f/*cm->m*/;
	const float Point3_LengthFromRoot = Point2_LengthFromRoot + Point2To3_Dist * 0.01f/*cm->m*/;
	const float Point4_LengthFromRoot = Point3_LengthFromRoot + Point3To4_Dist * 0.01f/*cm->m*/;
	const float Point5_LengthFromRoot = Point4_LengthFromRoot + Point4To5_Dist * 0.01f/*cm->m*/;
	const float Point6_LengthFromRoot = Point3_LengthFromRoot + Point3To6_Dist * 0.01f/*cm->m*/;
	const float Point7_LengthFromRoot = Point2_LengthFromRoot + Point2To7_Dist * 0.01f/*cm->m*/;
	const float Point8_LengthFromRoot = Point7_LengthFromRoot + Point7To8_Dist * 0.01f/*cm->m*/;
	const float Point9_LengthFromRoot = Point8_LengthFromRoot + Point8To9_Dist * 0.01f/*cm->m*/;
	const float Point10_LengthFromRoot = Point8_LengthFromRoot + Point8To10_Dist * 0.01f/*cm->m*/;
	const float Point11_LengthFromRoot = Point10_LengthFromRoot + Point10To11_Dist * 0.01f/*cm->m*/;

	const TArray<int32> Branch0_PointIndices = { 0, 1, 2, 3, 4, 5 };
	const TArray<int32> Branch1_PointIndices = { 2, 7, 8, 9 };
	const TArray<int32> Branch2_PointIndices = { 8, 10, 11 };
	const TArray<int32> Branch3_PointIndices = { 3, 6 };

	const int32 Branch0_BranchNumber = 1;
	const int32 Branch1_BranchNumber = 2;
	const int32 Branch2_BranchNumber = 3;
	const int32 Branch3_BranchNumber = 4;

	const TArray<FVector3f> Points = { Point0, Point1, Point2, Point3, Point4, Point5, Point6, Point7, Point8, Point9, Point10, Point11 };
	const TArray<float> PointsRadii = { Point0_Radius, Point1_Radius, Point2_Radius, Point3_Radius, Point4_Radius, Point5_Radius, Point6_Radius, Point7_Radius, Point8_Radius, Point9_Radius, Point10_Radius, Point11_Radius };
	const TArray<TArray<int32>> BranchPointIndices = { Branch0_PointIndices, Branch1_PointIndices, Branch2_PointIndices, Branch3_PointIndices };
}

namespace PVMockTreeCollection
{
	enum class CompareResult
	{
		Success,
		InvalidCollection,
		MismatchNrOfPoints,
		MismatchNrOfBranches,
		MismatchPointPositions
	};

	PROCEDURALVEGETATION_API extern const TCHAR* CompareResultToString(CompareResult InValue);

	PROCEDURALVEGETATION_API extern CompareResult Compare(const FManagedArrayCollection& Collection, bool bCheckPointPositions);

	PROCEDURALVEGETATION_API extern TSharedRef<FManagedArrayCollection> CreateCollection();
}

#endif // WITH_DEV_AUTOMATION_TESTS