// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"
#include "Tests/PVMockTree.h"
#include "Tests/PCGTestsCommon.h"

#include "Misc/AutomationTest.h"

PROCEDURALVEGETATION_API DECLARE_LOG_CATEGORY_EXTERN(LogPVTest, Log, All);

#define PV_IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(TClass, PrettyName, TFlags) \
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TClass, PrettyName, TFlags) \
	bool TClass::RunTest(const FString& Parameters)

#define PV_SIMPLE_AUTOMATION_TEST(CategoryName, TestName) \
	PV_IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(UE_JOIN(UE_JOIN(FPVAutomationTest_,CategoryName),TestName), "PVE." UE_STRINGIZE(CategoryName) "." UE_STRINGIZE(TestName), PCGTestsCommon::TestFlags)

#endif // WITH_DEV_AUTOMATION_TESTS
