// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"

#include "PVFloatRamp.generated.h"

USTRUCT(BlueprintType)
struct PROCEDURALVEGETATION_API FPVFloatRamp
{
	GENERATED_BODY()

	UPROPERTY()
	FRichCurve EditorCurveData;

	FRichCurve* GetRichCurve()
	{
		return &EditorCurveData;
	}
	const FRichCurve* GetRichCurveConst() const
	{
		return &EditorCurveData;
	}

	void InitializeLinearCurve(const FVector2f& Range = FVector2f(0.f, 1.f), const FVector2f& Value = FVector2f(0.f, 1.f))
	{
		EditorCurveData.Reset();
		const FKeyHandle Handle0 = EditorCurveData.AddKey(Range.X, Value.X);
		const FKeyHandle Handle1 = EditorCurveData.AddKey(Range.Y, Value.Y);
		EditorCurveData.SetKeyInterpMode(Handle0, RCIM_Linear);
		EditorCurveData.SetKeyInterpMode(Handle1, RCIM_Linear);
	}
};
