// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ProceduralVegetation.h"
#include "PVEditor.h"
#include "ProceduralVegetation.h"

#include "Editor.h"
#include "PCGGraph.h"
#include "Subsystems/AssetEditorSubsystem.h"

FText UAssetDefinition_ProceduralVegetation::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetDefinition_ProceduralVegetation", "AssetDisplayName", "Procedural Vegetation");
}

FLinearColor UAssetDefinition_ProceduralVegetation::GetAssetColor() const
{
	return FColor::Green;
}

TSoftClassPtr<UObject> UAssetDefinition_ProceduralVegetation::GetAssetClass() const
{
	return UProceduralVegetation::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ProceduralVegetation::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Foliage };
	return Categories;
}

EAssetCommandResult UAssetDefinition_ProceduralVegetation::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UProceduralVegetation* ProceduralVegetation : OpenArgs.LoadObjects<UProceduralVegetation>())
	{
		// If the wrapped graph is an embedded subgraph, defer to the standard PCG asset-editor flow on the
		// embedded graph itself. AssetDefinition_PCGGraph -> FPCGEditor::OpenAssets locates the existing
		// PVE editor for the parent graph and opens the embedded subgraph as a document tab inside it
		// (instead of spawning a separate editor window).
		UProceduralVegetationGraph* PVGraph = Cast<UProceduralVegetationGraph>(ProceduralVegetation->GetGraph());
		if (PVGraph && PVGraph->IsEmbeddedSubgraph())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(PVGraph);
			continue;
		}

		const TSharedRef<FPVEditor> PVEditor = MakeShared<FPVEditor>();
		PVEditor->Initialize(EToolkitMode::Standalone, OpenArgs.ToolkitHost, ProceduralVegetation);
	}

	return EAssetCommandResult::Handled;
}
