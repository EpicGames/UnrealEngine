// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVColliderParamsCustomization.h"

#include "ClassIconFinder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"

#include "ActorFactories/ActorFactoryBasicShape.h"

#include "AssetRegistry/IAssetRegistry.h"

#include "Nodes/PVObjectInteractionSettings.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

class SPVDefaultCollidersButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPVDefaultCollidersButton)
			: _Text()
			, _Image(FAppStyle::GetBrush("Default"))
		{}

		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ATTRIBUTE(const FSlateBrush*, Image)
		SLATE_EVENT(FSimpleDelegate, OnClickAction)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnClickAction = InArgs._OnClickAction;

		ChildSlot
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.WidthOverride(22.0f)
			.HeightOverride(22.0f)
			.ToolTipText(InArgs._Text)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SPVDefaultCollidersButton::OnClick)
				.ContentPadding(0.0f)
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(InArgs._Image)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

private:
	FReply OnClick()
	{
		OnClickAction.ExecuteIfBound();
		return FReply::Handled();
	}

private:
	FSimpleDelegate OnClickAction;
};

TSharedRef<SWidget> MakeDefaultColliderButton(const FName IconName, const TAttribute<FText>& ToolTipText, const FSimpleDelegate& OnClicked)
{
	return SNew(SPVDefaultCollidersButton)
		.Text(ToolTipText)
		.Image(FSlateIconFinder::FindCustomIconBrushForClass(nullptr, TEXT("ClassIcon"), IconName))
		.OnClickAction(OnClicked);
}

TSharedRef<IPropertyTypeCustomization> FPVColliderParamsCustomization::MakeInstance()
{
	return MakeShareable(new FPVColliderParamsCustomization());
}

void FPVColliderParamsCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
)
{}

void FPVColliderParamsCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
)
{
	const auto GetInternalName = [](const TSharedPtr<IPropertyHandle>& InHandle)
		{
			return InHandle->GetProperty()->GetName();
		};

	uint32 NumChild;
	InPropertyHandle->GetNumChildren(NumChild);

	for (uint32 ChildIndex = 0; ChildIndex < NumChild; ++ChildIndex)
	{
		const TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex);
		IDetailPropertyRow& RowBuilder = ChildBuilder.AddProperty(ChildHandle.ToSharedRef());

		if (GetInternalName(ChildHandle) == GET_MEMBER_NAME_STRING_VIEW_CHECKED(FPVColliderParams, Mesh))
		{
			CustomizeColliderSelectorWidget(ChildHandle, RowBuilder);
		}
	}
}

void FPVColliderParamsCustomization::CustomizeColliderSelectorWidget(
	const TSharedPtr<IPropertyHandle>& InChildHandle,
	IDetailPropertyRow& InRowBuilder
)
{
	const TSharedRef<SWidget> NameWidget = InChildHandle->CreatePropertyNameWidget();
	const TSharedRef<SWidget> ValueWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			InChildHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SSeparator)
				.Thickness(1.0f)
				.Orientation(Orient_Vertical)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(FText::FromString("Basic Shapes"))
					]
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Bottom)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(1.0f)
						[
							MakeDefaultColliderButton(
								"ClassIcon.Cube",
								FText::FromString("Cube"),
								FSimpleDelegate::CreateLambda(
									[=]()
										{
											InChildHandle->SetValue(IAssetRegistry::Get()->GetAssetByObjectPath(UActorFactoryBasicShape::BasicCube));
										}
								)
							)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(1.0f)
						[
							MakeDefaultColliderButton(
								"ClassIcon.Sphere",
								FText::FromString("Sphere"),
								FSimpleDelegate::CreateLambda(
									[=]()
										{
											InChildHandle->
												SetValue(IAssetRegistry::Get()->GetAssetByObjectPath(UActorFactoryBasicShape::BasicSphere));
										}
								)
							)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(1.0f)
						[
							MakeDefaultColliderButton(
								"ClassIcon.Cylinder",
								FText::FromString("Cylinder"),
								FSimpleDelegate::CreateLambda(
									[=]()
										{
											InChildHandle->SetValue(
												IAssetRegistry::Get()->GetAssetByObjectPath(UActorFactoryBasicShape::BasicCylinder));
										}
								)
							)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(1.0f)
						[
							MakeDefaultColliderButton(
								"ClassIcon.Cone",
								FText::FromString("Cone"),
								FSimpleDelegate::CreateLambda(
									[=]()
										{
											InChildHandle->SetValue(IAssetRegistry::Get()->GetAssetByObjectPath(UActorFactoryBasicShape::BasicCone));
										}
								)
							)
						]
					]
				]
			]
		];

	InRowBuilder
		.CustomWidget(true)
		.NameContent()
		[
			NameWidget
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			ValueWidget
		];
}
