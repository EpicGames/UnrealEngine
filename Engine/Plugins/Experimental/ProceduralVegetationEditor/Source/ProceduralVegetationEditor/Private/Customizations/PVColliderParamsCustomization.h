// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailPropertyRow.h"
#include "IPropertyTypeCustomization.h"

class FPVColliderParamsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils
	) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
	override;

private:
	void CustomizeColliderSelectorWidget(const TSharedPtr<IPropertyHandle>& InChildHandle, IDetailPropertyRow& InRowBuilder);
};
