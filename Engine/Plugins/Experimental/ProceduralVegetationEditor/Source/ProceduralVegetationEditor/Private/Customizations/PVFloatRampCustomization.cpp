// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVFloatRampCustomization.h"

#include "CurveEditorCommands.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "Utils/PVFloatRamp.h"
#include "ScopedTransaction.h"
#include "SCurveEditor.h"

#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#define CAN_EXECUTE_ACTION(ActionName) CommandList->CanExecuteAction(CurveEditorCommands->ActionName.ToSharedRef())
#define EXECUTE_ACTION(ActionName) CommandList->ExecuteAction(CurveEditorCommands->ActionName.ToSharedRef())

#define LOCTEXT_NAMESPACE "PVEFloatRampCustomization"

const FVector2D FPVFloatRampCustomization::DEFAULT_WINDOW_SIZE = FVector2D(800, 500);

FPVFloatRampOwner::FPVFloatRampOwner(FRichCurve* InCurve, const FName InCurveName)
	: FloatCurve(InCurve)
	, CurveOwner(nullptr)
{
	FloatCurveEditInfo.CurveToEdit = FloatCurve;
	FloatCurveEditInfo.CurveName = InCurveName;
	ConstFloatCurveEditInfo.CurveToEdit = FloatCurve;
	ConstFloatCurveEditInfo.CurveName = InCurveName;
}

FPVFloatRampOwner::FPVFloatRampOwner(const FPVFloatRampOwner& Other)
{
	this->operator=(Other);
}

FPVFloatRampOwner& FPVFloatRampOwner::operator=(const FPVFloatRampOwner& Other)
{
	FloatCurve = Other.FloatCurve;

	FloatCurveEditInfo.CurveToEdit = FloatCurve;
	FloatCurveEditInfo.CurveName = Other.FloatCurveEditInfo.CurveName;
	ConstFloatCurveEditInfo.CurveToEdit = FloatCurve;
	ConstFloatCurveEditInfo.CurveName = Other.ConstFloatCurveEditInfo.CurveName;

	return *this;
}

TArray<FRichCurveEditInfoConst> FPVFloatRampOwner::GetCurves() const
{
	return {ConstFloatCurveEditInfo};
}

void FPVFloatRampOwner::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(ConstFloatCurveEditInfo);
}

TArray<FRichCurveEditInfo> FPVFloatRampOwner::GetCurves()
{
	return {FloatCurveEditInfo};
}

const FName& FPVFloatRampOwner::GetCurveName() const
{
	return FloatCurveEditInfo.CurveName;
}

void FPVFloatRampOwner::SetCurveName(const FName InName)
{
	FloatCurveEditInfo.CurveName = InName;
	ConstFloatCurveEditInfo.CurveName = InName;
}

void FPVFloatRampOwner::SetCurve(FRichCurve* InCurve)
{
	FloatCurve = InCurve;

	FloatCurveEditInfo.CurveToEdit = FloatCurve;
	ConstFloatCurveEditInfo.CurveToEdit = FloatCurve;
}

FRichCurve& FPVFloatRampOwner::GetRichCurve()
{
	return *FloatCurve;
}

const FRichCurve& FPVFloatRampOwner::GetRichCurve() const
{
	return *FloatCurve;
}

void FPVFloatRampOwner::SetOwner(UObject* InOwner)
{
	CurveOwner = InOwner;
}

void FPVFloatRampOwner::ModifyOwner()
{
	if (CurveOwner)
	{
		CurveOwner->Modify(true);
	}
}

UObject* FPVFloatRampOwner::GetOwner()
{
	return CurveOwner;
}

const UObject* FPVFloatRampOwner::GetOwner() const
{
	return CurveOwner;
}

TArray<const UObject*> FPVFloatRampOwner::GetOwners() const
{
	if (CurveOwner)
	{
		return {CurveOwner};
	}
	return {};
}

void FPVFloatRampOwner::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	if (ChangedCurveEditInfos.Num() > 0)
	{
		FRealCurve* ChangedRamp = ChangedCurveEditInfos[0].CurveToEdit;
		OnFloatRampChanged().Broadcast(static_cast<FRichCurve*>(ChangedRamp));
	}
}

bool FPVFloatRampOwner::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return CurveInfo == FloatCurveEditInfo;
}

FPVFloatRampOwner::FOnFloatRampChanged& FPVFloatRampOwner::OnFloatRampChanged()
{
	return OnFloatRampChangedDelegate;
}

void FPVFloatRampOwner::MakeTransactional()
{
	if (CurveOwner)
	{
		CurveOwner->SetFlags(CurveOwner->GetFlags() | RF_Transactional);
	}
}

void ClampKeyTime(FRealCurve* Curve, const FKeyHandle& KeyHandle, const float Min, const float Max)
{
	const float KeyTime = Curve->GetKeyTime(KeyHandle);
	if (KeyTime < Min || KeyTime > Max)
	{
		Curve->SetKeyTime(KeyHandle, FMath::Clamp(KeyTime, Min, Max));
	}
};

void ClampKeyValue(FRealCurve* Curve, const FKeyHandle& KeyHandle, const float Min, const float Max)
{
	const float KeyValue = Curve->GetKeyValue(KeyHandle);
	if (KeyValue < Min || KeyValue > Max)
	{
		Curve->SetKeyValue(KeyHandle, FMath::Clamp(KeyValue, Min, Max), false);
	}
};

FPVFloatRampCustomization::FPVFloatRampCustomization()
	: CurveEditorCommands(&FCurveEditorCommands::Get())
{
	check(CurveEditorCommands);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FPVFloatRampCustomization::~FPVFloatRampCustomization()
{
	if (CurveEditor.IsValid() && CurveEditor->GetCurveOwner() == FloatRampOwner.Get())
	{
		CurveEditor->SetCurveOwner(nullptr, false);
	}

	DestroyPopOutWindow();

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

TSharedRef<IPropertyTypeCustomization> FPVFloatRampCustomization::MakeInstance()
{
	return MakeShareable(new FPVFloatRampCustomization);
}

void FPVFloatRampCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
)
{
	PropertyHandle = InPropertyHandle;

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);
	TArray<void*> StructPtrs;
	PropertyHandle->AccessRawData(StructPtrs);

	if (StructPtrs.Num() == 1)
	{
		FloatRamp = static_cast<FPVFloatRamp*>(StructPtrs[0]);
		const FName CurveName = *InPropertyHandle->GetPropertyDisplayName().ToString();

		FloatRampOwner = MakeUnique<FPVFloatRampOwner>(&FloatRamp->EditorCurveData, CurveName);
		check(FloatRampOwner);

		if (OuterObjects.Num() == 1)
		{
			FloatRampOwner->SetOwner(OuterObjects[0]);
		}

		const SCurveEditor::FArguments CurveEditorArgs = GetCurveEditorArgs(InPropertyHandle);
		const float XAxisMin = CurveEditorArgs._ViewMinInput.IsBound()
			? CurveEditorArgs._ViewMinInput.Get() + 0.05f
			: -FLT_MAX;
		const float XAxisMax = CurveEditorArgs._ViewMaxInput.IsBound()
			? CurveEditorArgs._ViewMaxInput.Get() - 0.05f
			: FLT_MAX;
		const float YAxisMin = CurveEditorArgs._ViewMinOutput.IsBound()
			? CurveEditorArgs._ViewMinOutput.Get() + 0.05f
			: -FLT_MAX;
		const float YAxisMax = CurveEditorArgs._ViewMaxOutput.IsBound()
			? CurveEditorArgs._ViewMaxOutput.Get() - 0.05f
			: FLT_MAX;

		FloatRampOwner->OnFloatRampChanged().AddLambda([=, this](FRichCurve* Curve)
			{
				for (auto KeyIter = Curve->GetKeyHandleIterator(); KeyIter; ++KeyIter)
				{
					if (XAxisMin != -FLT_MAX || XAxisMax != FLT_MAX)
						ClampKeyTime(Curve, *KeyIter, XAxisMin, XAxisMax);
					if (YAxisMin != -FLT_MAX || YAxisMax != FLT_MAX)
						ClampKeyValue(Curve, *KeyIter, YAxisMin, YAxisMax);
				}
				
				if (UObject* const Owner = FloatRampOwner->GetOwner()) 
				{
					Owner->PostEditChange();
				}
			}
		);

		bHasHorizontalRange = CurveEditorArgs._ViewMinInput.IsBound() && CurveEditorArgs._ViewMaxInput.IsBound();
		bHasVerticalRange = CurveEditorArgs._ViewMinOutput.IsBound() && CurveEditorArgs._ViewMaxOutput.IsBound();

		CurveEditor = CreateCurveEditor(CurveEditorArgs);
		check(CurveEditor);

		CommandList = CurveEditor->GetCommands();

		CurveEditor->SetRequireFocusToZoom(true);
		CurveEditor->SetCurveOwner(FloatRampOwner.Get(), true);
		CurveEditor->SetPropertyUtils(InCustomizationUtils.GetPropertyUtilities());
		TSharedPtr<IPropertyHandle> RootHandle = PropertyHandle;
		while (RootHandle->GetParentHandle().IsValid())
		{
			RootHandle = RootHandle->GetParentHandle();
		}
		CurveEditor->RegisterToPropertyChangedEvent(RootHandle);

		HeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Text_Lambda(
					[this]
						{
							return FText::Format(LOCTEXT("PVE_CurveKeyCount", "Curve with {0} Key(s)"), FloatRamp->GetRichCurve()->GetNumKeys());
						}
				)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			];
	}
	else
	{
		HeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(STextBlock)
					.Text(StructPtrs.Num() == 0 ? LOCTEXT("PVE_NoCurves", "No Curves - Unable to modify") : LOCTEXT("PVE_MultipleCurves", "Multiple Curves - Unable to modify"))
                	.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
                	.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			];
	}
}

void FPVFloatRampCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
)
{
	if (!PropertyHandle.IsValid())
	{
		PropertyHandle = InPropertyHandle;
	}

	if (!FloatRampOwner)
	{
		return;
	}
	
	check(CurveEditor);

	ChildBuilder
		.AddCustomRow(FText::FromName(FloatRampOwner->GetCurveName()))
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		.MinDesiredWidth(300)
		[
			SNew(SBorder)
			.BorderImage(nullptr)
			.Padding(0.0f)
			.OnMouseDoubleClick(this, &FPVFloatRampCustomization::OnRampPreviewDoubleClick)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SSeparator)
					.Orientation(Orient_Vertical)
					.Thickness(1.0f)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					CreateRampValueWidget(CurveEditor.ToSharedRef(), false)
				]
			]
		];
}

void FPVFloatRampCustomization::PostUndo(bool bSuccess)
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	TArray<void*> StructPtrs;
	PropertyHandle->AccessRawData(StructPtrs);
	if (StructPtrs.Num() == 1)
	{
		FloatRamp = static_cast<FPVFloatRamp*>(StructPtrs[0]);
		if (FloatRamp && FloatRampOwner)
		{
			if (OuterObjects.Num() == 1)
			{
				FloatRampOwner->SetOwner(OuterObjects[0]);
			}
			FloatRampOwner->SetCurve(&FloatRamp->EditorCurveData);
			CurveEditor->SetCurveOwner(FloatRampOwner.Get(), PropertyHandle->IsEditable());
		}
	}
}

void FPVFloatRampCustomization::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

SCurveEditor::FArguments FPVFloatRampCustomization::GetCurveEditorArgs(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	SCurveEditor::FArguments Args;

	static const FName XAxisName(TEXT("XAxisName"));
	static const FName YAxisName(TEXT("YAxisName"));

	static const FName XAxisMin(TEXT("XAxisMin"));
	static const FName XAxisMax(TEXT("XAxisMax"));
	static const FName YAxisMin(TEXT("YAxisMin"));
	static const FName YAxisMax(TEXT("YAxisMax"));

	if (InPropertyHandle->HasMetaData(XAxisName))
	{
		Args.XAxisName(InPropertyHandle->GetMetaData(XAxisName));
	}
	if (InPropertyHandle->HasMetaData(YAxisName))
	{
		Args.YAxisName(InPropertyHandle->GetMetaData(YAxisName));
	}

	if (InPropertyHandle->HasMetaData(XAxisMin))
	{
		Args.ViewMinInput_Lambda(
			[Value = InPropertyHandle->GetFloatMetaData(XAxisMin)]
				{
					return Value - 0.05f;
				}
		);
	}
	if (InPropertyHandle->HasMetaData(XAxisMax))
	{
		Args.ViewMaxInput_Lambda(
			[Value = InPropertyHandle->GetFloatMetaData(XAxisMax)]
				{
					return Value + 0.05f;
				}
		);
	}
	if (InPropertyHandle->HasMetaData(YAxisMin))
	{
		Args.ViewMinOutput_Lambda(
			[Value = InPropertyHandle->GetFloatMetaData(YAxisMin)]
				{
					return Value - 0.05f;
				}
		);
	}
	if (InPropertyHandle->HasMetaData(YAxisMax))
	{
		Args.ViewMaxOutput_Lambda(
			[Value = InPropertyHandle->GetFloatMetaData(YAxisMax)]
				{
					return Value + 0.05f;
				}
		);
	}

	Args.TimelineLength(0.0f)
	    .HideUI(false)
	    .AllowZoomOutput(true)
	    .ShowZoomButtons(false)
	    .ZoomToFitHorizontal(false)
	    .ZoomToFitVertical(false);

	return Args;
}

TSharedRef<SCurveEditor> FPVFloatRampCustomization::CreateCurveEditor(const SCurveEditor::FArguments& Args) const
{
	return SArgumentNew(Args, SCurveEditor);
}

TSharedRef<SWidget> FPVFloatRampCustomization::CreateRampValueWidget(const TSharedRef<SWidget>& ChildContent, bool bIsWindowWidget)
{
	SVerticalBox::FArguments Slots;

	Slots
		+ SVerticalBox::Slot()
		  .AutoHeight()
		  .HAlign(HAlign_Fill)
		  .VAlign(VAlign_Top)
		[
			SNew(SOverlay)
			+ SOverlay::Slot(0)
			[
				SNew(SColorBlock).Color(FLinearColor(0.018f, 0.018f, 0.018f))
			]
			+ SOverlay::Slot(1)
			.Padding(2.0f)
			[
				CreateRampControlsWidget(bIsWindowWidget)
			]
		];

	if (!bIsWindowWidget)
	{
		Slots
			+ SVerticalBox::Slot()
			[
				SNew(SSeparator)
				.Thickness(1.0f)
			];
	}

	Slots
		+ SVerticalBox::Slot()
		  .HAlign(HAlign_Fill)
		  .VAlign(VAlign_Fill)
		  .SizeParam(bIsWindowWidget
			  ? TOptional<FSizeParam>(FStretch(1.0f))
			  : TOptional<FSizeParam>(FAuto()))
		  .MinHeight(bIsWindowWidget
			  ? 400
			  : 150)
		[
			ChildContent
		];

	return SArgumentNew(Slots, SVerticalBox);
}

TSharedRef<SWidget> FPVFloatRampCustomization::CreateRampControlsWidget(bool bIsWindowWidget)
{
	SHorizontalBox::FArguments ToolsArgs;
	CommandList->CanExecuteAction(FCurveEditorCommands::Get().ZoomToFitHorizontal.ToSharedRef());
	if (CAN_EXECUTE_ACTION(ZoomToFitVertical) && !bHasHorizontalRange)
	{
		ToolsArgs
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.ToolTipText(FText::FromString("Zoom To Fit Horizontal"))
				.OnClicked_Lambda([this]
					{
						EXECUTE_ACTION(ZoomToFitHorizontal);
						return FReply::Handled();
					}
				)
				.ContentPadding(1)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("CurveEd.FitHorizontal"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	if (CAN_EXECUTE_ACTION(ZoomToFitVertical) && !bHasVerticalRange)
	{
		ToolsArgs
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.ToolTipText(FText::FromString("Zoom To Fit Vertical"))
				.OnClicked_Lambda([this]
					{
						EXECUTE_ACTION(ZoomToFitVertical);
						return FReply::Handled();
					}
				)
				.ContentPadding(1)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("CurveEd.FitVertical"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	static const TArray<TSharedPtr<FString>> Presets{
		MakeShared<FString>("Flat"),
		MakeShared<FString>("Hill"),
		MakeShared<FString>("Linear"),
		MakeShared<FString>("Smooth"),
		MakeShared<FString>("Steps"),
		MakeShared<FString>("Valley"),
	};

	SHorizontalBox::FArguments SettingsArgs;

	// TODO: Hide this until we have Icons for each curve preset. Only text looks ugly
	#if 0
	if (bIsWindowWidget)
	{
		SHorizontalBox::FArguments PresetsArgs;
		for (const TSharedPtr<FString>& String : Presets)
		{
			PresetsArgs
				+ SHorizontalBox::Slot()
				  .HAlign(HAlign_Right)
				  .VAlign(VAlign_Top)
				  .AutoWidth()
				[
					SNew(SButton)
					.Text(FText::FromString(*String))
					.OnClicked_Lambda([=, this]
						{
							OnRampPresetSelected(*String);
							return FReply::Handled();
						}
					)
				];
		}
		SettingsArgs
			+ SHorizontalBox::Slot()
			  .HAlign(HAlign_Right)
			  .VAlign(VAlign_Top)
			  .AutoWidth()
			[
				SArgumentNew(PresetsArgs, SHorizontalBox)
			];
	}
	else
	#endif
	{
		SettingsArgs
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SAssignNew(PresetsComboBox, SComboBox<TSharedPtr<FString>>)
				.HasDownArrow(false)
				.ToolTipText(FText::FromString("Presets"))
				.OptionsSource(&Presets)
				.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
				.OnSelectionChanged_Lambda(
					[this](const TSharedPtr<FString>& SelectedPreset, const ESelectInfo::Type SelectType)
						{
							if (SelectType != ESelectInfo::Direct)
							{
								OnRampPresetSelected(*SelectedPreset);
								PresetsComboBox->SetSelectedItem(nullptr);
							}
						}
				)
				.OnGenerateWidget_Lambda(
					[](const TSharedPtr<FString>& Item)
						{
							return SNew(STextBlock)
								.Text(FText::FromString(*Item));
						}
				)
				.ContentPadding(1)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Settings"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			SArgumentNew(ToolsArgs, SHorizontalBox)
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SArgumentNew(SettingsArgs, SHorizontalBox)
		];
}

void FPVFloatRampCustomization::OnRampPresetSelected(const FString& Selection) const
{
	check(FloatRamp && FloatRampOwner);
	FRichCurve& Curve = FloatRamp->EditorCurveData;

	FScopedTransaction PresetTransaction(FText::FromString("Apply Curve Preset"));

	FloatRampOwner->ModifyOwner();

	if (Selection == "Flat")
	{
		Curve.Reset();
		Curve.SetKeyInterpMode(Curve.AddKey(0.5f, 1.0f), RCIM_Constant);
	}
	else if (Selection == "Hill")
	{
		Curve.Reset();
		Curve.SetKeyInterpMode(Curve.AddKey(0.0f, 0.0f), RCIM_Cubic);
		Curve.SetKeyInterpMode(Curve.AddKey(0.5f, 1.0f), RCIM_Cubic);
		Curve.SetKeyInterpMode(Curve.AddKey(1.0f, 0.0f), RCIM_Cubic);
	}
	else if (Selection == "Linear")
	{
		Curve.Reset();
		Curve.SetKeyInterpMode(Curve.AddKey(0.0f, 0.0f), RCIM_Linear);
		Curve.SetKeyInterpMode(Curve.AddKey(1.0f, 1.0f), RCIM_Linear);
	}
	else if (Selection == "Smooth")
	{
		Curve.Reset();
		Curve.SetKeyInterpMode(Curve.AddKey(0.0f, 0.0f), RCIM_Cubic);
		Curve.SetKeyInterpMode(Curve.AddKey(1.0f, 1.0f), RCIM_Cubic);
	}
	else if (Selection == "Steps")
	{
		Curve.Reset();
		Curve.SetKeyInterpMode(Curve.AddKey(0.0f, 0.0f), RCIM_Constant);
		Curve.SetKeyInterpMode(Curve.AddKey(0.25f, 1.0f / 3.0f), RCIM_Constant);
		Curve.SetKeyInterpMode(Curve.AddKey(0.5f, 2.0f / 3.0f), RCIM_Constant);
		Curve.SetKeyInterpMode(Curve.AddKey(0.75f, 1.0f), RCIM_Constant);
	}
	else if (Selection == "Valley")
	{
		Curve.Reset();
		Curve.SetKeyInterpMode(Curve.AddKey(0.0f, 1.0f), RCIM_Cubic);
		Curve.SetKeyInterpMode(Curve.AddKey(0.5f, 0.0f), RCIM_Cubic);
		Curve.SetKeyInterpMode(Curve.AddKey(1.0f, 1.0f), RCIM_Cubic);
	}

	FloatRampOwner->OnCurveChanged(FloatRampOwner->GetCurves());
}

FReply FPVFloatRampCustomization::OnRampPreviewDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		DestroyPopOutWindow();

		// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
		const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
		FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);

		FVector2f AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition(Anchor,
			FPVFloatRampCustomization::DEFAULT_WINDOW_SIZE, true, FVector2f::ZeroVector, Orient_Horizontal);

		TSharedPtr<SWindow> Window = SNew(SWindow)
			.Title(FText::Format(FText::FromString("{0} - Internal Curve Editor"), PropertyHandle->GetPropertyDisplayName()))
			.ClientSize(FPVFloatRampCustomization::DEFAULT_WINDOW_SIZE)
			.ScreenPosition(AdjustedSummonLocation)
			.AutoCenter(EAutoCenter::None)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::FixedSize);

		check(CurveEditor);
		const TSharedRef<SWidget> CurveWindowWidget = CreateRampValueWidget(CurveEditor.ToSharedRef(), true);
		Window->SetContent(CurveWindowWidget);

		FWidgetPath WidgetPath;
		FSlateApplication::Get().GeneratePathToWidgetChecked(CurveEditor.ToSharedRef(), WidgetPath);
		Window = FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), WidgetPath.GetWindow());

		CurveEditorWindow = Window;
	}
	return FReply::Handled();
}

void FPVFloatRampCustomization::DestroyPopOutWindow()
{
	if (CurveEditorWindow.IsValid())
	{
		CurveEditorWindow.Pin()->RequestDestroyWindow();
		CurveEditorWindow.Reset();
	}
}

#undef LOCTEXT_NAMESPACE

#undef EXECUTE_ACTION
#undef CAN_EXECUTE_ACTION
