// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorCommands.h"
#include "IPropertyTypeCustomization.h"
#include "SCurveEditor.h"

#include "Widgets/Input/SComboBox.h"

struct FPVFloatRamp;
class SCurveEditor;

class FPVFloatRampOwner : public FCurveOwnerInterface
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFloatRampChanged, FRichCurve* /* ChangedCurve */);

public:
	FPVFloatRampOwner(FRichCurve* InCurve, const FName InCurveName = NAME_None);
	FPVFloatRampOwner(const FPVFloatRampOwner& Other);
	FPVFloatRampOwner& operator=(const FPVFloatRampOwner& Other);

	FPVFloatRampOwner(FPVFloatRampOwner&& Other) = delete;
	FPVFloatRampOwner& operator=(FPVFloatRampOwner&& Other) = delete;

	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfo> GetCurves() override;

	const FName& GetCurveName() const;
	void SetCurveName(const FName InName);
	void SetCurve(FRichCurve* InCurve);
	FRichCurve& GetRichCurve();
	const FRichCurve& GetRichCurve() const;

	void SetOwner(UObject* InOwner);
	UObject* GetOwner();
	const UObject* GetOwner() const;
	virtual TArray<const UObject*> GetOwners() const override;
	virtual void ModifyOwner() override;
	virtual void MakeTransactional() override;

	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;

	FOnFloatRampChanged& OnFloatRampChanged();

private:
	FRichCurve* FloatCurve;

	TObjectPtr<UObject> CurveOwner;

	FRichCurveEditInfo FloatCurveEditInfo;
	FRichCurveEditInfoConst ConstFloatCurveEditInfo;

	FOnFloatRampChanged OnFloatRampChangedDelegate;
};

class FPVFloatRampCustomization : public IPropertyTypeCustomization, FEditorUndoClient
{
public:
	FPVFloatRampCustomization();
	virtual ~FPVFloatRampCustomization() override;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils
	) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
	override;

	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:
	SCurveEditor::FArguments GetCurveEditorArgs(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	TSharedRef<SCurveEditor> CreateCurveEditor(const SCurveEditor::FArguments& Args) const;
	TSharedRef<SWidget> CreateRampValueWidget(const TSharedRef<SWidget>& ChildContent, bool bIsWindowWidget = false);
	TSharedRef<SWidget> CreateRampControlsWidget(bool bIsWindowWidget = false);

	void OnRampPresetSelected(const FString& Selection) const;

	FReply OnRampPreviewDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	void DestroyPopOutWindow();

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;

	FPVFloatRamp* FloatRamp = nullptr;
	TUniquePtr<FPVFloatRampOwner> FloatRampOwner;

	bool bHasHorizontalRange = false;
	bool bHasVerticalRange = false;

	TSharedPtr<SCurveEditor> CurveEditor;

	TWeakPtr<SWindow> CurveEditorWindow;

	TSharedPtr<FUICommandList> CommandList;
	FCurveEditorCommands* CurveEditorCommands;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> PresetsComboBox;

	static const FVector2D DEFAULT_WINDOW_SIZE;
};
