// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGrowerSettingsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"

#include "Nodes/PVGrowerSettings.h"

#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PVGrowerSettingsCustomization"

namespace
{
	static const FString SectionAll    (TEXT("All"));
	static const FString SectionPreset (TEXT("Preset"));
	static const FString SectionBasic  (TEXT("Basic"));
}

// TODO(Nain): verify "All" is the right default now that "Basic" is never a valid button.
FString FPVGrowerSettingsCustomization::ActiveSection(SectionAll);

namespace
{
	/** Returns the top-level category name (the part before the first '|', trimmed). */
	FString GetTopLevelCategory(const FString& Category)
	{
		int32 PipeIndex;
		if (Category.FindChar(TEXT('|'), PipeIndex))
		{
			return Category.Left(PipeIndex).TrimEnd();
		}
		return Category;
	}

	/** Returns the last pipe-segment of a category string, trimmed (used as button label). */
	FString GetSectionLabel(const FString& SectionId)
	{
		int32 PipeIndex;
		if (SectionId.FindLastChar(TEXT('|'), PipeIndex))
		{
			return SectionId.RightChop(PipeIndex + 1).TrimStart();
		}
		return SectionId;
	}
}

// ---------------------------------------------------------------------------
// SPVScrollAnchor
//
// Zero-height sentinel widget embedded in the Settings category.  On its first
// OnPaint call the widget is already parented into the full Slate tree, so
// GetParentWidget() can walk up to SDetailTree and call ScrollToTop().
//
// A new instance is created every time CustomizeDetails runs (i.e. on every
// node selection), so bScrolled resets naturally for each selection without
// any explicit reset logic.
// ---------------------------------------------------------------------------

class SPVScrollAnchor : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SPVScrollAnchor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs) {}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
	                      const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
	                      int32 LayerId, const FWidgetStyle& InWidgetStyle,
	                      bool bParentEnabled) const override
	{
		if (!bScrolled)
		{
			bScrolled = true;
			ScrollToTop(GetParentWidget());
		}
		return LayerId;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D::ZeroVector;
	}

	static void ScrollToTop(TSharedPtr<SWidget> Widget)
	{
		while (Widget.IsValid())
		{
			const FName WidgetType = Widget->GetType();
			if (WidgetType == TEXT("SDetailTree") ||
				WidgetType == TEXT("STreeView") ||
				WidgetType == TEXT("SListView"))
			{
				StaticCastSharedPtr<STableViewBase>(Widget)->ScrollToTop();
				break;
			}
			Widget = Widget->GetParentWidget();
		}
	}

private:
	mutable bool bScrolled = false;
};

// ---- FPVGrowerSettingsCustomization ----

TSharedRef<IDetailCustomization> FPVGrowerSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FPVGrowerSettingsCustomization());
}

void FPVGrowerSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// ---- Build the root-children list -----------------------------------------------
	//
	// For UPVGrowerSettings the root children are the members of the GrowerParams struct.
	// For UPVGrowerBaseSettings and its derived classes (Phyllotaxy, Branching, etc.)
	// there is no GrowerParams wrapper, so we iterate the class's own editable properties
	// directly (ExcludeSuper — only the leaf class's own properties are relevant here).
	// Both cases are then handled identically by the two passes below.

	TArray<TSharedPtr<IPropertyHandle>> RootChildren;

	{
		TSharedRef<IPropertyHandle> GrowerParamsHandle =
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPVGrowerSettings, GrowerParams));

		if (GrowerParamsHandle->IsValidHandle())
		{
			// UPVGrowerSettings path: unwrap the GrowerParams struct.
			DetailBuilder.HideProperty(GrowerParamsHandle);
			uint32 Num = 0;
			GrowerParamsHandle->GetNumChildren(Num);
			for (uint32 i = 0; i < Num; ++i)
				RootChildren.Add(GrowerParamsHandle->GetChildHandle(i));
		}
		else
		{
			// UPVGrowerBaseSettings path: iterate the class's own editable properties.
			TArray<TWeakObjectPtr<UObject>> TempObjects;
			DetailBuilder.GetObjectsBeingCustomized(TempObjects);
			if (TempObjects.IsValidIndex(0) && TempObjects[0].IsValid())
			{
				UClass* Class = TempObjects[0]->GetClass();
				for (TFieldIterator<FProperty> PropIt(Class, EFieldIteratorFlags::ExcludeSuper); PropIt; ++PropIt)
				{
					FProperty* Prop = *PropIt;
					if (!Prop->HasAnyPropertyFlags(CPF_Edit)) continue;
					TSharedRef<IPropertyHandle> Handle = DetailBuilder.GetProperty(Prop->GetFName(), Class);
					if (Handle->IsValidHandle())
						RootChildren.Add(Handle);
				}
			}
		}
	}

	TArray<FString> DiscoveredSections; // ordered, deduplicated top-level category names

	// ---- First pass: hide auto-layout entries and discover toggle sections ----

	for (const TSharedPtr<IPropertyHandle>& Child : RootChildren)
	{
		if (!Child.IsValid())
		{
			continue;
		}

		const FProperty* Prop = Child->GetProperty();
		if (!Prop)
		{
			continue;
		}

		const FString Category = Prop->GetMetaData(TEXT("Category"));
		if (Category.IsEmpty())
		{
			continue; // Skip non-editable / uncategorised properties (e.g. GuideSettings)
		}

		DetailBuilder.HideProperty(Child.ToSharedRef());
		DetailBuilder.HideCategory(FName(*Category));

		const FString TopLevel = GetTopLevelCategory(Category);
		DetailBuilder.HideCategory(FName(*TopLevel));

		// Recurse one level for ShowOnlyInnerProperties sub-structs so their inlined
		// children are also suppressed in the auto-layout.
		uint32 NumSubChildren = 0;
		Child->GetNumChildren(NumSubChildren);

		// Inner-category grouping applies when the parent uses a pipe-category
		// (e.g. "Phyllotaxy | Trunk") OR when any sub-child carries its own
		// Category metadata (e.g. plain Category = "Growth").
		bool bUseInnerCategoryGrouping = Category.Contains(TEXT("|"));
		if (!bUseInnerCategoryGrouping)
		{
			for (uint32 j = 0; j < NumSubChildren; ++j)
			{
				TSharedPtr<IPropertyHandle> SubChild = Child->GetChildHandle(j);
				const FProperty* SubProp = SubChild.IsValid() ? SubChild->GetProperty() : nullptr;
				if (SubProp && !SubProp->GetMetaData(TEXT("Category")).IsEmpty())
				{
					bUseInnerCategoryGrouping = true;
					break;
				}
			}
		}

		// Basic and Preset properties are handled elsewhere — no toggle button for them.
		// Properties marked InnerCategoryToggle get one toggle per inner category
		// instead of a single top-level toggle for the whole struct.
		if (TopLevel != SectionBasic && TopLevel != SectionPreset)
		{
			if (bUseInnerCategoryGrouping && Prop->HasMetaData(TEXT("InnerCategoryToggle")))
			{
				for (uint32 j = 0; j < NumSubChildren; ++j)
				{
					TSharedPtr<IPropertyHandle> SubChild = Child->GetChildHandle(j);
					if (!SubChild.IsValid()) continue;
					const FProperty* SubProp = SubChild->GetProperty();
					if (SubProp)
					{
						const FString InnerCat = SubProp->GetMetaData(TEXT("Category"));
						if (!InnerCat.IsEmpty())
							DiscoveredSections.AddUnique(InnerCat);
					}
				}
			}
			else
			{
				DiscoveredSections.AddUnique(TopLevel);
			}
		}

		for (uint32 j = 0; j < NumSubChildren; ++j)
		{
			TSharedPtr<IPropertyHandle> SubChild = Child->GetChildHandle(j);
			if (!SubChild.IsValid())
			{
				continue;
			}

			const FProperty* SubProp = SubChild->GetProperty();
			if (SubProp)
			{
				const FString SubCategory = SubProp->GetMetaData(TEXT("Category"));
				if (!SubCategory.IsEmpty())
				{
					DetailBuilder.HideCategory(FName(*SubCategory));
				}
			}

			// Hide sub-children so the auto-layout does not promote them.
			// Applies for pipe-categories and plain categories with inner grouping.
			if (bUseInnerCategoryGrouping)
			{
				DetailBuilder.HideProperty(SubChild.ToSharedRef());

				// Also hide grandchildren to prevent nested ShowOnlyInnerProperties structs
				// (e.g. FPVGrowerSettingsTargetInfo.bTrunk/bBranch, Category="Target") from
				// being promoted into the auto-layout and appearing as duplicate rows.
				uint32 NumGrandChildren = 0;
				SubChild->GetNumChildren(NumGrandChildren);
				for (uint32 k = 0; k < NumGrandChildren; ++k)
				{
					TSharedPtr<IPropertyHandle> GrandChild = SubChild->GetChildHandle(k);
					if (!GrandChild.IsValid()) continue;
					DetailBuilder.HideProperty(GrandChild.ToSharedRef());
					const FProperty* GrandProp = GrandChild->GetProperty();
					if (GrandProp)
					{
						const FString GrandCat = GrandProp->GetMetaData(TEXT("Category"));
						if (!GrandCat.IsEmpty())
							DetailBuilder.HideCategory(FName(*GrandCat));
					}
				}
			}
		}
	}

	// Preset: hide it from auto-layout and add it explicitly in the Preset section.
	// Returns an invalid handle for UPVGrowerBaseSettings derived nodes (Preset was removed).
	TSharedRef<IPropertyHandle> PresetHandle =DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPVGrowerSettings, Preset));
	
	DetailBuilder.HideCategory(TEXT("Preset"));
	
	if (PresetHandle->IsValidHandle())
	{
		DetailBuilder.HideProperty(PresetHandle);
	}

	// Ensure ActiveSection is still valid after a potential schema change.
	const bool bHasPreset = PresetHandle->IsValidHandle();
	
	if (ActiveSection != SectionAll && !(bHasPreset && ActiveSection == SectionPreset) && !DiscoveredSections.Contains(ActiveSection))
	{
		ActiveSection = SectionAll;
	}

	// ---- Settings category: button bar + section content ----

	IDetailCategoryBuilder& SettingsCategory =
		DetailBuilder.EditCategory(TEXT("Settings"), FText::GetEmpty(), ECategoryPriority::Important);

	// Build toggle button bar from the discovered sections.
	// An SPVScrollAnchor is overlaid inside the same row (zero height, no visual impact).
	// On its first paint pass it walks up to SDetailTree and calls ScrollToTop(), fixing
	// the bug where switching nodes leaves the panel scrolled to the previous position —
	// CustomizeDetails runs before the widget tree is ready, so a direct call here would
	// have no parent chain to traverse.
	TSharedRef<SWrapBox> ButtonBar = SNew(SWrapBox).UseAllottedSize(true);

	TWeakPtr<SWidget> WeakButtonBar = ButtonBar;
	TWeakPtr<IPropertyUtilities> WeakUtils = DetailBuilder.GetPropertyUtilities();

	auto AddSectionButton = [WeakButtonBar, WeakUtils, ButtonBar](const FString& SectionName, const FText& Label)
	{
		ButtonBar->AddSlot()
		.Padding(2.f, 2.f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.OnCheckStateChanged_Lambda([SectionName, WeakButtonBar, WeakUtils](ECheckBoxState)
			{
				ActiveSection = SectionName;

				if (TSharedPtr<SWidget> Widget = WeakButtonBar.Pin())
				{
					SPVScrollAnchor::ScrollToTop(Widget);
				}
			})
			.IsChecked_Lambda([SectionName]()
			{
				return ActiveSection == SectionName
					? ECheckBoxState::Checked
					: ECheckBoxState::Unchecked;
			})
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(Label)
			]
		];
	};

	AddSectionButton(SectionAll, LOCTEXT("All", "All"));
	if (bHasPreset)
	{
		AddSectionButton(SectionPreset, LOCTEXT("Preset", "Preset"));
	}
	for (const FString& Section : DiscoveredSections)
	{
		AddSectionButton(Section, FText::FromString(GetSectionLabel(Section)));
	}

	SettingsCategory.AddCustomRow(FText::GetEmpty())
	.WholeRowContent()
	[
		SNew(SOverlay)
		+ SOverlay::Slot()[ SNew(SPVScrollAnchor) ]
		+ SOverlay::Slot()[ ButtonBar ]
	];

	// ---- Section content ----
	//
	// Every child is added with a Visibility binding so switching sections only
	// requires Slate to re-evaluate the attribute — no panel rebuild needed.

	// Main-category groups (e.g. "Phyllotaxy", "Gravity") created lazily so that
	// all properties sharing the same top-level category land under one header.
	TMap<FString, IDetailGroup*> MainGroups;

	TSharedRef<IPropertyHandle> GrowerParamsHandle =
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPVGrowerSettings, GrowerParams));

	bool bCollapseHeader = GrowerParamsHandle->IsValidHandle();

	auto GetOrCreateMainGroup = [&](const FString& TopLevel) -> IDetailGroup&
	{
		if (IDetailGroup** Existing = MainGroups.Find(TopLevel))
			return **Existing;
		const FString GroupFName = TopLevel
			.Replace(TEXT(" | "), TEXT("_"))
			.Replace(TEXT("|"), TEXT("_"))
			.Replace(TEXT(" "), TEXT("_"));
		IDetailGroup& NewGroup = SettingsCategory.AddGroup(
			FName(*GroupFName), FText::FromString(TopLevel), false, true);
		NewGroup.HeaderRow()
			.Visibility(MakeAttributeLambda([TopLevel, bCollapseHeader]() -> EVisibility
			{
				return (GetTopLevelCategory(ActiveSection) == TopLevel || !bCollapseHeader || ActiveSection == SectionAll)
					? EVisibility::Visible : EVisibility::Collapsed;
			}))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TopLevel))
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			];
		MainGroups.Add(TopLevel, &NewGroup);
		return NewGroup;
	};

	for (const TSharedPtr<IPropertyHandle>& ChildHandle : RootChildren)
	{
		if (!ChildHandle.IsValid())
		{
			continue;
		}

		const FProperty* Prop = ChildHandle->GetProperty();
		if (!Prop)
		{
			continue;
		}

		const FString Category = Prop->GetMetaData(TEXT("Category"));
		if (Category.IsEmpty())
		{
			continue;
		}

		const FString TopLevel = GetTopLevelCategory(Category);

		// Basic properties are always shown regardless of the active section.
		if (TopLevel == SectionBasic)
		{
			SettingsCategory.AddProperty(ChildHandle.ToSharedRef())
				.ShouldAutoExpand(true);
			continue;
		}

		// Preset properties are shown via the dedicated Preset section above.
		if (TopLevel == SectionPreset)
		{
			continue;
		}

		auto VisAttr = MakeAttributeLambda([TopLevel]() -> EVisibility
		{
			return (TopLevel == ActiveSection || ActiveSection == SectionAll)
				? EVisibility::Visible : EVisibility::Collapsed;
		});

		// Inner-category grouping: pipe-categories always qualify; for plain
		// categories (e.g. "Growth") check whether sub-children have their own Category.
		bool bGroupByInnerCategory = Category.Contains(TEXT("|"));
		if (!bGroupByInnerCategory)
		{
			uint32 NumSubs = 0;
			ChildHandle->GetNumChildren(NumSubs);
			for (uint32 k = 0; k < NumSubs && !bGroupByInnerCategory; ++k)
			{
				TSharedPtr<IPropertyHandle> Sub = ChildHandle->GetChildHandle(k);
				const FProperty* SubProp = Sub.IsValid() ? Sub->GetProperty() : nullptr;
				if (SubProp && !SubProp->GetMetaData(TEXT("Category")).IsEmpty())
					bGroupByInnerCategory = true;
			}
		}
		if (bGroupByInnerCategory)
		{
			// For pipe-categories the suffix is the part after "|" (e.g. "Trunk").
			// For plain categories (e.g. "Growth") the suffix is empty so group
			// headers show just the inner category name.
			const FString SubCategorySuffix = Category.Contains(TEXT("|"))
				? GetSectionLabel(Category) : FString();

			// Collect sub-children in declaration order, grouped by their own category.
			TArray<FString> CategoryOrder;
			TMap<FString, TArray<TSharedPtr<IPropertyHandle>>> SubCategoryMap;

			uint32 NumSubChildren = 0;
			ChildHandle->GetNumChildren(NumSubChildren);
			for (uint32 j = 0; j < NumSubChildren; ++j)
			{
				TSharedPtr<IPropertyHandle> SubChild = ChildHandle->GetChildHandle(j);
				if (!SubChild.IsValid()) continue;

				const FProperty* SubProp = SubChild->GetProperty();
				const FString InnerCat = SubProp ? SubProp->GetMetaData(TEXT("Category")) : FString();

				if (!SubCategoryMap.Contains(InnerCat))
				{
					CategoryOrder.Add(InnerCat);
				}
				SubCategoryMap.FindOrAdd(InnerCat).Add(SubChild);
			}

			// Create one labelled, expanded group per inner category.
			for (const FString& InnerCat : CategoryOrder)
			{
				// When SubCategorySuffix is empty (plain-category case), the group
				// header is just the inner category name.  Skip uncategorised children.
				const FString GroupName = SubCategorySuffix.IsEmpty()
					? InnerCat
					: (InnerCat.IsEmpty()
						? SubCategorySuffix
						: FString::Printf(TEXT("%s %s"), *InnerCat, *SubCategorySuffix));
				if (GroupName.IsEmpty()) continue;

				// FName must not contain '|' — UE parses it as a category hierarchy separator,
				// producing empty outer headers. Use a sanitized identifier; display text keeps '|'.
				const FString GroupFName = GroupName.Replace(TEXT(" | "), TEXT("_")).Replace(TEXT("|"), TEXT("_")).Replace(TEXT(" "), TEXT("_"));
				IDetailGroup& MainGroup = GetOrCreateMainGroup(TopLevel);
				IDetailGroup& Group = MainGroup.AddGroup(
					FName(*GroupFName), FText::FromString(GroupName),
					/*bStartExpanded=*/true);
				
				// InnerCategoryToggle: each inner category has its own toggle button,
				// so visibility is keyed on InnerCat rather than TopLevel.
				const bool bInnerCategoryToggle = Prop && Prop->HasMetaData(TEXT("InnerCategoryToggle"));
				TAttribute<EVisibility> GroupVisAttr = bInnerCategoryToggle
					? MakeAttributeLambda([InnerCat]() -> EVisibility
					  {
						  return (InnerCat == ActiveSection || ActiveSection == SectionAll)
							  ? EVisibility::Visible : EVisibility::Collapsed;
					  })
					: VisAttr;

				// Set header label and hide the header when its section is not active.
				// NOTE: once HeaderRow() is called the FText passed to AddGroup is ignored —
				// we must supply the label ourselves, otherwise the header renders empty.
				Group.HeaderRow()
					.Visibility(GroupVisAttr)
					.NameContent()
					[
						SNew(STextBlock)
						.Text(FText::FromString(GroupName))
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
					];

				for (TSharedPtr<IPropertyHandle>& PropHandle : SubCategoryMap[InnerCat])
				{
					// If the sub-property uses ShowOnlyInnerProperties, AddPropertyRow renders
					// it as a single collapsed struct row and ignores the meta.  Detect this and
					// add its children individually so they appear as editable rows in the group.
					const FProperty* SubHandleProp = PropHandle->GetProperty();
					const bool bShowOnlyInner = SubHandleProp
						&& SubHandleProp->HasMetaData(TEXT("ShowOnlyInnerProperties"));
					if (bShowOnlyInner)
					{
						uint32 NumInner = 0;
						PropHandle->GetNumChildren(NumInner);
						for (uint32 k = 0; k < NumInner; ++k)
						{
							TSharedPtr<IPropertyHandle> InnerHandle = PropHandle->GetChildHandle(k);
							if (InnerHandle.IsValid())
							{
								Group.AddPropertyRow(InnerHandle.ToSharedRef())
									.Visibility(GroupVisAttr);
							}
						}
					}
					else
					{
						Group.AddPropertyRow(PropHandle.ToSharedRef())
							.Visibility(GroupVisAttr);
					}
				}
			}
		}
		else
		{
			IDetailGroup& MainGroup = GetOrCreateMainGroup(TopLevel);
			MainGroup.AddPropertyRow(ChildHandle.ToSharedRef())
				.ShouldAutoExpand(true)
				.Visibility(VisAttr);
		}
	}

	// ---- Preset section content ----

	if (bHasPreset)
	{
		auto PresetVis = MakeAttributeLambda([]() -> EVisibility
		{
			return (ActiveSection == SectionPreset || ActiveSection == SectionAll)
				? EVisibility::Visible : EVisibility::Collapsed;
		});

		if (PresetHandle->IsValidHandle())
		{
			SettingsCategory.AddProperty(PresetHandle)
				.ShouldAutoExpand(true)
				.Visibility(PresetVis);
		}

	}

}

#undef LOCTEXT_NAMESPACE
