// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

class FPVGrowerSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	// Persists across panel instances; Slate visibility lambdas read it every tick.
	static FString ActiveSection;
};
