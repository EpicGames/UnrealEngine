// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVLoopDebugStepperCustomization.h"

#include "DetailWidgetRow.h"

#include "Helpers/PVUtilities.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"

TSharedRef<IPropertyTypeCustomization> FPVLoopDebugStepperCustomization::MakeInstance()
{
	return MakeShareable(new FPVLoopDebugStepperCustomization);
}

void FPVLoopDebugStepperCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow,
                                                       IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	TArray<void*> StructPtrs;
	InPropertyHandle->AccessRawData(StructPtrs);

	if (StructPtrs.Num() == 1)
	{
		LoopDebugStepper = static_cast<FLoopDebugStepper*>(StructPtrs[0]);
		ParentObject = OuterObjects[0];

		check(LoopDebugStepper);
		check(ParentObject);

		HeaderRow
			.Visibility(TAttribute<EVisibility>::CreateLambda([]()
				{
					return PV::Utilities::DebugModeEnabled()
						? EVisibility::Visible
						: EVisibility::Collapsed;
				}))
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SNumericEntryBox<uint32>)
					.MinDesiredValueWidth(107)
					.AllowSpin(true)
					.AllowWheel(true)
					.Delta(1)
					.CtrlMultiplier(5)
					.MinValue(0)
					.MinSliderValue(0)
					.MaxValue(NullOpt)
					.MaxSliderValue(NullOpt)
					.Value_Lambda([this]()
						{
							return LoopDebugStepper->MaxSteps;
						})
					.OnValueChanged_Lambda([this](const float NewValue)
						{
							LoopDebugStepper->MaxSteps = NewValue;
						})
					.OnValueCommitted_Lambda([this](uint32 NewValue, ETextCommit::Type CommitType)
						{
							LoopDebugStepper->MaxSteps = NewValue;
							FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(PropertyHandle->GetProperty(),
								EPropertyChangeType::ValueSet);
							ParentObject->PostEditChangeProperty(PropertyChangedEvent);
						})
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(0.0f)
						.OnClicked_Lambda([this]()
							{
								LoopDebugStepper->DecrementStep();
								FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(PropertyHandle->GetProperty(),
									EPropertyChangeType::ValueSet);
								ParentObject->PostEditChangeProperty(PropertyChangedEvent);
								return FReply::Handled();
							}
						)
						[
							SNew(SImage)
							.Image_Lambda([this]()
								{
									return FAppStyle::Get().GetBrush("Icons.ChevronLeft");
								})
						]
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(0.0f)
						.OnClicked_Lambda([this]()
							{
								LoopDebugStepper->IncrementStep();
								FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(PropertyHandle->GetProperty(),
									EPropertyChangeType::ValueSet);
								ParentObject->PostEditChangeProperty(PropertyChangedEvent);
								return FReply::Handled();
							}
						)
						[
							SNew(SImage)
							.Image_Lambda([this]()
								{
									return FAppStyle::Get().GetBrush("Icons.ChevronRight");
								})
						]
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(2.0f)
						.OnClicked_Lambda([this]()
							{
								LoopDebugStepper->bDebug = !LoopDebugStepper->bDebug;
								FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(PropertyHandle->GetProperty(),
									EPropertyChangeType::ValueSet);
								ParentObject->PostEditChangeProperty(PropertyChangedEvent);
								return FReply::Handled();
							}
						)
						[
							SNew(SImage)
							.Image_Lambda([this]()
								{
									return LoopDebugStepper->bDebug
										? FAppStyle::Get().GetBrush("GenericStop")
										: FAppStyle::Get().GetBrush("GenericPlay");
								})
						]
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SCheckBox)
						.ToolTipText(FText::FromString("Auto-Focus debug stats"))
						.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
						.OnCheckStateChanged_Lambda([this](const ECheckBoxState State)
							{
								LoopDebugStepper->bDebugFocus = State == ECheckBoxState::Checked;
								if (LoopDebugStepper->bDebugFocus)
								{
									FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(PropertyHandle->GetProperty(),
										EPropertyChangeType::ValueSet);
									ParentObject->PostEditChangeProperty(PropertyChangedEvent);
								}
							}
						)
						.IsChecked_Lambda([this]()
							{
								return LoopDebugStepper->bDebugFocus
									? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							})
						.Padding(4.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Symbols.SearchGlass"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			];
	}
}

void FPVLoopDebugStepperCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder,
                                                         IPropertyTypeCustomizationUtils& InCustomizationUtils)
{}
