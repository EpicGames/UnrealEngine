// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct FLoopDebugStepper;

class FPVLoopDebugStepperCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils
	) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
	override;

private:
	FLoopDebugStepper* LoopDebugStepper = nullptr;
	TSharedPtr<IPropertyHandle> PropertyHandle = nullptr;
	UObject* ParentObject = nullptr;
};
