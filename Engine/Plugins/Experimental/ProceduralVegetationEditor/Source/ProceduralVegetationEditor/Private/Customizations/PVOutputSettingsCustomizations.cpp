// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVOutputSettingsCustomizations.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "ISettingsModule.h"
#include "RenderUtils.h"

#include "Framework/Docking/TabManager.h"

#include "Nodes/PVExportSettings.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

FReply OnOpenProjectSettings()
{
	FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Engine", "Rendering");
	return FReply::Handled();
}

TSharedRef<IPropertyTypeCustomization> FPVOutputSettingsCustomizations::MakeInstance()
{
	return MakeShareable(new FPVOutputSettingsCustomizations());
}

void FPVOutputSettingsCustomizations::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils
)
{}

void FPVOutputSettingsCustomizations::CustomizeChildren(
	TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils
)
{
	const auto GetInternalName = [](const TSharedPtr<IPropertyHandle>& ChildHandle)
		{
			return ChildHandle->GetProperty()->GetName();
		};
	
	uint32 NumChild;
	PropertyHandle->GetNumChildren(NumChild);

	for (uint32 ChildIndex = 0; ChildIndex < NumChild; ++ChildIndex)
	{
		const TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex);
		IDetailPropertyRow& RowBuilder = ChildBuilder.AddProperty(ChildHandle.ToSharedRef());

		if (GetInternalName(ChildHandle) == GET_MEMBER_NAME_STRING_VIEW_CHECKED(FPVExportParams, bCreateNaniteFoliage))
		{
			CustomizeNaniteAssembliesWidget(ChildHandle, RowBuilder);
		}
		else if (GetInternalName(ChildHandle) == GET_MEMBER_NAME_STRING_VIEW_CHECKED(FPVExportParams, NaniteShapePreservation))
		{
			CustomizeNaniteShapePreservationWidget(ChildHandle, RowBuilder);
		}
	}
}

bool IsNaniteSupported()
{
	return DoesPlatformSupportNanite(GMaxRHIShaderPlatform);
}

FText FPVOutputSettingsCustomizations::GetWarningTextForAssemblies() const
{
	FText WarningText;

	if (!NaniteAssembliesSupported())
	{
		WarningText = FText::FromString("WARNING!! Enable Nanite Foliage in Project Settings to enable support for Nanite Assemblies.");
	}

	if (!IsNaniteSupported())
	{
		WarningText = FText::FromString(FString::Format(TEXT("{0}{1}WARNING!! Nanite is not supported. Check default RHI in Project Settings for Nanite Assemblies to work."),{WarningText.ToString(), WarningText.IsEmpty()? "" : "\n"}));
	}

	return WarningText;
}

void FPVOutputSettingsCustomizations::CustomizeNaniteAssembliesWidget(
	const TSharedPtr<IPropertyHandle>& ChildHandle,
	IDetailPropertyRow& RowBuilder
)
{
	const TSharedRef<SWidget> NameWidget = ChildHandle->CreatePropertyNameWidget();
	const TSharedRef<SWidget> ValueWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			ChildHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Visibility_Lambda([=]()
				{
					if (bool bCreateNaniteAssemblies; ChildHandle->GetValue(bCreateNaniteAssemblies) == FPropertyAccess::Success)
					{
						return (bCreateNaniteAssemblies && (!NaniteAssembliesSupported() || !IsNaniteSupported()))
							? EVisibility::Visible
							: EVisibility::Collapsed;
					}
					return EVisibility::Collapsed;
				})
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FLinearColor::Yellow)
					.ShadowColorAndOpacity(FLinearColor::Black)
					.ShadowOffset(FVector2D(3.0f, 3.0f))
					.Text(this, &FPVOutputSettingsCustomizations::GetWarningTextForAssemblies)
					.ToolTipText(this, &FPVOutputSettingsCustomizations::GetWarningTextForAssemblies)
				]
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SButton)
					.Text(FText::FromString("Open Project Settings"))
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("TinyText"))
					.OnClicked_Static(&OnOpenProjectSettings)
				]
			]
		];

	RowBuilder
		.CustomWidget(true)
		.NameContent()
		[
			NameWidget
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			ValueWidget
		];
}

void FPVOutputSettingsCustomizations::CustomizeNaniteShapePreservationWidget(
	const TSharedPtr<IPropertyHandle>& ChildHandle,
	IDetailPropertyRow& RowBuilder
)
{
	const static FText WarningText = FText::FromString(
		"WARNING!! Voxels are not enabled for the project and therefore the exported mesh will not render as voxels. "
		"Enable Nanite Foliage in the project settings to enable support."
	);

	const TSharedRef<SWidget> NameWidget = ChildHandle->CreatePropertyNameWidget();
	const TSharedRef<SWidget> ValueWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.MinDesiredWidth(125)
			[
				ChildHandle->CreatePropertyValueWidget()
			]
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Visibility_Lambda([=]()
				{
					if (
						ENaniteShapePreservation NaniteShapePreservation;
						ChildHandle->GetValue(reinterpret_cast<uint8&>(NaniteShapePreservation)) == FPropertyAccess::Success
					)
					{
						return (NaniteShapePreservation == ENaniteShapePreservation::Voxelize && !NaniteVoxelsSupported())
							? EVisibility::Visible
							: EVisibility::Collapsed;
					}
					return EVisibility::Collapsed;
				})
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FLinearColor::Yellow)
					.ShadowColorAndOpacity(FLinearColor::Black)
					.ShadowOffset(FVector2D(3.0f, 3.0f))
					.Text(WarningText)
					.ToolTipText(WarningText)
				]
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SButton)
					.Text(FText::FromString("Open Project Settings"))
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("TinyText"))
					.OnClicked_Static(&OnOpenProjectSettings)
				]
			]
		];

	RowBuilder
		.CustomWidget()
		.NameContent()
		[
			NameWidget
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			ValueWidget
		];
}
