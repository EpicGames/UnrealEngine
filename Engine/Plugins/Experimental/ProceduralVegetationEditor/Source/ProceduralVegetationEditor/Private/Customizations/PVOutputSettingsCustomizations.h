// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IDetailPropertyRow;

class FPVOutputSettingsCustomizations : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	FText GetWarningTextForAssemblies() const;

private:
	void CustomizeNaniteAssembliesWidget(const TSharedPtr<IPropertyHandle>& ChildHandle, IDetailPropertyRow& RowBuilder);
	void CustomizeNaniteShapePreservationWidget(const TSharedPtr<IPropertyHandle>& ChildHandle, IDetailPropertyRow& RowBuilder);
};
