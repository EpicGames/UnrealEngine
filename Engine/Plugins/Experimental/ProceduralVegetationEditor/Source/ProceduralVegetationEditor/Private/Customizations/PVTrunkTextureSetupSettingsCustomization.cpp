// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVTrunkTextureSetupSettingsCustomization.h"

#include "ClassIconFinder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"

#include "ActorFactories/ActorFactoryBasicShape.h"

#include "AssetRegistry/IAssetRegistry.h"

#include "Nodes/PVTrunkTextureSetupSettings.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PVTrunkTextureSetupSettingsCustomization"

TSharedRef<IDetailCustomization> FPVTrunkTextureSetupSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FPVTrunkTextureSetupSettingsCustomization());
}

void FPVTrunkTextureSetupSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	SelectedSettings.Empty();
	
	const FName CategoryName("Baked Textures");
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName);

	FDetailWidgetRow& NewRow = Category.AddCustomRow(FText::GetEmpty());
	
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	for (const TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
	{
		UPVTrunkTextureSetupSettings* Settings = Cast<UPVTrunkTextureSetupSettings>(Object.Get());
		if (ensure(Settings))
		{
			SelectedSettings.Add(Settings);
		}
	}

	if (SelectedSettings.Num() == 1)
	{
		NewRow.ValueContent()
		.MaxDesiredWidth(120.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f)
			[
				SNew(SButton)
				.OnClicked(this, &FPVTrunkTextureSetupSettingsCustomization::OnBakeTexturesClicked)
				.IsEnabled_Raw(this, &FPVTrunkTextureSetupSettingsCustomization::BakeTexturesButtonEnabled)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("BakeTexturesButton", "Bake Textures"))
				]
			]
		];
	}
}

bool FPVTrunkTextureSetupSettingsCustomization::BakeTexturesButtonEnabled() const
{
	UPVTrunkTextureSetupSettings* Settings = !SelectedSettings.IsEmpty() ? SelectedSettings[0].Get() : nullptr;
	if (Settings)
	{
		return !Settings->GetTextureBaked();
	}
	
	return true;
}

FReply FPVTrunkTextureSetupSettingsCustomization::OnBakeTexturesClicked()
{
	UPVTrunkTextureSetupSettings* Settings = !SelectedSettings.IsEmpty() ? SelectedSettings[0].Get() : nullptr;

	if (!Settings || !IsValid(Settings))
	{
		return FReply::Handled();
	}

	Settings->BakeTextures();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

