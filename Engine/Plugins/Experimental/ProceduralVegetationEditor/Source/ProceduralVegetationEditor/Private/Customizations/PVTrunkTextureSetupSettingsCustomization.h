// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailPropertyRow.h"
#include "Nodes/PVTrunkTextureSetupSettings.h"

class FPVTrunkTextureSetupSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	bool BakeTexturesButtonEnabled() const;
	FReply OnBakeTexturesClicked();

private:
	TArray<TWeakObjectPtr<UPVTrunkTextureSetupSettings>> SelectedSettings;
};
