// Copyright Epic Games, Inc. All Rights Reserved.

#include "RichTextColorDecorator.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/SlateTextRun.h"

#include "Internationalization/Text.h"

#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"

TOptional<FLinearColor> ParseNamed(const FString& Name)
{
	if (Name == TEXT("red")) return FLinearColor::Red;
	if (Name == TEXT("green")) return FLinearColor::Green;
	if (Name == TEXT("blue")) return FLinearColor::Blue;
	if (Name == TEXT("yellow")) return FLinearColor::Yellow;
	if (Name == TEXT("white")) return FLinearColor::White;
	if (Name == TEXT("black")) return FLinearColor::Black;
	if (Name == TEXT("orange")) return FLinearColor(1.f, 0.5f, 0.f, 1.f);
	if (Name == TEXT("purple")) return FLinearColor(0.5f, 0.f, 0.5f, 1.f);

	return {};
}

bool GetAttr(const TMap<FString, FTextRange>& Meta, const FString& Key, FString& Out,
             const FString& Source)
{
	if (const FTextRange* Range = Meta.Find(Key))
	{
		Out = Source.Mid(Range->BeginIndex, Range->Len());
		return true;
	}
	return false;
}


bool GetAttr(const TMap<FString, FTextRange>& Meta, const FString& Key, int32& Out,
             const FString& Source)
{
	if (FString Value; GetAttr(Meta, Key, Value, Source))
	{
		Out = FCString::Atoi(*Value);
		return true;
	}
	return false;
}

FColorBlockTextRun::FColorBlockTextRun(
	const FRunInfo& InRunInfo,
	const TSharedRef<const FString>& InText,
	const FTextBlockStyle& InStyle,
	const FTextRange& InRange,
	const FLinearColor& InBg
)
	: FSlateTextRun(InRunInfo, InText, InStyle, InRange)
	, BgColor(InBg)
{}

TSharedRef<FColorBlockTextRun> FColorBlockTextRun::Create(
	const FRunInfo& InRunInfo,
	const TSharedRef<const FString>& InText,
	const FTextBlockStyle& InStyle,
	const FTextRange& InRange,
	const FLinearColor& InBg
)
{
	return MakeShared<FColorBlockTextRun>(InRunInfo, InText, InStyle, InRange, InBg);
}

int32 FColorBlockTextRun::OnPaint(const FPaintArgs& Args, const FTextArgs& TextArgs, const FGeometry& Allotted, const FSlateRect& Culling,
                                  FSlateWindowElementList& Out, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Measure the exact width of this run’s substring (the spaces inside <block>...</>)
	const FString RunText = Text->Mid(Range.BeginIndex, Range.Len());

	// Use the font set on this run (from your TextStyle)
	const FSlateFontMeasure& FM = *FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FVector2D Size(
		FM.Measure(RunText, Style.Font).X,
		FM.GetMaxCharacterHeight(Style.Font)
	);

	// The base class will draw the text at the proper offset; paint the bg at the same origin
	const FSlateBrush* Brush = FCoreStyle::Get().GetBrush("WhiteBrush");
	FSlateDrawElement::MakeBox(
		Out, LayerId,
		Allotted.ToPaintGeometry(Size, FSlateLayoutTransform()),
		Brush, ESlateDrawEffect::None, BgColor
	);

	return FSlateTextRun::OnPaint(Args, TextArgs, Allotted, Culling, Out, LayerId + 1, InWidgetStyle, bParentEnabled);
}

TSharedRef<FColorBlockDecorator> FColorBlockDecorator::CreateDecorator()
{
	return MakeShared<FColorBlockDecorator>();
}

bool FColorBlockDecorator::Supports(const FTextRunParseResults& RunParseResults, const FString& Text) const
{
	return RunParseResults.Name.Equals(TEXT("block"), ESearchCase::IgnoreCase);
}

TSharedRef<ISlateRun> FColorBlockDecorator::Create(
	const TSharedRef<FTextLayout>& TextLayout,
	const FTextRunParseResults& RunInfo,
	const FString& OriginalText,
	const TSharedRef<FString>& ModelText,
	const ISlateStyle* Style
)
{
	FLinearColor Color = FLinearColor::White;
	if (FString HexAttr; GetAttr(RunInfo.MetaData, TEXT("hex"), HexAttr, OriginalText))
	{
		Color = FLinearColor::FromSRGBColor(FColor::FromHex(HexAttr));
	}
	else if (FString NameAttr; GetAttr(RunInfo.MetaData, TEXT("name"), NameAttr, OriginalText))
	{
		if (TOptional<FLinearColor> Parsed = ParseNamed(NameAttr))
		{
			Color = Parsed.GetValue();
		}
	}

	FTextBlockStyle TextStyle = Style
		? Style->GetWidgetStyle<FTextBlockStyle>("NormalText")
		: FTextBlockStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"));
	TextStyle.SetColorAndOpacity(Color);

	FTextRange ModelRange;
	ModelRange.BeginIndex = ModelText->Len();
	ModelText->Append(OriginalText.Mid(RunInfo.ContentRange.BeginIndex, RunInfo.ContentRange.Len()));
	ModelRange.EndIndex = ModelText->Len();

	const FRunInfo Info(RunInfo.Name);
	return FColorBlockTextRun::Create(Info, ModelText, TextStyle, ModelRange, Color);
}
