// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/Text/ITextDecorator.h"
#include "Framework/Text/SlateTextRun.h"

class FColorBlockTextRun : public FSlateTextRun
{
public:
	FColorBlockTextRun(
		const FRunInfo& InRunInfo,
		const TSharedRef<const FString>& InText,
		const FTextBlockStyle& InStyle,
		const FTextRange& InRange,
		const FLinearColor& InBg
	);

	static TSharedRef<FColorBlockTextRun> Create(
		const FRunInfo& InRunInfo,
		const TSharedRef<const FString>& InText,
		const FTextBlockStyle& InStyle,
		const FTextRange& InRange,
		const FLinearColor& InBg
	);

	virtual int32 OnPaint(
		const FPaintArgs& Args,
		const FTextArgs& TextArgs,
		const FGeometry& Allotted,
		const FSlateRect& Culling,
		FSlateWindowElementList& Out,
		int32 LayerId,
		const FWidgetStyle& InWidgetStyle,
		bool bParentEnabled
	) const override;

private:
	FLinearColor BgColor;
};

class FColorBlockDecorator : public ITextDecorator
{
public:
	static TSharedRef<FColorBlockDecorator> CreateDecorator();

	virtual bool Supports(const FTextRunParseResults& RunParseResults, const FString& Text) const override;

	virtual TSharedRef<ISlateRun> Create(
		const TSharedRef<FTextLayout>& TextLayout,
		const FTextRunParseResults& RunInfo,
		const FString& OriginalText,
		const TSharedRef<FString>& ModelText,
		const ISlateStyle* Style
	) override;
};
