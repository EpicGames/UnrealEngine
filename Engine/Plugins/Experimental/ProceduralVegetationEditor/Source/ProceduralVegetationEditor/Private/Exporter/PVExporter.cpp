// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVExporter.h"

#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "PackagesDialog.h"
#include "ProceduralVegetationEditorModule.h"

#include "Facades/PVAttributesNames.h"

#include "GeometryCollection/GeometryCollection.h"

#include "Helpers/PVExportHelper.h"

#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/UObjectToken.h"

#include "Styling/SlateTypes.h"

#define LOCTEXT_NAMESPACE "PVExporter"

void FPVOutputEntryStats::SetupStats(const FManagedArrayCollection& InCollection)
{
	if (InCollection.HasGroup(PV::GroupNames::PointGroup))
	{
		NumPoints = InCollection.NumElements(PV::GroupNames::PointGroup);
	}
	if (InCollection.HasGroup(PV::GroupNames::BranchGroup))
	{
		NumBranches = InCollection.NumElements(PV::GroupNames::BranchGroup);
	}
	if (InCollection.HasGroup(FGeometryCollection::VerticesGroup))
	{
		NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);
	}
	if (InCollection.HasGroup(FGeometryCollection::FacesGroup))
	{
		NumTris = InCollection.NumElements(FGeometryCollection::FacesGroup);
	}
	if (InCollection.HasGroup(PV::GroupNames::BonesGroup))
	{
		NumBones = InCollection.NumElements(PV::GroupNames::BonesGroup);
	}
	if (InCollection.HasGroup(PV::GroupNames::FoliageNamesGroup))
	{
		NumFoliage = InCollection.NumElements(PV::GroupNames::FoliageNamesGroup);
	}
	if (InCollection.HasGroup(PV::GroupNames::FoliageGroup))
	{
		NumFoliageInstances = InCollection.NumElements(PV::GroupNames::FoliageGroup);
	}
}

void UPVExportEntry::Initialize(
	const TSharedPtr<FManagedArrayCollection>& InCollection,
	const TObjectPtr<UPCGNode> InNode,
	const bool InIsNodeSelected,
	const bool InHasFoliage
)
{
	check(InNode);

	bIsNodeSelected = InIsNodeSelected;
	Node = InNode;
	bExportFoliage = InHasFoliage;
	Settings = CastChecked<UPVExportSettings>(InNode->GetSettings());
	OutputCollection = InCollection;

	check(Settings && OutputCollection);
	
	Stats.SetupStats(*OutputCollection);
}

TSharedRef<FTokenizedMessage> FPVExporter::FScopedMessages::Error(const FText& ErrorMsg)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, ErrorMsg);
	Messages.Add(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FPVExporter::FScopedMessages::Warning(const FText& ErrorMsg)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, ErrorMsg);
	Messages.Add(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FPVExporter::FScopedMessages::Info(const FText& ErrorMsg)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info, ErrorMsg);
	Messages.Add(Message);
	return Message;
}

bool FPVExporter::FScopedMessages::ContainsErrors() const
{
	for (TSharedRef<FTokenizedMessage> Message : Messages)
	{
		if (Message->GetSeverity() == EMessageSeverity::Error)
		{
			return true;
		}
	}

	return false;
}

FPVExporter::FScopedMessages::~FScopedMessages()
{
	if (Messages.Num() > 0)
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		const TSharedRef<IMessageLogListing> PVEditorMessageLogListing = MessageLogModule.GetLogListing(PVEditor::MessageLogName);

		PVEditorMessageLogListing->ClearMessages();
		PVEditorMessageLogListing->AddMessages(Messages);
		PVEditorMessageLogListing->NotifyIfAnyMessages(
			ContainsErrors()
			? LOCTEXT("PVEMeshExportFailed", "Mesh Export Failed")
			: LOCTEXT("PVEMeshExportComplete", "Mesh Export Completed"),
			EMessageSeverity::Info,
			true
		);
	}
}


FPVExporter::FPVExporter(const TObjectPtr<UProceduralVegetation> InProceduralVegetation, const TArray<TObjectPtr<UPVExportEntry>>& InExportEntries)
	: ProceduralVegetation(InProceduralVegetation)
{
	ExportEntries = InExportEntries.FilterByPredicate(
		[](const TObjectPtr<UPVExportEntry>& Entry)
			{
				return Entry->bExport;
			}
	);
}

bool FPVExporter::Export()
{
	if (!ValidateExportEntries())
	{
		return false;
	}

	if (!ShouldOverwriteAssets())
	{
		return false;
	}

	FScopedSlowTask ProgressSlowTask(ExportEntries.Num(), LOCTEXT("ExportAsset_SlowTask", "Exporting Asset(s) ..."));
	ProgressSlowTask.MakeDialog(false);

	bool bExportFailed = false;
	for (int32 NodeIndex = 0; NodeIndex < ExportEntries.Num(); NodeIndex++)
	{
		const TObjectPtr<UPVExportEntry>& ExportEntry = ExportEntries[NodeIndex];
		if (ProgressSlowTask.ShouldCancel())
		{
			break;
		}

		const FText AssetCountText = ExportEntries.Num() > 1
			? FText::FromString(FString::Printf(TEXT(" [%d/%d]"), NodeIndex + 1, ExportEntries.Num()))
			: FText::FromString("");

		ProgressSlowTask.EnterProgressFrame(
			1.f,
			FText::Format(
				LOCTEXT("ExportingAsset", "Exporting \"{0}\"{1}"),
				FText::FromName(ExportEntry->Settings->ExportSettings.MeshName), AssetCountText
			)
		);

		FScopedSlowTask ExportCollectionSlowTask(1.f, LOCTEXT("GeneratingMesh", "Generating Mesh ..."));
		float PrevProgress = 0.f;

		const auto OnStatusUpdated = [&](const FText& Stage, const float Progress)-> bool
			{
				const float WorkThisFrame = Progress - PrevProgress;
				PrevProgress = Progress;

				ExportCollectionSlowTask.FrameMessage = Stage;

				ExportCollectionSlowTask.CompletedWork += ExportCollectionSlowTask.CurrentFrameScope;

				const float WorkRemaining = ExportCollectionSlowTask.TotalAmountOfWork - ExportCollectionSlowTask.CompletedWork;
				ExportCollectionSlowTask.CurrentFrameScope = FMath::Min(WorkRemaining, WorkThisFrame);
				ExportCollectionSlowTask.ForceRefresh();

				return !ProgressSlowTask.ShouldCancel();
			};

		TArray<FString> CreatedAssets;
		const PV::Export::EExportResult ExportResult = PV::Export::ExportCollectionAsMesh(
			ProceduralVegetation,
			*ExportEntry->OutputCollection,
			ExportEntry,
			CreatedAssets,
			OnStatusUpdated
		);

		if (ExportResult == PV::Export::EExportResult::Fail)
		{
			Messages.Error(FText::Format(
				LOCTEXT("ExportFailed_UnknownError", "Failed to export mesh \"{0}\" due to unknown error, export aborted."),
				FText::FromName(ExportEntry->Settings->ExportSettings.MeshName)
			));
			bExportFailed = true;
			break;
		}
		else if (ExportResult == PV::Export::EExportResult::Cancelled)
		{
			FString MeshNames = "{ ";
			for (int32 SkippedEntryIndex = NodeIndex; SkippedEntryIndex < ExportEntries.Num(); SkippedEntryIndex++)
			{
				MeshNames += ExportEntries[SkippedEntryIndex]->Settings->ExportSettings.MeshName.ToString();
				if (SkippedEntryIndex != (ExportEntries.Num() - 1))
				{
					MeshNames += ", ";
				}
			}
			MeshNames += " }";

			Messages.Warning(
				FText::Format(
					LOCTEXT("ExportFailed_ExportCanceled", "Failed to export meshes \"{0}\", operation canceled by user."),
					FText::FromString(MeshNames)
				));
			bExportFailed = true;
			break;
		}
		else if (ExportResult == PV::Export::EExportResult::Skipped)
		{
			Messages.Info(FText::Format(
				LOCTEXT("ExportSkipped", "Export of mesh \"{0}\" skipped due to conflicting asset at output location"),
				FText::FromName(ExportEntry->Settings->ExportSettings.MeshName)
			));
		}
		else
		{
			const TSharedRef<FTokenizedMessage> Message = Messages.Info(FText::Format(
				LOCTEXT("ExportComplete", "Mesh exported successfully \"{0}\""),
				FText::FromName(ExportEntry->Settings->ExportSettings.MeshName)
			));

			for (const FString& AssetPath : CreatedAssets)
			{
				Message->AddToken(FAssetNameToken::Create(AssetPath));
			}
		}
	}

	return !bExportFailed;
}

bool FPVExporter::ValidateExportEntries()
{
	bool bHasInvalidPath = false;
	for (const TObjectPtr<UPVExportEntry>& ExportEntry : ExportEntries)
	{
		if (FString PathError; !ExportEntry->Settings->ExportSettings.Validate(PathError))
		{
			TSharedRef<FTokenizedMessage> Error = Messages.Error(
				FText::Format(
					LOCTEXT("ExportFailed_InvalidPath", "Invalid export path for node \"{0}\": \"{1}\""),
					ExportEntry->Node->GetDefaultTitle(),
					FText::FromString(PathError)
				)
			);
			Error->AddToken(FUObjectToken::Create(ExportEntry->Settings));
			bHasInvalidPath = true;
		}
	}

	return !bHasInvalidPath;
}

bool FPVExporter::ShouldOverwriteAssets()
{
	TArray<FString> ExportPaths;
	ExportPaths.Reserve(ExportEntries.Num() * 2);

	for (const TObjectPtr<UPVExportEntry>& ExportEntry : ExportEntries)
	{
		const FPVExportParams& ExportParams = ExportEntry->Settings->ExportSettings;
		if (ExportParams.ReplacePolicy != EPVAssetReplacePolicy::Replace)
		{
			continue;
		}
		ExportPaths.Add(ExportParams.GetOutputMeshPackagePath());
		if (ExportParams.ExportMeshType == EPVExportMeshType::SkeletalMesh)
		{
			ExportPaths.Add(ExportParams.GetOutputSkeletonPackagePath());
			if (ExportParams.IsCollisionEnable())
			{
				ExportPaths.Add(ExportParams.GetOutputPhysicsAssetPackagePath());
			}
		}
	}

	TArray<UPackage*> ExistingPackages;
	ExistingPackages.Reserve(ExportPaths.Num());
	for (const FString& AssetPath : ExportPaths)
	{
		if (UPackage* ExistingPackage = LoadPackage(nullptr, *AssetPath, LOAD_NoWarn | LOAD_Quiet))
		{
			ExistingPackages.Add(ExistingPackage);
		}
	}

	if (ExistingPackages.Num() == 0)
	{
		return true;
	}

	FPackagesDialogModule& PackagesDialogModule = FModuleManager::LoadModuleChecked<FPackagesDialogModule>(TEXT("PackagesDialog"));
	PackagesDialogModule.CreatePackagesDialog(
		LOCTEXT("PackagesDialog_OverwriteFiles_Title", "WARNING: Assets will be overwritten"),
		LOCTEXT("PackagesDialog_OverwriteFiles_Message", "The following assets will be overwritten. Would you like to continue?"),
		/*InReadOnly=*/true
	);
	PackagesDialogModule.AddButton(
		DRT_Save,
		LOCTEXT("PackagesDialog_OverwriteFiles_ContinueButton", "Continue"),
		LOCTEXT("PackagesDialog_OverwriteFiles_ButtonTip", "Continues the export, overwrites any existing assets")
	);
	PackagesDialogModule.AddButton(
		DRT_Cancel,
		LOCTEXT("PackagesDialog_OverwriteFiles_CancelButton", "Cancel"),
		LOCTEXT("CancelDeleteButtonTip", "Cancel export")
	);

	for (UPackage* Package : ExistingPackages)
	{
		PackagesDialogModule.AddPackageItem(Package, ECheckBoxState::Checked);
	}

	const EDialogReturnType UserResponse = PackagesDialogModule.ShowPackagesDialog();
	return UserResponse == DRT_Save;
}

#undef LOCTEXT_NAMESPACE
