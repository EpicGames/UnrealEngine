// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/PVBaseSettings.h"
#include "Nodes/PVExportSettings.h"

#include "PVExporter.generated.h"

class UProceduralVegetation;



USTRUCT()
struct FPVOutputEntryStats
{
	GENERATED_BODY()
	
	UPROPERTY(VisibleAnywhere, Category="Stats", meta=(EditCondition="NumPoints != -1"))
	int32 NumPoints = -1;

	UPROPERTY(VisibleAnywhere, Category="Stats", meta=(EditCondition="NumBranches != -1"))
	int32 NumBranches = -1;

	UPROPERTY(VisibleAnywhere, Category="Stats", DisplayName="Num Vertices (Trunk)", meta=(EditCondition="NumVertices != -1"))
	int32 NumVertices = -1;

	UPROPERTY(VisibleAnywhere, Category="Stats", DisplayName="Num Triangles (Trunk)", meta=(EditCondition="NumTris != -1"))
	int32 NumTris = -1;

	UPROPERTY(VisibleAnywhere, Category="Stats", meta=(EditCondition="NumBones != -1"))
	int32 NumBones = -1;
	
	UPROPERTY(VisibleAnywhere, Category="Stats", meta=(EditCondition="NumFoliage != -1"))
	int32 NumFoliage = -1;

	UPROPERTY(VisibleAnywhere, Category="Stats", meta=(EditCondition="NumFoliageInstances != -1"))
	int32 NumFoliageInstances = -1;
	
	void SetupStats(const FManagedArrayCollection& InCollection);
};

UCLASS()
class UPVExportEntry : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Transient)
	bool bExport = true;

	UPROPERTY(Transient)
	bool bIsNodeSelected = false;

	UPROPERTY(Transient)
	TObjectPtr<UPCGNode> Node = nullptr;

	UPROPERTY(VisibleAnywhere, Category="Stats", DisplayName="Has Foliage")
	bool bExportFoliage = true;
	
	UPROPERTY(VisibleAnywhere, Category="Stats", meta=(ShowOnlyInnerProperties))
	FPVOutputEntryStats Stats;

	UPROPERTY(EditAnywhere, Category="Export Settings", meta=(ShowOnlyInnerProperties))
	TObjectPtr<UPVExportSettings> Settings = nullptr;

	TSharedPtr<FManagedArrayCollection> OutputCollection;

	void Initialize(
		const TSharedPtr<FManagedArrayCollection>& InCollection,
		const TObjectPtr<UPCGNode> InNode,
		bool InIsNodeSelected = false,
		bool InHasFoliage = true
	);
};

class FPVExporter
{
	struct FScopedMessages
	{
		TSharedRef<FTokenizedMessage> Error(const FText& ErrorMsg);
		TSharedRef<FTokenizedMessage> Warning(const FText& ErrorMsg);
		TSharedRef<FTokenizedMessage> Info(const FText& ErrorMsg);

		~FScopedMessages();

	private:
		bool ContainsErrors() const;

		TArray<TSharedRef<FTokenizedMessage>> Messages;
	};

public:
	FPVExporter(TObjectPtr<UProceduralVegetation> InProceduralVegetation, const TArray<TObjectPtr<UPVExportEntry>>& InExportEntries);

	bool Export();

private:
	bool ValidateExportEntries();
	bool ShouldOverwriteAssets();

private:
	TObjectPtr<UProceduralVegetation> ProceduralVegetation = nullptr;

	TArray<TObjectPtr<UPVExportEntry>> ExportEntries;

	FScopedMessages Messages;
};
