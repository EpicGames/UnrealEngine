// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Helpers/PVExportHelper.h"
 
#include "DynamicWindImportData.h"
#include "DynamicWindSkeletalData.h"
#include "Editor.h"
#include "EditorSupportDelegates.h"
#include "EngineAnalytics.h"
#include "IAssetTools.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "PlanarCut.h"
#include "ProceduralVegetation.h"
#include "ProceduralVegetationEditorModule.h"
#include "ProceduralVegetationLink.h"
#include "Params/PVExportParams.h"
#include "PVWindSettings.h"
#include "RenderUtils.h"
#include "Selection.h"
#include "SkinnedAssetCompiler.h"
#include "StaticMeshCompiler.h"
#include "UDynamicMesh.h"
 
#include "Animation/Skeleton.h"
 
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
 
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
 
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/MeshTransforms.h"
 
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#include "Exporter/PVExporter.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVPlantFacade.h"
#include "Facades/PVPointFacade.h"
 
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
 
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "GeometryScript/MeshBoneWeightFunctions.h"
#include "Helpers/PVAnalyticsHelper.h"
 
#include "Helpers/PVUtilities.h"
 
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "PhysicsEngine/ConstraintUtils.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "ProceduralVegetation/Private/Helpers/PVDistributionHelper.h"
#include "ProceduralVegetation/Private/Implementations/PVFoliage.h"
 
#include "Subsystems/AssetEditorSubsystem.h"

#include "UObject/Package.h"
 
#define LOCTEXT_NAMESPACE "PVExportHelper"
 
namespace PV::Export::Internal
{
	enum class EExportMeshStage
	{
		InitializingExportMesh,
		CombiningMeshes,
		CopyingMeshDataToOutputMesh,
		RecreatingBoneTree,
		ComputingBoneWeights,
		CopyingSkinWeightProfiles,
		BuildingNaniteAssemblies,
		ImportingDynamicWind,
		BuildingMesh,
		Complete,
		MAX
	};
 
	float CalculateMeshExportStageWeight(EExportMeshStage InStage, bool bIsBuildingNaniteAssemblies, bool bIsExportingSkeletalMesh)
	{
		switch (InStage)
		{
		case EExportMeshStage::InitializingExportMesh:
			return 1.f;
		case EExportMeshStage::CombiningMeshes:
			return bIsBuildingNaniteAssemblies ? 0.f : 1.f;
		case EExportMeshStage::CopyingMeshDataToOutputMesh:
			return 1.f;
		case EExportMeshStage::RecreatingBoneTree:
			return bIsExportingSkeletalMesh ? 0.2f : 0.f;
		case EExportMeshStage::ComputingBoneWeights:
			return bIsExportingSkeletalMesh ? 2.f : 0.f;
		case EExportMeshStage::CopyingSkinWeightProfiles:
			return bIsExportingSkeletalMesh ? 0.1f : 0.f;
		case EExportMeshStage::BuildingNaniteAssemblies:
			return bIsBuildingNaniteAssemblies ? 3.f : 0.f;
		case EExportMeshStage::ImportingDynamicWind:
			return bIsExportingSkeletalMesh ? 0.1f : 0.f;
		case EExportMeshStage::BuildingMesh:
			return bIsBuildingNaniteAssemblies ? 3.f : 4.f;
		case EExportMeshStage::Complete:
			return 0.5f;
		default:
			break;
		}
 
		check(false);
		return 0.f;
	}
 
	float CalculateMeshExportProgress(EExportMeshStage InStage, float StageProgress, bool bIsBuildingNaniteAssemblies, bool bIsExportingSkeletalMesh)
	{
		ensure(StageProgress >= 0 && StageProgress <= 1.01f);
		StageProgress = FMath::Clamp(StageProgress, 0.f, 1.f);
 
		float TotalWeightSum = 0;
		float ThisStageWeightSum = 0;
		for (int32 i = 0; i < static_cast<int32>(EExportMeshStage::MAX); ++i)
		{
			const float StatusWeight = CalculateMeshExportStageWeight(static_cast<EExportMeshStage>(i), bIsBuildingNaniteAssemblies, bIsExportingSkeletalMesh);
			TotalWeightSum += StatusWeight;
			if (i < static_cast<int32>(InStage))
			{
				ThisStageWeightSum += StatusWeight;
			}
		}
 
		const float ThisStatusWeight = CalculateMeshExportStageWeight(InStage, bIsBuildingNaniteAssemblies, bIsExportingSkeletalMesh);
		ThisStageWeightSum += ThisStatusWeight * StageProgress;
 
		return TotalWeightSum > 0
			? ThisStageWeightSum / TotalWeightSum
			: 0.f;
	}
 
	FText GetMeshExportStageText(EExportMeshStage InStage, bool bIsBuildingNaniteAssemblies, bool bIsExportingSkeletalMesh)
	{
		switch (InStage)
		{
		case EExportMeshStage::InitializingExportMesh:
			return LOCTEXT("ExportMeshStatus_InitializingExportMesh", "Initializing mesh data");
		case EExportMeshStage::CombiningMeshes:
			return LOCTEXT("ExportMeshStatus_CombiningMeshes", "Combining foliage meshes");
		case EExportMeshStage::CopyingMeshDataToOutputMesh:
			return LOCTEXT("ExportMeshStatus_CopyingMeshDataToOutputMesh", "Copying mesh data to output mesh");
		case EExportMeshStage::RecreatingBoneTree:
			return LOCTEXT("ExportMeshStatus_RecreatingBoneTree", "Recreating bone tree");
		case EExportMeshStage::ComputingBoneWeights:
			return LOCTEXT("ExportMeshStatus_ComputingBoneWeights", "Computing bone weights");
		case EExportMeshStage::CopyingSkinWeightProfiles:
			return LOCTEXT("ExportMeshStatus_CopyingSkinWeightProfiles", "Copying skin weight profiles");
		case EExportMeshStage::BuildingNaniteAssemblies:
			return LOCTEXT("ExportMeshStatus_BuildingNaniteAssemblies", "Building nanite assemblies");
		case EExportMeshStage::ImportingDynamicWind:
			return LOCTEXT("ExportMeshStatus_ImportingDynamicWind", "Importing dynamic wind");
		case EExportMeshStage::BuildingMesh:
		{
			if (!bIsBuildingNaniteAssemblies)
			{
				return LOCTEXT("ExportMeshStatus_BuildingMesh_Slow", "Building mesh (this might take a while)");
			}
			else
			{
				return LOCTEXT("ExportMeshStatus_BuildingMesh", "Building mesh");
			}
		}
		case EExportMeshStage::Complete:
			return LOCTEXT("ExportMeshStatus_Complete", "Complete");
		default:
			break;
		}
 
		check(false);
		return FText::FromString("Error");
	}
 
	void CleanupPackages(const TArray<UPackage*>& Packages)
	{
		TArray<UPackage*> PackagesToReload;
		PackagesToReload.Reserve(Packages.Num());
 
		TArray<UObject*> ObjectsToDelete;
		ObjectsToDelete.Reserve(Packages.Num());
 
		for (UPackage* Package : Packages)
		{
			if (Package->HasAnyPackageFlags(PKG_InMemoryOnly))
			{
				ObjectsToDelete.Add(Package->FindAssetInPackage());
			}
			else
			{
				PackagesToReload.Add(Package);
			}
		}
 
		if (PackagesToReload.Num() > 0)
		{
			FText UnloadError;
			UPackageTools::ReloadPackages(PackagesToReload, UnloadError, EReloadPackagesInteractionMode::AssumePositive);
		}
		
		if (ObjectsToDelete.Num() > 0)
		{
			ObjectTools::ForceDeleteObjects(ObjectsToDelete, false);
		}
	}
 
	enum class EGetValidAssetPathResult
	{
		Success,
		InvalidPath,
		ConflictingPackageExist,
		SkipAsset
	};
 
	EGetValidAssetPathResult GetValidAssetPath(EPVAssetReplacePolicy AssetReplacePolicy, UClass* AssetClass, FString& InOutPackageName, FString& InOutAssetName)
	{
		if (!FPackageName::IsValidTextForLongPackageName(InOutPackageName))
		{
			return EGetValidAssetPathResult::InvalidPath;
		}
 
		if (FindPackage(nullptr, *InOutPackageName) != nullptr || FPackageName::DoesPackageExist(InOutPackageName))
		{
			if (AssetReplacePolicy == EPVAssetReplacePolicy::Append)
			{
				IAssetTools::Get().CreateUniqueAssetName(InOutPackageName, TEXT(""), InOutPackageName, InOutAssetName);
			}
			else if (AssetReplacePolicy == EPVAssetReplacePolicy::Ignore)
			{
				return EGetValidAssetPathResult::SkipAsset;
			}
		}
 
		if (Utilities::DoesConflictingPackageExist(InOutPackageName, AssetClass))
		{
			return EGetValidAssetPathResult::ConflictingPackageExist;
		}
 
		return EGetValidAssetPathResult::Success;
	}
 
	UPackage* CreateAssetPackage(const FString& LongPackageName)
	{
		UPackage* Package = LoadPackage(nullptr, *LongPackageName, LOAD_None);
 
		if (Package == nullptr)
		{
			Package = CreatePackage(*LongPackageName);
		}

		// LoadPackage can return an in-memory UPackage stub even when no file exists on disk
		// (e.g. left behind by a prior failed LoadObject on the same path), so it isn't a
		// reliable signal that the asset is already saved. DoesPackageExist consults disk truth.
		if (!FPackageName::DoesPackageExist(LongPackageName))
		{
			Package->SetPackageFlags(PKG_NewlyCreated);
		}
		
		return Package;
	}

	bool IsEngineContentPath(const FString& InPackageOrObjectPath)
	{
		return InPackageOrObjectPath.StartsWith(TEXT("/Engine/"));
	}

	// Resolves the parent package path under which a generated companion mesh
	// for FoliageInstancePackagePath should live.
	//   - Non-engine source: original long package path (sibling to the source asset).
	//   - Engine source:     "<ExportMeshParent>/<ExportMeshShortName>_NaniteParts/Engine/<rest of source dir>".
	//                        Engine content is read-only on disk under Perforce, so the companion
	//                        mesh must be rebased into the user's project. The "Engine/" segment is
	//                        preserved so the origin is visible in the Content Browser and so
	//                        same-named assets in different engine folders do not collide.
	FString ResolveGeneratedMeshParentPath(
		const FString& FoliageInstancePackagePath,
		const FString& ExportMeshPackageName)
	{
		const FString LongFoliagePackagePath = FPackageName::GetLongPackagePath(FoliageInstancePackagePath);

		if (!IsEngineContentPath(FoliageInstancePackagePath))
		{
			return LongFoliagePackagePath;
		}

		const FString ExportParent    = FPackageName::GetLongPackagePath(ExportMeshPackageName);
		const FString ExportShortName = FPackageName::GetShortName(ExportMeshPackageName);

		FString RelativeEngineDir = LongFoliagePackagePath;
		RelativeEngineDir.RemoveFromStart(TEXT("/"));

		return FPaths::Combine(ExportParent, ExportShortName + TEXT("_NaniteParts"), RelativeEngineDir);
	}

	void DeletePackage(UPackage* InPackage)
	{
		TArray<UObject*> PackageObjects;
		GetObjectsWithPackage(InPackage, PackageObjects, EGetObjectsFlags::None);
		for (UObject* ExistingObject : PackageObjects)
		{
			ExistingObject->MarkAsGarbage();
			IAssetRegistry::Get()->AssetDeleted(ExistingObject);
		}
	}
 
	void ReplaceAssetInPackage(const TObjectPtr<UObject>& NewObject)
	{
		UPackage* Package = NewObject->GetPackage();
		
		const FString ObjectName = FPackageName::GetShortName(Package->GetName());
		TArray<UObject*> PackageObjects;
		GetObjectsWithPackage(Package, PackageObjects, EGetObjectsFlags::None);
		PackageObjects = PackageObjects.FilterByPredicate(
			[&](UObject* Object)
			{
				return Object != NewObject;
			}
		);
		for (UObject* ExistingObject : PackageObjects)
		{
			// Mimic ObjectTools::DeleteSingleObject. 
			// Deselect the old asset, close any open editors for the asset,
			// notify the asset registry, and clear relevant flags on object.
			//
			// Additionally call ObjectTools::ForceReplaceReferences to ensure
			// references to ExistingObject gets updated in the engine, and finally
			// rename the object so that we can rename the NewObject to the same location
			// as the old asset.

			if (GEditor)
			{
				GEditor->GetSelectedObjects()->Deselect(ExistingObject);

				if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					AssetEditorSubsystem->CloseAllEditorsForAsset(ExistingObject);
				}
			}

			// Technically shouldn't be possible for ExistingObject class to differ from NewObject class.
			// However, we check here as ObjectTools::ForceReplaceReferences only makes sense for objects of the same type.
			// We want to delete all other objects regardless.
			if (ExistingObject->IsA(NewObject->GetClass())) 
			{
				TArray<UObject*> ObjectsToReplace = { ExistingObject };
				ObjectTools::ForceReplaceReferences(NewObject, ObjectsToReplace);
			}

			ExistingObject->MarkPackageDirty();
			IAssetRegistry::Get()->AssetDeleted(ExistingObject);
			ExistingObject->ClearFlags(RF_Standalone | RF_Public);

			// Remove the deleted object out of the package and mark as garbage
			const FString NewName = MakeUniqueObjectName(GetTransientPackage(), ExistingObject->GetClass(), *(ExistingObject->GetName() + "_DEADOBJECT")).ToString();
			ExistingObject->Rename(*NewName, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional);
			ExistingObject->MarkAsGarbage();
		}

		NewObject->Rename(*ObjectName, Package, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional);

		// If this is a newly created asset then the asset registry will be notified by ExportCollectionAsMesh.
		// If not then we notify asset registry manually here to ensure it's aware of the swapped in asset.
		if (!Package->HasAnyPackageFlags(PKG_NewlyCreated)) 
		{
			IAssetRegistry::Get()->AssetCreated(NewObject);
		}
	}
 
	TObjectPtr<UDynamicMesh> CollectionToDynamicMesh(const FManagedArrayCollection& Collection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::CollectionToDynamicMesh);
		
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		if (Collection.NumElements(FGeometryCollection::TransformGroup) == 0)
		{
			return NewMesh;
		}
 
		const TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(Collection.NewCopy<FGeometryCollection>());
		if (!GeomCollection.IsValid())
		{
			return NewMesh;
		}
 
		const TManagedArray<FTransform3f>& BoneTransforms = Collection.GetAttribute<FTransform3f>(
			"Transform", FGeometryCollection::TransformGroup);
 
		TArray<int32> TransformIndices;
		TransformIndices.AddUninitialized(BoneTransforms.Num());
 
		int32 Idx = 0;
		for (int32& TransformIdx : TransformIndices)
		{
			TransformIdx = Idx++;
		}
 
		constexpr bool bCenterPivot = false;
		FTransform TransformOut;
		FDynamicMesh3 CombinedMesh;
		ConvertGeometryCollectionToDynamicMesh(CombinedMesh, TransformOut, bCenterPivot, *GeomCollection, false, BoneTransforms.GetConstArray(), true, TransformIndices);
 
		NewMesh->SetMesh(CombinedMesh);
 
		return NewMesh;
	}
 
	bool IsNaniteSupported()
	{
		return DoesPlatformSupportNanite(GMaxRHIShaderPlatform);
	}
 
	FGeometryScriptCopyMeshToAssetOptions GetCopyMeshToAssetOptions(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, const FMeshNaniteSettings& InNaniteSettings, const ENaniteShapePreservation InShapePreservation)
	{
		FGeometryScriptCopyMeshToAssetOptions Options;
		Options.bEmitTransaction = false;
		Options.bDeferMeshPostEditChange = true;
		Options.bEnableRecomputeTangents = false;
		Options.bReplaceMaterials = true;
		Options.NewMaterials = InMaterials;
 
		Options.NewNaniteSettings = InNaniteSettings;
		Options.NewNaniteSettings.bEnabled = IsNaniteSupported();
		Options.NewNaniteSettings.ShapePreservation = InShapePreservation;
		Options.bApplyNaniteSettings = true;
 
		return Options;
	}
 
	FString GetFoliageName(const Facades::FFoliageFacade& FoliageFacade, int32 Id)
	{
		Facades::FFoliageEntryData Data = FoliageFacade.GetFoliageEntry(Id);
		
		FPVFoliageInfo Info = FoliageFacade.GetFoliageInfo(Data.NameId);
		
		return Info.Mesh.ToString();
	}
 
	TArray<TObjectPtr<UMaterialInterface>> GetMaterialsFromCollection(const FManagedArrayCollection& Collection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::GetMaterialsFromCollection);
 
		TArray<TObjectPtr<UMaterialInterface>> Materials;
		if (Collection.HasAttribute("MaterialPath", FGeometryCollection::MaterialGroup))
		{
			const TManagedArray<FString>& MaterialPaths = Collection.GetAttribute<FString>("MaterialPath", FGeometryCollection::MaterialGroup);
			for (const FString& Path : MaterialPaths)
			{
				Materials.Add(LoadObject<UMaterialInterface>(nullptr, *Path, {}, LOAD_NoWarn | LOAD_Quiet));
			}
		}
		return Materials;
	}
 
	static bool SkeletalMeshToDynamicMesh(const USkeletalMesh* SkeletalMesh, const int32 LodIndex, FDynamicMesh3& ToDynamicMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::SkeletalMeshToDynamicMesh);
 
		if (SkeletalMesh->HasMeshDescription(LodIndex))
		{
			const FMeshDescription* SourceMesh = SkeletalMesh->GetMeshDescription(LodIndex);
			if (!SourceMesh)
			{
				return false;
			}
 
			FMeshDescriptionToDynamicMesh Converter;
			Converter.Convert(SourceMesh, ToDynamicMesh, true);
			return true;
		}
 
		return false;
	}
 
	bool CombineMeshInstancesToDynamicMesh(
		const FManagedArrayCollection& Collection,
		UDynamicMesh* DynamicMesh,
		TArray<TObjectPtr<UMaterialInterface>>& Materials,
		const FMeshInstanceCombined& OnMeshInstanceCombined,
		const TFunction<bool(float)>& OnProgressUpdated
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::CombineMeshInstancesToDynamicMesh);
 
		const Facades::FFoliageFacade FoliageFacade(Collection);
		const int32 NumInstances = FoliageFacade.NumFoliageEntries();
		const int32 NumFoliageNames = FoliageFacade.NumFoliageInfo();
		
		if (NumInstances == 0 || NumFoliageNames == 0)
		{
			return true;
		}
 
		struct FConvertFoliageToDynamicMeshResult
		{
			FString FoliageName;
			FDynamicMesh3 OutDynamicMesh;
			TArray<UMaterialInterface*> OutMaterials;
			FString OutMeshPath;
			bool bSuccess = true;
		};
 
		TArray<FConvertFoliageToDynamicMeshResult> ConvertFoliageToDynamicMeshResults;
		TMap<FString, int32> FoliageNameToDynamicMeshResult;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConvertFoliageToDynamicMeshes);
 
			ConvertFoliageToDynamicMeshResults.Reserve(NumFoliageNames);
			FoliageNameToDynamicMeshResult.Reserve(NumFoliageNames);
 
			for (int32 FoliageNameIndex = 0; FoliageNameIndex < NumFoliageNames; ++FoliageNameIndex)
			{
				FPVFoliageInfo Info = FoliageFacade.GetFoliageInfo(FoliageNameIndex);
				const FString FoliageName = Info.Mesh.ToString();
				
				if (FoliageName.IsEmpty())
				{
					continue;
				}
				
				if (FoliageNameToDynamicMeshResult.Contains(FoliageName))
				{
					UE_LOGF(LogProceduralVegetationEditor, Warning, "Duplicate foliage name %ls in foliage facade", *FoliageName);
					continue;
				}
 
				FConvertFoliageToDynamicMeshResult& Result = ConvertFoliageToDynamicMeshResults.AddDefaulted_GetRef();
				Result.FoliageName = FoliageName;
 
				FoliageNameToDynamicMeshResult.Add(FoliageName, ConvertFoliageToDynamicMeshResults.Num() - 1);
			}
 
			ParallelFor(ConvertFoliageToDynamicMeshResults.Num(), [&ConvertFoliageToDynamicMeshResults](int32 Index)
			{
				FConvertFoliageToDynamicMeshResult& Result = ConvertFoliageToDynamicMeshResults[Index];
 
				const FString& FoliageName = Result.FoliageName;
 
				if (const TObjectPtr<UStaticMesh> StaticMesh = LoadObject<UStaticMesh>(nullptr, *FoliageName, {}, LOAD_NoWarn | LOAD_Quiet))
				{
					FText OutErrorMessage;
					UE::Conversion::FStaticMeshConversionOptions Options;
					if (!StaticMeshToDynamicMesh(StaticMesh, Result.OutDynamicMesh, OutErrorMessage, Options))
					{
						Result.bSuccess = false;
						return;
					}
 
					for (auto Material : StaticMesh->GetStaticMaterials())
					{
						Result.OutMaterials.Add(Material.MaterialInterface);
					}
 
					Result.OutMeshPath = StaticMesh->GetFullName();
				}
				else if (const TObjectPtr<USkeletalMesh> SkeletalMesh = LoadObject<USkeletalMesh>(nullptr, *FoliageName, {}, LOAD_NoWarn | LOAD_Quiet))
				{
					if (!SkeletalMeshToDynamicMesh(SkeletalMesh, 0, Result.OutDynamicMesh))
					{
						Result.bSuccess = false;
						return;
					}
 
					for (auto Material : SkeletalMesh->GetMaterials())
					{
						Result.OutMaterials.Add(Material.MaterialInterface);
					}
 
					Result.OutDynamicMesh.Attributes()->RemoveAttribute("BoneIndex");
					Result.OutDynamicMesh.Attributes()->RemoveAttribute("BoneParentIndex");
					Result.OutMeshPath = SkeletalMesh->GetFullName();
				}
				else
				{
					Result.bSuccess = false;
				}
			});
 
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RemapMaterials);
 
				TSet<UMaterialInterface*> UniqueMaterials;
				UniqueMaterials.Reserve(ConvertFoliageToDynamicMeshResults.Num() * 2);
				for (const FConvertFoliageToDynamicMeshResult& Result : ConvertFoliageToDynamicMeshResults)
				{
					UniqueMaterials.Append(Result.OutMaterials);
				}
 
				for (UMaterialInterface* Material : UniqueMaterials)
				{
					Materials.AddUnique(Material);
				}
 
				for (FConvertFoliageToDynamicMeshResult& Result : ConvertFoliageToDynamicMeshResults)
				{
					if (Result.bSuccess == false)
					{
						continue;
					}
					
					UE::Geometry::FDynamicMeshMaterialAttribute* MaterialIDs = Result.OutDynamicMesh.Attributes()->GetMaterialID();
					if (!MaterialIDs)
					{
						continue;
					}
					
					for (int32 Tid : Result.OutDynamicMesh.TriangleIndicesItr())
					{
						const int32 MaterialID = MaterialIDs->GetValue(Tid);
						if (MaterialID == INDEX_NONE)
						{
							continue;
						}
 
						check(Result.OutMaterials.IsValidIndex(MaterialID));
 
						const int32 NewMaterialID = Materials.Find(Result.OutMaterials[MaterialID]);
						check(NewMaterialID != INDEX_NONE);
 
						MaterialIDs->SetValue(Tid, NewMaterialID);
					}
				}
			}
		}
 
		const float ConvertToDynamicMeshProgress = 0.1f;
		if (!OnProgressUpdated(ConvertToDynamicMeshProgress))
		{
			return false;
		}
 
		struct FGenerateDynamicMeshesResult
		{
			FDynamicMesh3 DynamicMesh;
			int32 ConvertToDynamicMeshResultIndex;
		};
 
		TArray<FGenerateDynamicMeshesResult> GenerateDynamicMeshesResults;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GenerateDynamicMeshInstances);
 
			GenerateDynamicMeshesResults.SetNum(NumInstances);
			ParallelFor(NumInstances, [
				&GenerateDynamicMeshesResults,
				&FoliageFacade, 
				&FoliageNameToDynamicMeshResult,
				&ConvertFoliageToDynamicMeshResults
			](int32 Index)
			{
				const FString FoliageName = GetFoliageName(FoliageFacade, Index);
 
				const int32* PrevResultIndex = FoliageNameToDynamicMeshResult.Find(FoliageName);
				if (PrevResultIndex == nullptr)
				{
					return;
				}
 
				FConvertFoliageToDynamicMeshResult& ConvertFoliageToDynamicMeshResult = ConvertFoliageToDynamicMeshResults[*PrevResultIndex];
				if (!ConvertFoliageToDynamicMeshResult.bSuccess)
				{
					return;
				}

				if (FMath::Abs(FoliageFacade.GetFoliageEntry(Index).Scale) <= UE_KINDA_SMALL_NUMBER)
				{
					return;
				}

				FGenerateDynamicMeshesResult& Result = GenerateDynamicMeshesResults[Index];
				Result.DynamicMesh = ConvertFoliageToDynamicMeshResult.OutDynamicMesh;
				Result.ConvertToDynamicMeshResultIndex = *PrevResultIndex;
 
				const int32 FoliageParentBoneID = FoliageFacade.GetParentBoneID(Index);
				UE::Geometry::FDynamicMeshAttributeSet* MeshAttributes = Result.DynamicMesh.Attributes();
				UE::Geometry::FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = MeshAttributes->GetBoneParentIndices();
				if (FoliageParentBoneID != INDEX_NONE && BoneParentIndices != nullptr)
				{
					// Reparent root bone according to the instanced transform
					BoneParentIndices->SetValue(0, FoliageParentBoneID);
				}
 
				const FTransform Transform = FoliageFacade.GetFoliageTransform(Index);
				MeshTransforms::ApplyTransform(Result.DynamicMesh, Transform, true);
			});
		}
 
		bool bWasCanceled = false;
		DynamicMesh->EditMesh([&](FDynamicMesh3& Mesh)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AppendDynamicMeshes);
 
			for (int32 i = 0; i < GenerateDynamicMeshesResults.Num(); ++i)
			{
				const FGenerateDynamicMeshesResult& GenerateDynamicMeshesResult = GenerateDynamicMeshesResults[i];
				const FDynamicMesh3& MeshToAppend = GenerateDynamicMeshesResult.DynamicMesh;
				if (MeshToAppend.VertexCount() == 0)
				{
					continue;
				}
 
				const int32 VertexStart = Mesh.VertexCount();
				Mesh.AppendWithOffsets(MeshToAppend);
				
				const FConvertFoliageToDynamicMeshResult& ConvertFoliageToDynamicMeshResult = ConvertFoliageToDynamicMeshResults[GenerateDynamicMeshesResult.ConvertToDynamicMeshResultIndex];
				OnMeshInstanceCombined(ConvertFoliageToDynamicMeshResult.OutMeshPath, i, VertexStart, Mesh.VertexCount());
 
				const float CombineMeshesProgress = ConvertToDynamicMeshProgress + i / (float)NumInstances * (1.f - ConvertToDynamicMeshProgress);
				if (!OnProgressUpdated(CombineMeshesProgress))
				{
					bWasCanceled = true;
					return;
				}
			}
		});
 
		return !bWasCanceled;
	}
 
	bool ContainsFoliageName(const Facades::FFoliageFacade& FoliageFacade, const FString& InFoliageName)
	{
		int32 NumInstances = FoliageFacade.NumFoliageEntries();
			
		for(int32 Id = 0; Id < NumInstances; Id++)
		{
			FString FoliageName = GetFoliageName(FoliageFacade, Id);
			if(FoliageName == InFoliageName)
			{
				return true;
			}
		}
 
		return false;
	}
 
	TArray<FString> GetUsedFoliage(const Facades::FFoliageFacade& FoliageFacade)
	{
		TArray<FString> OutFoliageNames;
		OutFoliageNames.Reserve(FoliageFacade.NumFoliageInfo());
 
		for(int32 FoliageNameIndex = 0; FoliageNameIndex < FoliageFacade.NumFoliageInfo(); ++FoliageNameIndex)
		{
			FPVFoliageInfo Info = FoliageFacade.GetFoliageInfo(FoliageNameIndex);
			FString FoliageName = Info.Mesh.ToString();
			
			if (ContainsFoliageName(FoliageFacade, FoliageName))
			{
				OutFoliageNames.Add(FoliageName);
			}
		}
 
		return OutFoliageNames;
	}
 
	void BuildNaniteAssemblyData(
		const FManagedArrayCollection& Collection,
		UStaticMesh* StaticMesh,
		TArray<UPackage*>& OutModifiedPackages
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::BuildNaniteAssemblyData_StaticMesh);
 
		//Create NaniteAssemblyDataBuilder and the main static mesh materials to it
		FNaniteAssemblyDataBuilder AssemblyBuilder;
		for (const FStaticMaterial& Material : StaticMesh->GetStaticMaterials())
		{
			AssemblyBuilder.AddMaterialSlot(Material.MaterialInterface);
		}
 
		const Facades::FFoliageFacade FoliageFacade(Collection);
		const TArray<FString> UsedFoliage = GetUsedFoliage(FoliageFacade);
 
		TMap<FString, int32> MeshNamePartMap;
		MeshNamePartMap.Reserve(UsedFoliage.Num());
		TMap<FString, TArray<TObjectPtr<UMaterialInterface>>> MeshMaterialsMap;
		MeshMaterialsMap.Reserve(UsedFoliage.Num() * 2);
 
		const auto AddStaticMeshToAssembly = [&](const FString& FoliageInstanceObjectPath, TObjectPtr<UStaticMesh> InStaticMesh)
		{
			const int32 PartIndex = AssemblyBuilder.FindOrAddPart(InStaticMesh.GetPath());
			MeshNamePartMap.FindOrAdd(FoliageInstanceObjectPath) = PartIndex;
 
			TArray<TObjectPtr<UMaterialInterface>>& MeshMaterials = MeshMaterialsMap.FindOrAdd(FoliageInstanceObjectPath);
			for (const FStaticMaterial& Material : InStaticMesh->GetStaticMaterials())
			{
				MeshMaterials.AddUnique(Material.MaterialInterface);
			}
		};
		
		for (const FString& FoliageInstanceObjectPath : UsedFoliage)
		{
			if (FoliageInstanceObjectPath.IsEmpty())
			{
				continue;
			}
			
			TObjectPtr<UStaticMesh> PartStaticMesh = LoadObject<UStaticMesh>(nullptr, *FoliageInstanceObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
			if (PartStaticMesh)
			{
				AddStaticMeshToAssembly(FoliageInstanceObjectPath, PartStaticMesh);
				continue;
			}
 
			const FString FoliageInstancePackagePath = FPackageName::ObjectPathToPackageName(FoliageInstanceObjectPath);
			const FString LongFoliagePackagePath = Internal::ResolveGeneratedMeshParentPath(
				FoliageInstancePackagePath, StaticMesh->GetOutermost()->GetName());
			const FString ShortFoliagePackageName = FPackageName::GetShortName(FoliageInstancePackagePath);
 
			const FString StaticMeshName = "SKM_" + ShortFoliagePackageName;
			const FString StaticMeshPackagePath = FPaths::Combine(LongFoliagePackagePath, StaticMeshName);
			const FString StaticMeshObjectPath = FString::Printf(TEXT("%s.%s"), *StaticMeshPackagePath, *StaticMeshName);
 
			PartStaticMesh = LoadObject<UStaticMesh>(nullptr, StaticMeshObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
			if (PartStaticMesh)
			{
				const TObjectPtr<USkeletalMesh> SourceSkeletalMesh = LoadObject<USkeletalMesh>(nullptr, *FoliageInstanceObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
				if (SourceSkeletalMesh)
				{
					const FString SkeletalMeshFile = FPackageName::LongPackageNameToFilename(SourceSkeletalMesh->GetPackage()->GetName(), FPackageName::GetAssetPackageExtension());
					const FString StaticMeshFile = FPackageName::LongPackageNameToFilename(PartStaticMesh->GetPackage()->GetName(), FPackageName::GetAssetPackageExtension());
					const FDateTime SkeletalMeshTime = IFileManager::Get().GetTimeStamp(*SkeletalMeshFile);
					const FDateTime StaticMeshTime = IFileManager::Get().GetTimeStamp(*StaticMeshFile);
					if (SkeletalMeshTime > StaticMeshTime)
					{
						// Source skeletal mesh was modified after the static mesh was generated; fall through to regenerate
						PartStaticMesh = nullptr;
					}
				}

				if (PartStaticMesh)
				{
					AddStaticMeshToAssembly(FoliageInstanceObjectPath, PartStaticMesh);
					continue;
				}
			}
 
			const TObjectPtr<USkeletalMesh> PartSkeletalMesh = LoadObject<USkeletalMesh>(nullptr, *FoliageInstanceObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
			if (!PartSkeletalMesh)
			{
				UE_LOGF(LogProceduralVegetationEditor, Warning, "Could not find foliage instance %ls", *FoliageInstanceObjectPath);
				continue;
			}
 
			FDynamicMesh3 PartDynamicMesh;
			if (!SkeletalMeshToDynamicMesh(PartSkeletalMesh, 0, PartDynamicMesh))
			{
				UE_LOGF(LogProceduralVegetationEditor, Warning, "Failed to convert foliage instance \"%ls\" to dynamic mesh", *FoliageInstancePackagePath);
				continue;
			}
					
			UPackage* StaticMeshPackage = Internal::CreateAssetPackage(*StaticMeshPackagePath);
			OutModifiedPackages.Add(StaticMeshPackage);
 
			PartStaticMesh = NewObject<UStaticMesh>(StaticMeshPackage, FName(StaticMeshName), RF_Standalone | RF_Public);
 
			TArray<TObjectPtr<UMaterialInterface>> PartMaterials;
			for (auto Material : PartSkeletalMesh->GetMaterials())
			{
				PartMaterials.Add(Material.MaterialInterface);
			}
 
			const FGeometryScriptCopyMeshToAssetOptions Options = GetCopyMeshToAssetOptions(
				PartMaterials, 
				PartStaticMesh->GetNaniteSettings(), 
				NaniteVoxelsSupported() ? ENaniteShapePreservation::Voxelize : ENaniteShapePreservation::PreserveArea
			);
 
			TObjectPtr<UDynamicMesh> DynamicMesh = NewObject<UDynamicMesh>();
			DynamicMesh->SetMesh(MoveTemp(PartDynamicMesh));
 
			EGeometryScriptOutcomePins Output;
			UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(
				DynamicMesh, 
				PartStaticMesh, 
				Options, 
				{}, 
				Output
			);

			if (Output != EGeometryScriptOutcomePins::Success)
			{
				OutModifiedPackages.Remove(StaticMeshPackage);
				PartStaticMesh->MarkAsGarbage();
				StaticMeshPackage->MarkAsGarbage();
				continue;
			}
 
			AddStaticMeshToAssembly(FoliageInstanceObjectPath, PartStaticMesh);
		}
 
		AddNodeToBuilder(AssemblyBuilder, Collection, MeshNamePartMap, MeshMaterialsMap);
		AssemblyBuilder.ApplyToStaticMesh(*StaticMesh);
	}
 
	void BuildNaniteAssemblyData(
		const FManagedArrayCollection& Collection,
		USkeletalMesh* SkeletalMesh,
		TArray<UPackage*>& OutModifiedPackages
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::BuildNaniteAssemblyData_SkeletalMesh);
 
		//Create NaniteAssemblyDataBuilder and the main static mesh materials to it
		FNaniteAssemblyDataBuilder AssemblyBuilder;
		for (const FSkeletalMaterial& Material : SkeletalMesh->GetMaterials())
		{
			AssemblyBuilder.AddMaterialSlot(Material.MaterialInterface);
		}
 
		const Facades::FFoliageFacade FoliageFacade(Collection);
		const TArray<FString> UsedFoliage = GetUsedFoliage(FoliageFacade);
 
		TMap<FString, int32> MeshNamePartMap;
		MeshNamePartMap.Reserve(UsedFoliage.Num());
		TMap<FString, TArray<TObjectPtr<UMaterialInterface>>> MeshMaterialsMap;
		MeshMaterialsMap.Reserve(UsedFoliage.Num() * 2);
 
		const auto AddSkeletalMeshToAssembly = [&](const FString& FoliageInstanceObjectPath, TObjectPtr<USkeletalMesh> InSkeletalMesh)
		{
			const int32 PartIndex = AssemblyBuilder.FindOrAddPart(InSkeletalMesh.GetPath());
			MeshNamePartMap.FindOrAdd(FoliageInstanceObjectPath) = PartIndex;
 
			TArray<TObjectPtr<UMaterialInterface>>& MeshMaterials = MeshMaterialsMap.FindOrAdd(FoliageInstanceObjectPath);
			for (const FSkeletalMaterial& Material : InSkeletalMesh->GetMaterials())
			{
				MeshMaterials.AddUnique(Material.MaterialInterface);
			}
		};
 
		for (const FString& FoliageInstanceObjectPath : UsedFoliage)
		{
			if (FoliageInstanceObjectPath.IsEmpty())
			{
				continue;
			}
			
			TObjectPtr<USkeletalMesh> PartSkeletalMesh = LoadObject<USkeletalMesh>(nullptr, *FoliageInstanceObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
			if (PartSkeletalMesh)
			{
				AddSkeletalMeshToAssembly(FoliageInstanceObjectPath, PartSkeletalMesh);
				continue;
			}
 
			const FString FoliageInstancePackagePath = FPackageName::ObjectPathToPackageName(FoliageInstanceObjectPath);
			const FString LongFoliagePackagePath = Internal::ResolveGeneratedMeshParentPath(
				FoliageInstancePackagePath, SkeletalMesh->GetOutermost()->GetName());
			const FString ShortFoliagePackageName = FPackageName::GetShortName(FoliageInstancePackagePath);
 
			const FString SkeletalMeshName = "SKM_" + ShortFoliagePackageName;
			const FString SkeletalMeshPackagePath = FPaths::Combine(LongFoliagePackagePath, SkeletalMeshName);
			const FString SkeletalMeshObjectPath = FString::Printf(TEXT("%s.%s"), *SkeletalMeshPackagePath, *SkeletalMeshName);
 
			const FString SkeletonName = "SK_" + ShortFoliagePackageName;
			const FString SkeletonPackagePath = FPaths::Combine(LongFoliagePackagePath, SkeletonName);
			const FString SkeletonObjectPath = FString::Printf(TEXT("%s.%s"), *SkeletonPackagePath, *SkeletonName);
 
			PartSkeletalMesh = LoadObject<USkeletalMesh>(nullptr, SkeletalMeshObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
			TObjectPtr<USkeleton> Skeleton = LoadObject<USkeleton>(nullptr, SkeletonObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
			if (PartSkeletalMesh && Skeleton)
			{
				const TObjectPtr<UStaticMesh> SourceStaticMesh = LoadObject<UStaticMesh>(nullptr, *FoliageInstanceObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
				if (SourceStaticMesh)
				{
					const FString StaticMeshFile = FPackageName::LongPackageNameToFilename(SourceStaticMesh->GetPackage()->GetName(), FPackageName::GetAssetPackageExtension());
					const FString SkeletalMeshFile = FPackageName::LongPackageNameToFilename(PartSkeletalMesh->GetPackage()->GetName(), FPackageName::GetAssetPackageExtension());
					const FDateTime StaticMeshTime = IFileManager::Get().GetTimeStamp(*StaticMeshFile);
					const FDateTime SkeletalMeshTime = IFileManager::Get().GetTimeStamp(*SkeletalMeshFile);
					if (StaticMeshTime > SkeletalMeshTime)
					{
						// Source static mesh was modified after the skeletal mesh was generated; fall through to regenerate
						PartSkeletalMesh = nullptr;
					}
				}
			}

			const auto DeleteNewlyCreatedPackage = [&](UPackage* InPackage)
			{
				if (InPackage && OutModifiedPackages.Remove(InPackage) > 0)
				{
					DeletePackage(InPackage);
				}
			};
 
			UPackage* SkeletonPackage = nullptr;
			if (!Skeleton)
			{
				SkeletonPackage = Internal::CreateAssetPackage(*SkeletonPackagePath);
				OutModifiedPackages.Add(SkeletonPackage);
 
				Skeleton = NewObject<USkeleton>(SkeletonPackage, FName(SkeletonName), RF_Standalone | RF_Public);
 
				if (PartSkeletalMesh)
				{
					PartSkeletalMesh->SetSkeleton(Skeleton);
					Skeleton->RecreateBoneTree(PartSkeletalMesh);
				}
			}
 
			if (!PartSkeletalMesh)
			{
				const TObjectPtr<UStaticMesh> PartStaticMesh = LoadObject<UStaticMesh>(nullptr, *FoliageInstanceObjectPath, {}, LOAD_NoWarn | LOAD_Quiet);
				if (!PartStaticMesh)
				{
					DeleteNewlyCreatedPackage(SkeletonPackage);
					UE_LOGF(LogProceduralVegetationEditor, Warning, "Could not find foliage instance %ls", *FoliageInstanceObjectPath);
					continue;
				}
 
				FDynamicMesh3 PartDynamicMesh;
				FText OutErrorMessage;
				UE::Conversion::FStaticMeshConversionOptions Options;
				if (!StaticMeshToDynamicMesh(PartStaticMesh, PartDynamicMesh, OutErrorMessage, Options))
				{
					DeleteNewlyCreatedPackage(SkeletonPackage);
					UE_LOGF(LogProceduralVegetationEditor, Warning, "Failed to convert foliage instance \"%ls\" to dynamic mesh", *FoliageInstancePackagePath);
					continue;
				}
 
				UPackage* SkeletalMeshPackage = Internal::CreateAssetPackage(*SkeletalMeshPackagePath);
				OutModifiedPackages.Add(SkeletalMeshPackage);
				PartSkeletalMesh = NewObject<USkeletalMesh>(SkeletalMeshPackage, FName(SkeletalMeshName), RF_Standalone | RF_Public);
				PartSkeletalMesh->SetSkeleton(Skeleton);
 
				TArray<TObjectPtr<UMaterialInterface>> PartMaterials;
				for (auto Material : PartStaticMesh->GetStaticMaterials())
				{
					PartMaterials.Add(Material.MaterialInterface);
				}
 
				if (!ConvertToDefaultSkeletalMesh(PartSkeletalMesh, PartDynamicMesh, PartMaterials))
				{
					UE_LOGF(LogProceduralVegetationEditor, Warning, "Failed to create part skeletal mesh %ls.", *PartSkeletalMesh->GetFullName());
					DeleteNewlyCreatedPackage(SkeletalMeshPackage);
					DeleteNewlyCreatedPackage(SkeletonPackage);
					continue;
				}
			}
			
			AddSkeletalMeshToAssembly(FoliageInstanceObjectPath, PartSkeletalMesh);
		}
 
		AddNodeToBuilder(AssemblyBuilder, Collection, MeshNamePartMap, MeshMaterialsMap);
		AssemblyBuilder.ApplyToSkeletalMesh(*SkeletalMesh);
	}
 
	void AddNodeToBuilder(FNaniteAssemblyDataBuilder& AssemblyBuilder, const FManagedArrayCollection& Collection,
		const TMap<FString, int32>& InMeshNamePartMap, const TMap<FString, TArray<TObjectPtr<UMaterialInterface>>>& InMeshMaterialsMap)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::AddNodeToBuilder);
 
		Facades::FFoliageFacade FoliageFacade(Collection);
		const int32 NumInstances = FoliageFacade.NumFoliageEntries();
			
		for (int32 Id = 0; Id < NumInstances; Id++)
		{
			const FString FoliageName = GetFoliageName(FoliageFacade, Id);
 
			const int32* PartIndex = InMeshNamePartMap.Find(FoliageName);
			if (PartIndex == nullptr)
			{
				continue;
			}

			if (FMath::Abs(FoliageFacade.GetFoliageEntry(Id).Scale) <= UE_KINDA_SMALL_NUMBER)
			{
				continue;
			}

			const FTransform Transform = FoliageFacade.GetFoliageTransform(Id);
			const UE::Geometry::FTransformSRT3d GeoTransform(Transform);
 
			const int32 FoliageParentBoneID = FoliageFacade.GetParentBoneID(Id);
				
			FNaniteAssemblyBoneInfluence Influence;
			Influence.BoneIndex = FoliageParentBoneID;
			Influence.BoneWeight = 1.0;
				
			AssemblyBuilder.AddNode(*PartIndex, FTransform3f(GeoTransform), ENaniteAssemblyNodeTransformSpace::Local, { Influence });
 
			//Add the materials for part to the assembly builder
			auto* FoliageMaterials = InMeshMaterialsMap.Find(FoliageName);
			if (FoliageMaterials == nullptr)
			{
				continue;
			}
 
			int32 LocalMaterialIndex = 0;
			for (auto PartMaterial : *FoliageMaterials)
			{
				int32 PartMaterialIndex = AssemblyBuilder.GetMaterialSlots().IndexOfByPredicate(
					[&] (const auto& Slot) { return Slot.Material == PartMaterial; }
				);
				if (PartMaterialIndex == INDEX_NONE)
				{
					PartMaterialIndex = AssemblyBuilder.AddMaterialSlot(PartMaterial);
				}
 
				AssemblyBuilder.RemapPartMaterial(*PartIndex, LocalMaterialIndex, PartMaterialIndex);
				LocalMaterialIndex++;
			}
		}
	}
 
	bool ConvertToDefaultSkeletalMesh(USkeletalMesh* SkeletalMesh, FDynamicMesh3& Mesh, const TArray<TObjectPtr<UMaterialInterface>>& Materials)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::ConvertToDefaultSkeletalMesh);
 
		TObjectPtr<UDynamicMesh> DynamicMesh = NewObject<UDynamicMesh>();
		DynamicMesh->SetMesh(MoveTemp(Mesh));
		
		//Figure out the correct parent bone index
		int ParentBoneIndex = INDEX_NONE;
		UE::Geometry::FDynamicMeshAttributeSet* MeshAttributes = DynamicMesh->GetMeshPtr()->Attributes();
		MeshAttributes->EnableBones(1);
 
		UE::Geometry::FDynamicMeshBoneNameAttribute* BoneNames = MeshAttributes->GetBoneNames();
		UE::Geometry::FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = MeshAttributes->GetBoneParentIndices();
		UE::Geometry::FDynamicMeshBonePoseAttribute* BonePoses = MeshAttributes->GetBonePoses();
 
		int BoneIndex = 0;
		BoneNames->SetValue(BoneIndex, "Root");
		BonePoses->SetValue(BoneIndex, FTransform::Identity);
		BoneParentIndices->SetValue(BoneIndex, ParentBoneIndex);
		
		//Convert the dynamic mesh to static mesh
		EGeometryScriptOutcomePins Output;
		FGeometryScriptCopyMeshToAssetOptions Options = GetCopyMeshToAssetOptions(Materials, SkeletalMesh->GetNaniteSettings(), NaniteVoxelsSupported() ? ENaniteShapePreservation::Voxelize : ENaniteShapePreservation::PreserveArea);
		Options.BoneHierarchyMismatchHandling = EGeometryScriptBoneHierarchyMismatchHandling::CreateNewReferenceSkeleton;
		
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToSkeletalMesh(
			DynamicMesh,
			SkeletalMesh,
			Options,
			{},
			Output
		);

		if (Output != EGeometryScriptOutcomePins::Success)
		{
			return false;
		}
 
		TObjectPtr<USkeleton> Skeleton = SkeletalMesh->GetSkeleton();
		Skeleton->RecreateBoneTree(SkeletalMesh);
 
		const FGeometryScriptBoneWeightProfile SkinProfile = FGeometryScriptBoneWeightProfile();
		FGeometryScriptSmoothBoneWeightsOptions SmoothBoneWeightsOptions;
		SmoothBoneWeightsOptions.DistanceWeighingType = EGeometryScriptSmoothBoneWeightsType::GeodesicVoxel;
		SmoothBoneWeightsOptions.MaxInfluences = 2;
		SmoothBoneWeightsOptions.VoxelResolution = 512;
		UGeometryScriptLibrary_MeshBoneWeightFunctions::ComputeSmoothBoneWeights(
			DynamicMesh,
			Skeleton,
			SmoothBoneWeightsOptions,
			SkinProfile
		);
 
		const FName ProfileName = SkinProfile.GetProfileName();
		UGeometryScriptLibrary_StaticMeshFunctions::CopySkinWeightProfileToSkeletalMesh(
			DynamicMesh,
			SkeletalMesh,
			ProfileName,
			ProfileName,
			{},
			{},
			Output
		);

		if (Output != EGeometryScriptOutcomePins::Success)
		{
			return false;
		}

		return true;
	}
 
	void AddCollisionToStaticMesh(const FManagedArrayCollection& Collection, TObjectPtr<UStaticMesh> ExportMesh)
	{
		UBodySetup* BodySetup = ExportMesh->GetBodySetup();
		if(!BodySetup)
		{
			BodySetup = NewObject<UBodySetup>(ExportMesh);
		}
		else
		{
			BodySetup->RemoveSimpleCollision();
		}
			
		FKConvexElem ConvexElem;
 
		FGeometryCollection GeoCollection;
		Collection.CopyTo(&GeoCollection);
		
		TArray<FVector> Vertices;
		for (const auto&Vertex : GeoCollection.Vertex)
		{
			Vertices.Add(FVector(Vertex));
		}
		
		ConvexElem.VertexData = Vertices;
		ConvexElem.UpdateElemBox();
		BodySetup->AggGeom.ConvexElems.Add(ConvexElem);
 
		BodySetup->bHasCookedCollisionData = true;
		BodySetup->InvalidatePhysicsData();
		BodySetup->CreatePhysicsMeshes();
 
		ExportMesh->SetBodySetup(BodySetup);
	}
 
	bool ExportCollectionToStaticMesh(
		TObjectPtr<UStaticMesh> ExportMesh,
		const FManagedArrayCollection& Collection,
		ENaniteShapePreservation InShapePreservation,
		bool bBuildNaniteAssemblies,
		bool bShouldExportFoliage,
		bool bCollision,
		TArray<UPackage*>& OutModifiedPackages,
		const FStatusReportCallback& StatusReportCallback
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::ExportCollectionToStaticMesh);
 
		const auto UpdateStage = [&](EExportMeshStage NewStage, float StageProgress)->bool
		{
			return StatusReportCallback(GetMeshExportStageText(NewStage, bBuildNaniteAssemblies, false), CalculateMeshExportProgress(NewStage, StageProgress, bBuildNaniteAssemblies, false));
		};
 
		FManagedArrayCollection ExportedCollection;
		TObjectPtr<UDynamicMesh> DynamicMesh;
		TArray<TObjectPtr<UMaterialInterface>> Materials;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InitExportMesh);
 
			if (!UpdateStage(EExportMeshStage::InitializingExportMesh, 0.f))
			{
				return false;
			}
 
			Collection.CopyTo(&ExportedCollection);
 
			DynamicMesh = CollectionToDynamicMesh(ExportedCollection);
 
			if (bCollision)
			{
				AddCollisionToStaticMesh(ExportedCollection, ExportMesh);
			}
 
			if (!UpdateStage(EExportMeshStage::InitializingExportMesh, 0.5f))
			{
				return false;
			}
 
			Materials = GetMaterialsFromCollection(ExportedCollection);
 
			if (!UpdateStage(EExportMeshStage::InitializingExportMesh, 1.f))
			{
				return false;
			}
		}
		
		// Combine all the foliage instances into one dynamic mesh if not building nanite assemblies
		if (!bBuildNaniteAssemblies && bShouldExportFoliage)
		{
			if (!UpdateStage(EExportMeshStage::CombiningMeshes, 0.f))
			{
				return false;
			}
			if (!CombineMeshInstancesToDynamicMesh(
				ExportedCollection,
				DynamicMesh,
				Materials,
				[](FString MeshName, int32 InstanceID, int32 VertexStart, int32 VertexCount) {},
				[&](float Progress) { return UpdateStage(EExportMeshStage::CombiningMeshes, Progress); }
			))
			{
				return false;
			}
		}
 
		EGeometryScriptOutcomePins Output;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CopyMeshToStaticMesh);
 
			if (!UpdateStage(EExportMeshStage::CopyingMeshDataToOutputMesh, 1.f))
			{
				return false;
			}
 
			FGeometryScriptCopyMeshToAssetOptions Options = GetCopyMeshToAssetOptions(Materials, ExportMesh->GetNaniteSettings(), InShapePreservation);
			UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(DynamicMesh, ExportMesh, Options, {}, Output);
		}
 
		if (bBuildNaniteAssemblies && bShouldExportFoliage)
		{
			if (!UpdateStage(EExportMeshStage::BuildingNaniteAssemblies, 1.f))
			{
				return false;
			}
 
			if (NaniteAssembliesSupported())
			{
				//Build the nanite assemblies
            	BuildNaniteAssemblyData(ExportedCollection, ExportMesh, OutModifiedPackages);
			}
			else
			{
				UE_LOGF(LogProceduralVegetationEditor, Warning, "Failed to build Nanite Assemblies, because neither Nanite Foliage nor Nanite Assemblies are enabled for the project.");
			}
		}
 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildStaticMesh);
			if (!UpdateStage(EExportMeshStage::BuildingMesh, 1.f))
			{
				return false;
			}
			
			ExportMesh->Build(true);
			FStaticMeshCompilingManager::Get().FinishAllCompilation();
		}
 
		UpdateStage(EExportMeshStage::Complete, 1.f);
		return true;
	}
 
	void BuildWindImportData(FDynamicWindSkeletalImportData& OutImportData, const FManagedArrayCollection& Collection, const TArray<PV::Facades::FBoneNode>& Bones)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::BuildWindImportData);
 
		Facades::FBranchFacade BranchFacade(Collection);
 
		for (const auto& Bone : Bones)
		{
			int32 SimulationGroupIndex = BranchFacade.GetBranchSimulationGroupIndex(Bone.BranchIndex);
			OutImportData.Joints.Add({Bone.BoneName, SimulationGroupIndex});
		}
	}
 
	bool ExportCollectionToSkeletalMesh(
		TObjectPtr<USkeletalMesh> ExportMesh,
		const FManagedArrayCollection& Collection,
		ENaniteShapePreservation InShapePreservation,
		TObjectPtr<const UPVWindSettings> InWindSettings,
		TObjectPtr<UPhysicsAsset> InPhysicsAsset,
		bool bBuildNaniteAssemblies,
		bool bShouldExportFoliage,
		EPVCollisionGeneration CollisionGeneration,
		TArray<UPackage*>& OutModifiedPackages,
		const FStatusReportCallback& StatusReportCallback
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::ExportCollectionToSkeletalMesh);
 
		const auto UpdateStage = [&](EExportMeshStage NewStage, float StageProgress)->bool
		{
			return StatusReportCallback(
				GetMeshExportStageText(NewStage, bBuildNaniteAssemblies, true), 
				CalculateMeshExportProgress(NewStage, StageProgress, bBuildNaniteAssemblies, true)
			);
		};
 
		FManagedArrayCollection ExportedCollection;
		TObjectPtr<UDynamicMesh> DynamicMesh;
		TArray<TObjectPtr<UMaterialInterface>> Materials;
		TArray<int32> VertexBoneIDs;
		TArray<Facades::FBoneNode> BoneNodes;
		int32 FoliageVertexStart = 0;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InitExportMesh);
 
			if (!UpdateStage(EExportMeshStage::InitializingExportMesh, 0.f))
			{
				return false;
			}
 
			Collection.CopyTo(&ExportedCollection);
 
			DynamicMesh = CollectionToDynamicMesh(ExportedCollection);
 
			if (!UpdateStage(EExportMeshStage::InitializingExportMesh, 0.5f))
			{
				return false;
			}
 
			Materials = GetMaterialsFromCollection(ExportedCollection);
 
			UE::Geometry::FDynamicMeshAttributeSet* MeshAttributes = DynamicMesh->GetMeshPtr()->Attributes();
 
			Facades::FBoneFacade BoneFacade = Facades::FBoneFacade(ExportedCollection);
			BoneNodes = BoneFacade.GetBoneDataFromCollection();
 
			// if bones already not created with bone reduction node create bones with full density
			if (BoneNodes.IsEmpty())
			{
				BoneFacade.CreateBoneData(BoneNodes, 0);
			}
 
			if (bShouldExportFoliage)
			{
				AssignBoneIDsToFoliage(BoneNodes, ExportedCollection);
			}
 
			const int32 BoneCount = BoneNodes.Num();
			MeshAttributes->EnableBones(BoneCount);
 
			TArray<int32> VertexPointIDs;
			BoneFacade.GetPointIds(VertexPointIDs);
			
			// Assign the Bone id to every vertex in the base mesh through VertexPointIDs
			for (const int32& PointID : VertexPointIDs)
			{
				if (Facades::FBoneNode* BoneNode = Facades::FBoneFacade::FindClosestBone(ExportedCollection, BoneNodes, PointID))
				{
					VertexBoneIDs.Add(BoneNode->BoneIndex);
				}
				else
				{
					VertexBoneIDs.Add(INDEX_NONE);
					UE_LOGF(LogProceduralVegetationEditor, Log, "Invalid Bone Assigned to PointID %i", PointID);
				}
			}
 
			FoliageVertexStart = VertexBoneIDs.Num();
 
			UE::Geometry::FDynamicMeshBoneNameAttribute* BoneNames = MeshAttributes->GetBoneNames();
			UE::Geometry::FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = MeshAttributes->GetBoneParentIndices();
			UE::Geometry::FDynamicMeshBonePoseAttribute* BonePoses = MeshAttributes->GetBonePoses();
 
			for (int i = 0; i < BoneCount; i++)
			{
				BoneNames->SetValue(i, BoneNodes[i].BoneName);
				BonePoses->SetValue(i, BoneNodes[i].BoneTransform);
				BoneParentIndices->SetValue(i, BoneNodes[i].ParentBoneIndex);
			}
 
			if (!UpdateStage(EExportMeshStage::InitializingExportMesh, 1.f))
			{
				return false;
			}
		}
 
		if (!bBuildNaniteAssemblies && bShouldExportFoliage)
		{
			if (!UpdateStage(EExportMeshStage::CombiningMeshes, 0.f))
			{
				return false;
			}
 
			Facades::FFoliageFacade FoliageFacade(ExportedCollection);
 
			// Combine all the foliage instances into one dynamic mesh if not building nanite assemblies
			if (!CombineMeshInstancesToDynamicMesh(
				ExportedCollection,
				DynamicMesh,
				Materials,
				[&](const FString& MeshName, int32 InstanceID, int32 VertexStart, int32 VertexCount)
				{
					// Assign parent bone id to foliage vertices
					int32 FoliageParentBoneID = FoliageFacade.GetParentBoneID(InstanceID);
 
					for (int VertexID = VertexStart; VertexID < VertexCount; ++VertexID)
					{
						VertexBoneIDs.Add(FoliageParentBoneID);
					}
				},
				[&](float Progress) { return UpdateStage(EExportMeshStage::CombiningMeshes, Progress); }
			))
			{
				return false;
			}
		}
 
		// Convert the dynamic mesh to static mesh
		EGeometryScriptOutcomePins Output;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CopyMeshToSkeletalMesh);
 
			if (!UpdateStage(EExportMeshStage::CopyingMeshDataToOutputMesh, 1.f))
			{
				return false;
			}
 
			FGeometryScriptCopyMeshToAssetOptions Options = GetCopyMeshToAssetOptions(Materials, ExportMesh->GetNaniteSettings(), InShapePreservation);
			Options.BoneHierarchyMismatchHandling = EGeometryScriptBoneHierarchyMismatchHandling::CreateNewReferenceSkeleton;
			UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToSkeletalMesh(
				DynamicMesh,
				ExportMesh,
				Options,
				{},
				Output
			);
		}
 
		TObjectPtr<USkeleton> Skeleton = ExportMesh->GetSkeleton();
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RecreateBoneTree);
			if (!UpdateStage(EExportMeshStage::RecreatingBoneTree, 1.f))
			{
				return false;
			}
			Skeleton->RecreateBoneTree(ExportMesh);
		}
 
		const FGeometryScriptBoneWeightProfile SkinProfile = FGeometryScriptBoneWeightProfile();
		const FName ProfileName = SkinProfile.GetProfileName();
 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ComputeSmoothBoneWeights);
 
			if (!UpdateStage(EExportMeshStage::ComputingBoneWeights, 1.f))
			{
				return false;
			}
 
			FGeometryScriptSmoothBoneWeightsOptions SmoothBoneWeightsOptions;
			SmoothBoneWeightsOptions.DistanceWeighingType = EGeometryScriptSmoothBoneWeightsType::DirectDistance;
			SmoothBoneWeightsOptions.MaxInfluences = 3;
			SmoothBoneWeightsOptions.Stiffness = 0.5;
 
			UGeometryScriptLibrary_MeshBoneWeightFunctions::ComputeSmoothBoneWeights(
				DynamicMesh,
				Skeleton,
				SmoothBoneWeightsOptions,
				SkinProfile
			);
 
			if (!UpdateStage(EExportMeshStage::ComputingBoneWeights, 0.8f))
			{
				return false;
			}
 
			// Here we make the correction to the weights
			RemoveUnwantedSkinWeights(DynamicMesh, ProfileName, VertexBoneIDs, FoliageVertexStart, BoneNodes);
 
			if (!UpdateStage(EExportMeshStage::ComputingBoneWeights, 1.f))
			{
				return false;
			}
		}
		
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CopySkinWeightProfileToSkeletalMesh);
 
			if (!UpdateStage(EExportMeshStage::CopyingSkinWeightProfiles, 1.f))
			{
				return false;
			}
 
			FGeometryScriptCopySkinWeightProfileToAssetOptions Options;
			Options.bEmitTransaction = false;
			Options.bDeferMeshPostEditChange = true;
			UGeometryScriptLibrary_StaticMeshFunctions::CopySkinWeightProfileToSkeletalMesh(
				DynamicMesh,
				ExportMesh,
				ProfileName,
				ProfileName,
				Options,
				{},
				Output
			);
		}
 
		if (bBuildNaniteAssemblies && bShouldExportFoliage)
		{
			if (!UpdateStage(EExportMeshStage::BuildingNaniteAssemblies, 1.f))
			{
				return false;
			}
 
			if (NaniteAssembliesSupported())
			{
				//Build the nanite assemblies
				BuildNaniteAssemblyData(ExportedCollection, ExportMesh, OutModifiedPackages);
			}
			else
			{
				UE_LOGF(LogProceduralVegetationEditor, Warning, "Failed to build Nanite Assemblies, because neither Nanite Foliage nor Nanite Assemblies are enabled for the project.");
			}
		}
 
		if (InPhysicsAsset)
		{
			SetupPhysicAsset(Collection, CollisionGeneration, InPhysicsAsset, BoneNodes);
		}
 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicWind);
 
			if (!UpdateStage(EExportMeshStage::ImportingDynamicWind, 1.f))
			{
				return false;
			}
 
			const bool bWindDataExist = ExportMesh->HasAssetUserDataOfClass(UDynamicWindSkeletalData::StaticClass());
			const bool bOverwriteWindSettings = InWindSettings ? InWindSettings->bOverwriteExisting : false;
			
			FDynamicWindSkeletalImportData ImportData;
			BuildWindImportData(ImportData, ExportedCollection, BoneNodes);
 
			if (InWindSettings)
			{
				ImportData.SimulationGroups = InWindSettings->SimulationGroupData;
			}
		
			if (bWindDataExist && !bOverwriteWindSettings)
			{
				//Get Existing WindSettings
				UDynamicWindSkeletalData* WindSkeletalData = Cast<UDynamicWindSkeletalData>(ExportMesh->GetAssetUserDataOfClass(UDynamicWindSkeletalData::StaticClass()));
				if (WindSkeletalData)
				{
					ImportData.SimulationGroups = WindSkeletalData->SimulationGroups;
				}
			}
 
			if (!DynamicWind::ImportSkeletalData(*ExportMesh, ImportData))
			{
				UE_LOGF(LogProceduralVegetationEditor, Warning, "Failed to build wind data");
			}
		}
 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildSkinnedMesh);
			if (!UpdateStage(EExportMeshStage::BuildingMesh, 1.f))
			{
				return false;
			}
 
			ExportMesh->Build();
			FSkinnedAssetCompilingManager::Get().FinishAllCompilation();
		}
 
		UpdateStage(EExportMeshStage::Complete, 1.f);
		return true;
	}
 
	void AttachProceduralVegetationLink(UObject* InExportedMesh, const TObjectPtr<UProceduralVegetation>& InProceduralVegetation)
	{
		if (IInterface_AssetUserData* IAssetUserData = Cast<IInterface_AssetUserData>(InExportedMesh))
		{
			UProceduralVegetationLink* Data = NewObject<UProceduralVegetationLink>(InExportedMesh);
			Data->Source = InProceduralVegetation;
		
			IAssetUserData->AddAssetUserData(Data);	
		}
	}
 
	void AssignBoneIDsToFoliage(const TArray<Facades::FBoneNode>& Bones, FManagedArrayCollection& Collection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::AssignBoneIDsToFoliage);
 
		Facades::FPointFacade PointFacade(Collection);
		Facades::FFoliageFacade FoliageFacade(Collection);
		
		int32 NumInstances = FoliageFacade.NumFoliageEntries();
 
		auto FindBoneById = ( [&](const int32 Id)
		{
			const Facades::FBoneNode* BoneNode = Bones.FindByPredicate([Id](const Facades::FBoneNode& Node)
			{
				return Node.BoneIndex == Id;
			});
			return BoneNode;
		});
		
		for(int32 Id = 0; Id < NumInstances; Id++)
		{
			Facades::FFoliageEntryData Data = FoliageFacade.GetFoliageEntry(Id);
			float FoliageLFR = Data.LengthFromRoot;
 
			TArray<int32> BoneIDs;
			for (Facades::FBoneNode BoneNode : Bones)
			{
				if (Data.BranchId == BoneNode.BranchIndex)
				{
					BoneIDs.Add(BoneNode.BoneIndex);
				}
			}
 
			bool bBoneIDAssigned = false;
			for (const int32& BoneId : BoneIDs)
			{
				const Facades::FBoneNode* BoneNode = FindBoneById(BoneId);
				check(BoneNode);
				
				int PointIndex = BoneNode->PointIndex;
				const Facades::FBoneNode* BoneParentNode = FindBoneById(BoneNode->ParentBoneIndex);
				
				int ParentPointIndex = BoneParentNode ? BoneParentNode->PointIndex : INDEX_NONE;
				float BoneLFR = PointFacade.GetLengthFromRoot(PointIndex);
				float BoneParentLFR = PointFacade.GetLengthFromRoot(ParentPointIndex);
 
				if (FoliageLFR <= BoneLFR && FoliageLFR >= BoneParentLFR)
				{
					FoliageFacade.SetParentBoneID(Id, BoneId);
					bBoneIDAssigned = true;
					break;
				}
			}
 
			if (!bBoneIDAssigned)
			{
				UE_LOGF(LogProceduralVegetationEditor, Warning, "No bone assigned for foliage Id {%i} , Branch Bone count {%i}", Id , BoneIDs.Num());
			}
		}
	}
 
	void RemoveUnwantedSkinWeights(TObjectPtr<UDynamicMesh> DynamicMesh, const FName ProfileName, const TArray<int32>& VertexBoneIDs, int32 FoliageVertexStart, const TArray<Facades::FBoneNode>& BoneNodes)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::RemoveUnwantedSkinWeights);
 
		auto InMesh = DynamicMesh->GetMeshPtr();
		check(InMesh)
		const int32 NumVertices = VertexBoneIDs.Num();
		
		DynamicMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute *SkinWeights = EditMesh.Attributes()->GetSkinWeightsAttribute(ProfileName);
 
			ParallelFor(NumVertices, [&](const int32 VertexIdx)
			{
				int32 BoneID = VertexBoneIDs[VertexIdx];
 
				if (BoneID != INDEX_NONE)
				{
					int32 ParentBoneID = BoneNodes[BoneID].ParentBoneIndex;
				
					UE::AnimationCore::FBoneWeights Weights;
					SkinWeights->GetValue(VertexIdx, Weights);
 
					auto NextBone = BoneNodes.FindByPredicate([&](const Facades::FBoneNode& Node)
					{
						return Node.BoneIndex == BoneID + 1 && Node.BranchIndex == BoneNodes[BoneID].BranchIndex;
					});
 
					for (int32 WeightID = Weights.Num() - 1 ; WeightID >= 0; WeightID--)
					{
						auto Weight = Weights[WeightID];
 
						bool bValidBone = (Weight.GetBoneIndex() == ParentBoneID);
						bValidBone |= (Weight.GetBoneIndex() == BoneID);
						bValidBone |= NextBone && (Weight.GetBoneIndex() == NextBone->BoneIndex);
						
						if (!bValidBone || VertexIdx > FoliageVertexStart || BoneID == 0)
						{
							UE::AnimationCore::FBoneWeightsSettings Settings;
							Settings.SetNormalizeType(UE::AnimationCore::EBoneWeightNormalizeType::Always);
							Weights.RemoveBoneWeight(Weight.GetBoneIndex(),Settings);
						}
					}
 
					if (VertexIdx > FoliageVertexStart || BoneID == 0 || Weights.Num() == 0)
					{
						UE::AnimationCore::FBoneWeightsSettings Settings;
						Settings.SetNormalizeType(UE::AnimationCore::EBoneWeightNormalizeType::Always);
						Weights.SetBoneWeight(UE::AnimationCore::FBoneWeight(BoneID, 1.0f), Settings);
					}
					
					SkinWeights->SetNewValue(VertexIdx, Weights);
				}
				else
				{
					UE_LOGF(LogProceduralVegetationEditor, Log, "Invalid bone index found while removing bones for vertex %i", VertexIdx);
				}
			});
				
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
 
	void SetupPhysicAsset(const FManagedArrayCollection& Collection, const EPVCollisionGeneration& CollisionGeneration,UPhysicsAsset* PhysicsAsset, TArray<Facades::FBoneNode> InBones)
	{
		if (!PhysicsAsset)
		{
			UE_LOGF(LogProceduralVegetationEditor, Warning, "No physics asset specified to create physics bodies");
			return;
		}
 
		TArray<TObjectPtr<USkeletalBodySetup>> SkeletalBodySetups;
		
		TArray<Facades::FBoneNode> ReducedBones;
		Facades::FBoneFacade BoneFacade = Facades::FBoneFacade(Collection);
		Facades::FPointFacade PointFacade = Facades::FPointFacade(Collection);
		Facades::FPlantFacade PlantFacade = Facades::FPlantFacade(Collection);
		
		BoneFacade.CreateBoneData(ReducedBones, 1.0, false);
 
		for (const Facades::FBoneNode&  Node : ReducedBones)
		{
			USkeletalBodySetup* BodySetup = NewObject<USkeletalBodySetup>(PhysicsAsset);
				
			auto MappedBone = InBones.FindByPredicate([Node](const Facades::FBoneNode& IterNode){ return Node.PointIndex == IterNode.PointIndex; });
 
			if (!MappedBone)
			{
				UE_LOGF(LogProceduralVegetationEditor, Log, "MappedBone not found for BodySetup");
				continue;
			}
 
			if (!InBones.IsValidIndex(MappedBone->ParentBoneIndex))
			{
				UE_LOGF(LogProceduralVegetationEditor, Log, "MapedBone not found for BodySetup");
				continue;
			}
 
			bool bTrunk = PlantFacade.IsTrunkIndex(Node.BranchIndex);
 
			if (bTrunk || CollisionGeneration == EPVCollisionGeneration::AllGenerations)
			{
				BodySetup->BoneName = MappedBone->BoneName;
				
				FKSphylElem Element;
				
				FVector BasePosition = Node.AbsolutePosition;
				float Radius = PointFacade.GetPointScale(Node.PointIndex);
				if (ReducedBones.IsValidIndex(Node.ParentBoneIndex))
				{
					BasePosition = ReducedBones[Node.ParentBoneIndex].AbsolutePosition;
					Radius = PointFacade.GetPointScale(Node.ParentBoneIndex);
				}
				
				float Length = FVector::Dist(Node.AbsolutePosition ,  BasePosition);
				Element.Length = Length;
				Element.Radius = Radius * 1.25f;
				
				FVector Direction = (BasePosition - Node.AbsolutePosition);
				Direction.Normalize();
 
				FVector Axis = FVector::UpVector;
				FQuat Quat = FQuat::FindBetween(Axis, Direction);
 
				Element.Center = Direction * Length * 0.5;
				Element.Rotation = Quat.Rotator();
 
				if (Element.Length > 0)
				{
					BodySetup->AggGeom.SphylElems.Add(Element);
					BodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
					BodySetup->CreatePhysicsMeshes();
					SkeletalBodySetups.Add(BodySetup);
				}
			}
		}
 
		PhysicsAsset->SkeletalBodySetups = SkeletalBodySetups;
		PhysicsAsset->UpdateBodySetupIndexMap();
		PhysicsAsset->UpdateBoundsBodiesArray();
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}
}
 
namespace PV::Export
{
	EExportResult ExportCollectionAsMesh(
		const TObjectPtr<UProceduralVegetation> InProceduralVegetation,
		const FManagedArrayCollection& Collection,
		const TObjectPtr<UPVExportEntry>& ExportEntry,
		TArray<FString>& OutCreatedAssets,
		const FStatusReportCallback& StatusReportCallback
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PV::Export::ExportCollectionAsMesh);
		
		const FPVExportParams& ExportParams = ExportEntry->Settings->ExportSettings;
 
		TArray<UPackage*> ModifiedPackages;
 
		const FDateTime ExportStartTime = FDateTime::Now();
		TArray<FAnalyticsEventAttribute> AnalyticsEventAttribute = Analytics::GatherExportCommonAttributes(Collection, ExportParams);
		Analytics::SendExportStartEvent(AnalyticsEventAttribute);
 
		const auto ExitWithResult = [&](EExportResult InResult)->EExportResult
		{
			const static auto ExportResultToAnalyticsResult = [](EExportResult InExportResult)->PV::Analytics::EExportResult
			{
				switch (InExportResult)
				{
					case EExportResult::Success:
						return PV::Analytics::EExportResult::Success;
					case EExportResult::Fail:
						return PV::Analytics::EExportResult::Failed;
					case EExportResult::Cancelled:
						return PV::Analytics::EExportResult::Failed;
					case EExportResult::Skipped:
						return PV::Analytics::EExportResult::Skipped;
				}
				ensure(false);
				return PV::Analytics::EExportResult::Failed;
			};
 
			if (InResult != EExportResult::Success)
			{
				Internal::CleanupPackages(ModifiedPackages);
			}
 
			const double ExportTime = (FDateTime::Now() - ExportStartTime).GetTotalSeconds();
			Analytics::SendExportFinishedEvent(MoveTemp(AnalyticsEventAttribute), ExportTime, ExportResultToAnalyticsResult(InResult));
			return InResult;
		};
 
		const auto ExitWithAssetPathResult = [&](Internal::EGetValidAssetPathResult InAssetPathResult)->EExportResult
		{
			if (InAssetPathResult == Internal::EGetValidAssetPathResult::SkipAsset)
			{
				return ExitWithResult(EExportResult::Skipped);
			}
 
			return ExitWithResult(EExportResult::Fail);
		};
 
		Internal::EGetValidAssetPathResult AssetPathResult;
		
		FString MeshName = ExportParams.MeshName.ToString();
		FString MeshPackageName = ExportParams.GetOutputMeshPackagePath();
		AssetPathResult = Internal::GetValidAssetPath(ExportParams.ReplacePolicy, ExportParams.GetMeshClass(), MeshPackageName, MeshName);
		if (AssetPathResult != Internal::EGetValidAssetPathResult::Success)
		{
			return ExitWithAssetPathResult(AssetPathResult);
		}
		
		const bool bCreateNaniteFoliage = ExportParams.bCreateNaniteFoliage && NaniteAssembliesSupported();
		const bool bShouldExportFoliage = ExportEntry->bExportFoliage;
 
		if (ExportParams.ExportMeshType == EPVExportMeshType::StaticMesh)
		{
			const TObjectPtr<UStaticMesh> ExportMesh = NewObject<UStaticMesh>(Internal::CreateAssetPackage(MeshPackageName), NAME_None, RF_Public | RF_Standalone);
 
			if (!PV::Export::Internal::ExportCollectionToStaticMesh(
				ExportMesh,
				Collection,
				ExportParams.NaniteShapePreservation,
				bCreateNaniteFoliage,
				bShouldExportFoliage,
				ExportParams.bCollision,
				ModifiedPackages,
				StatusReportCallback
			))
			{
				ExportMesh->MarkAsGarbage();
				return ExitWithResult(EExportResult::Cancelled);
			}
 
			Internal::ReplaceAssetInPackage(ExportMesh);
			ModifiedPackages.Add(ExportMesh->GetOutermost());
 
			OutCreatedAssets.Add(MeshPackageName);
 
			Internal::AttachProceduralVegetationLink(ExportMesh, InProceduralVegetation);
		}
		else if (ExportParams.ExportMeshType == EPVExportMeshType::SkeletalMesh)
		{
			FString SkeletonName = ExportParams.GetOutputSkeletonName();
			FString SkeletonPackageName = ExportParams.GetOutputSkeletonPackagePath();
			AssetPathResult = Internal::GetValidAssetPath(ExportParams.ReplacePolicy, USkeleton::StaticClass(), SkeletonPackageName, SkeletonName);
			if (AssetPathResult != Internal::EGetValidAssetPathResult::Success)
			{
				return ExitWithAssetPathResult(AssetPathResult);
			}
 
			const TObjectPtr<USkeletalMesh> ExportMesh = NewObject<USkeletalMesh>(Internal::CreateAssetPackage(MeshPackageName), NAME_None, RF_Standalone | RF_Public);
			const TObjectPtr<USkeleton> ExportSkeleton = NewObject<USkeleton>(Internal::CreateAssetPackage(SkeletonPackageName), NAME_None, RF_Standalone | RF_Public);
 
			ExportMesh->SetSkeleton(ExportSkeleton);
 
			FString PhysicsAssetName = ExportParams.GetOutputPhysicsAssetName();
			FString PhysicsPackageName = ExportParams.GetOutputPhysicsAssetPackagePath();
			AssetPathResult = Internal::GetValidAssetPath(ExportParams.ReplacePolicy, UPhysicsAsset::StaticClass(), PhysicsPackageName, PhysicsAssetName);
			if (AssetPathResult != Internal::EGetValidAssetPathResult::Success)
			{
				return ExitWithAssetPathResult(AssetPathResult);
			}
 
			TObjectPtr<UPhysicsAsset> PhysicsAsset = nullptr;
			if (ExportParams.IsCollisionEnable())
			{
				PhysicsAsset = NewObject<UPhysicsAsset>(Internal::CreateAssetPackage(PhysicsPackageName), NAME_None, RF_Standalone | RF_Public);
				ExportMesh->SetPhysicsAsset(PhysicsAsset);
				PhysicsAsset->SetPreviewMesh(ExportMesh, false);
			}
 
			if (!PV::Export::Internal::ExportCollectionToSkeletalMesh(
				ExportMesh,
				Collection,
				ExportParams.NaniteShapePreservation,
				ExportParams.WindSettings,
				PhysicsAsset,
				bCreateNaniteFoliage,
				bShouldExportFoliage,
				ExportParams.CollisionGeneration,
				ModifiedPackages,
				StatusReportCallback
			))
			{
				ExportMesh->SetSkeleton(nullptr);
				if (PhysicsAsset)
				{
					ExportMesh->SetPhysicsAsset(nullptr);
					PhysicsAsset->MarkAsGarbage();
				}
				
				ExportSkeleton->MarkAsGarbage();
				ExportMesh->MarkAsGarbage();
				
				return ExitWithResult(EExportResult::Cancelled);
			}
			
			if (PhysicsAsset)
			{
				Internal::ReplaceAssetInPackage(PhysicsAsset);
				ModifiedPackages.Add(PhysicsAsset->GetOutermost());
 
				OutCreatedAssets.Add(PhysicsPackageName);
				Internal::AttachProceduralVegetationLink(PhysicsAsset, InProceduralVegetation);
			}
			
			Internal::ReplaceAssetInPackage(ExportSkeleton);
			Internal::ReplaceAssetInPackage(ExportMesh);
			ModifiedPackages.Add(ExportMesh->GetOutermost());
			ModifiedPackages.Add(ExportSkeleton->GetOutermost());
			
			OutCreatedAssets.Add(MeshPackageName);
			OutCreatedAssets.Add(SkeletonPackageName);
 
			Internal::AttachProceduralVegetationLink(ExportMesh, InProceduralVegetation);
			Internal::AttachProceduralVegetationLink(ExportSkeleton, InProceduralVegetation);
		}
 
		for (UPackage* Package : ModifiedPackages)
		{
			Package->MarkPackageDirty();
 
			if (Package->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				UObject* Asset = Package->FindAssetInPackage();
				if (Asset)
				{
					IAssetRegistry::Get()->AssetCreated(Asset);
				}
			}
		}
 
		return ExitWithResult(EExportResult::Success);
	}
}
 
#undef LOCTEXT_NAMESPACE 
