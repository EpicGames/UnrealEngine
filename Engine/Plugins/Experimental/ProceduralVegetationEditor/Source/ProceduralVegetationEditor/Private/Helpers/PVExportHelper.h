// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NaniteAssemblyDataBuilder.h"
#include "Facades/PVBoneFacade.h"
#include "Engine/EngineTypes.h"

#include "Exporter/PVExporter.h"

enum class EPVCollisionGeneration : uint8;

class UDynamicMesh;
class UInstancedStaticMeshComponent;
class UMaterialInterface;
class UPVExportEntry;
class UPVWindSettings;
class UPackage;
class UProceduralVegetation;
class USkeletalMesh;
class UStaticMesh;

struct FGeometryScriptCopyMeshToAssetOptions;
struct FManagedArrayCollection;
struct FPVExportParams;

using namespace UE::Geometry;
namespace UE::Geometry { class FDynamicMesh3; }

namespace PV::Export
{
	typedef TFunction<bool(const FText& Stage, float Progress)> FStatusReportCallback;

	enum class EExportResult
	{
		Success,
		Fail,
		Cancelled,
		Skipped
	};

	EExportResult ExportCollectionAsMesh(
		const TObjectPtr<UProceduralVegetation> InProceduralVegetation,
		const FManagedArrayCollection& Collection,
		const TObjectPtr<UPVExportEntry>& ExportEntry,
		TArray<FString>& OutCreatedAssets,
		const FStatusReportCallback& StatusReportCallback
	);
}

namespace PV::Export::Internal
{
	typedef TFunction<void(const FString& MeshName, int32 InstanceID, int32 VertexStart, int32 VertexCount)> FMeshInstanceCombined;
	
	TObjectPtr<UDynamicMesh> CollectionToDynamicMesh(const FManagedArrayCollection& Collection);

	FGeometryScriptCopyMeshToAssetOptions GetCopyMeshToAssetOptions(
		const TArray<TObjectPtr<UMaterialInterface>>& InMaterials,
		const FMeshNaniteSettings& InNaniteSettings,
		const ENaniteShapePreservation InShapePreservation
	);
	
	bool CombineMeshInstancesToDynamicMesh(
		const FManagedArrayCollection& Collection,
		UDynamicMesh* DynamicMesh,
		TArray<TObjectPtr<UMaterialInterface>>& Materials,
		const FMeshInstanceCombined& OnMeshInstanceCombined,
		const TFunction<bool(float)>& OnProgressUpdated
	);

	void BuildNaniteAssemblyData(
		const FManagedArrayCollection& Collection,
		UStaticMesh* StaticMesh,
		TArray<UPackage*>& OutModifiedPackages
	);

	void AddNodeToBuilder(
		FNaniteAssemblyDataBuilder& AssemblyBuilder,
		const FManagedArrayCollection& Collection,
		const TMap<FString, int32>& InMeshNamePartMap,
		const TMap<FString, TArray<TObjectPtr<UMaterialInterface>>>& InMeshMaterialsMap
	);
	
	bool ExportCollectionToStaticMesh(
		TObjectPtr<UStaticMesh> ExportMesh,
		const FManagedArrayCollection& Collection,
		ENaniteShapePreservation InShapePreservation,
		bool bBuildNaniteAssemblies,
		bool bShouldExportFoliage,
		bool bCollision,
		TArray<UPackage*>& OutModifiedPackages,
		const FStatusReportCallback& StatusReportCallback
	);

	bool ExportCollectionToSkeletalMesh(
		TObjectPtr<USkeletalMesh> ExportMesh,
		const FManagedArrayCollection& Collection,
		ENaniteShapePreservation InShapePreservation,
		TObjectPtr<const UPVWindSettings> InWindSettings,
		TObjectPtr<UPhysicsAsset> InPhysicsAsset,
		bool bBuildNaniteAssemblies,
		bool bShouldExportFoliage,
		EPVCollisionGeneration CollisionGeneration,
		TArray<UPackage*>& OutModifiedPackages,
		const FStatusReportCallback& StatusReportCallback
	);

	void AttachProceduralVegetationLink(UObject* InExportedMesh, const TObjectPtr<UProceduralVegetation>& InProceduralVegetation);
	
	void AssignBoneIDsToFoliage(const TArray<Facades::FBoneNode>& Bones, FManagedArrayCollection& Collection);

	void RemoveUnwantedSkinWeights(TObjectPtr<UDynamicMesh> DynamicMesh, const FName ProfileName, const TArray<int32>& VertexBoneIDs,
		int32 FoliageVertexStart, const TArray<Facades::FBoneNode>& BoneNodes);

	void SetupPhysicAsset(const FManagedArrayCollection& Collection,const EPVCollisionGeneration& CollisionGeneration, UPhysicsAsset* PhysicsAsset, TArray<Facades::FBoneNode> InBones);

	bool ConvertToDefaultSkeletalMesh(USkeletalMesh* SkeletalMesh,
		FDynamicMesh3& Mesh,
		const TArray<TObjectPtr<UMaterialInterface>>& Materials
	);
}