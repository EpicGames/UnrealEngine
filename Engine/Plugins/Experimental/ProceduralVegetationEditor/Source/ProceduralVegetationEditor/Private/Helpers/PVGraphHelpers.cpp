// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVGraphHelpers.h"

#include "PCGEdge.h"

#include "Nodes/PVBaseSettings.h"
#include "Dataflow/DataflowCollectionSpreadSheetWidget.h"

#include "DataTypes/PVGrowthData.h"
#include "DataTypes/PVMeshData.h"

namespace PV::Graph
{
	UPVBaseSettings* GetNodeSettings(const UPCGNode* InNode)
	{
		return InNode
			? Cast<UPVBaseSettings>(InNode->GetSettings())
			: nullptr;
	}

	IPVRenderSettings* GetRenderSettings(const UPCGNode* InNode)
	{
		if (InNode && InNode->GetSettings())
		{
			if (InNode->GetSettings()->Implements<UPVRenderSettings>())
			{
				return Cast<IPVRenderSettings>(InNode->GetSettings());
			}
		}

		return nullptr;
	}

	UPCGPin* GetInPinFromNode(UPCGNode* InNode)
	{
		return InNode
			? InNode->GetInputPin(PCGPinConstants::DefaultInputLabel)
			: nullptr;
	}

	UPCGPin* GetOutPinFromNode(UPCGNode* InNode)
	{
		return InNode
			? InNode->GetOutputPin(PCGPinConstants::DefaultOutputLabel)
			: nullptr;
	}

	UPCGPin* GetUpstreamPin(UPCGPin* InPin)
	{
		return (InPin && InPin->Edges.Num())
			? InPin->Edges[0]->InputPin
			: nullptr;
	}

	UPCGPin* GetDownstreamPin(UPCGPin* InPin)
	{
		return (InPin && InPin->Edges.Num())
			? InPin->Edges[0]->OutputPin
			: nullptr;
	}

	TArray<TObjectPtr<UPCGPin>> GetAllPVPinsFromNode(const UPCGNode* InNode)
	{
		static TArray PinTypes = {
			FPCGDataTypeIdentifier{FPVDataTypeInfoGrowth::AsId()},
			FPCGDataTypeIdentifier{FPVDataTypeInfoMesh::AsId()}
		};

		TArray<TObjectPtr<UPCGPin>> AllPins;
		for (TObjectPtr<UPCGPin> OutputPin : InNode->GetOutputPins())
		{
			if (PinTypes.Contains(OutputPin->GetCurrentTypesID()))
			{
				AllPins.Add(OutputPin);
			}
		}
		for (TObjectPtr<UPCGPin> InputPin : InNode->GetInputPins())
		{
			if (PinTypes.Contains(InputPin->GetCurrentTypesID()))
			{
				AllPins.Add(InputPin);
			}
		}

		return AllPins;
	}
}
