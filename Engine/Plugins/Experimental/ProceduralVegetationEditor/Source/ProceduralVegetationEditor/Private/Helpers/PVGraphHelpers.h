// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PCGPin.h"

class IPVRenderSettings;
class SCollectionSpreadSheetWidget;
class UPCGNode;
class UPVBaseSettings;

namespace PV::Graph
{
	UPVBaseSettings* GetNodeSettings(const UPCGNode* InNode);
	IPVRenderSettings* GetRenderSettings(const UPCGNode* InNode);
	
	UPCGPin* GetInPinFromNode(UPCGNode* InNode);
	UPCGPin* GetOutPinFromNode(UPCGNode* InNode);
	UPCGPin* GetUpstreamPin(UPCGPin* InPin);
	UPCGPin* GetDownstreamPin(UPCGPin* InPin);

	TArray<TObjectPtr<UPCGPin>> GetAllPVPinsFromNode(const UPCGNode* InNode);

	template <typename PVDataType>
	requires std::is_base_of_v<FPCGDataTypeInfo, PVDataType>
	bool IsPinOfType(UPCGPin* InPin)
	{
		return InPin && InPin->GetCurrentTypesID() == FPCGDataTypeIdentifier{PVDataType::AsId()};
	}
}
