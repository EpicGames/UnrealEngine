// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVEditor.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneModule.h"
#include "AssetEditorModeManager.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "EngineAnalytics.h"
#include "InteractiveToolManager.h"
#include "PackagesDialog.h"
#include "PCGDefaultExecutionSource.h"
#include "PCGEditorCommands.h"
#include "PCGEditorTabFactories.h"
#include "ProceduralVegetation.h"
#include "ProceduralVegetationEditorModule.h"
#include "PVEditorCommands.h"
#include "PVEditorSchema.h"
#include "SPVEditorViewport.h"
#include "ToolMenus.h"

#include "Dataflow/DataflowCollectionSpreadSheetWidget.h"

#include "DataTypes/PVData.h"
#include "DataTypes/PVMeshData.h"

#include "Exporter/PVExporter.h"

#include "Facades/PVAttributesNames.h"

#include "GeometryCollection/GeometryCollection.h"

#include "Helpers/PVAnalyticsHelper.h"
#include "Helpers/PVExportHelper.h"
#include "Helpers/PVGraphHelpers.h"
#include "Helpers/PVUtilities.h"

#include "Misc/MessageDialog.h"
#include "Misc/UObjectToken.h"

#include "Nodes/PCGEditorGraphNodeBase.h"
#include "Nodes/PVExportSettings.h"

#include "PCGEditor/Private/PCGEditorGraph.h"

#include "Subsystems/PCGEngineSubsystem.h"

#include "AssetEditorMode/PCGAssetEdMode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Tools/Contexts/PVToolContext.h"

#include "Widgets/SPVExportSelectionDialog.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "PVEditor"

const FName FPVEditor::CollectionSpreadSheetTabId(TEXT("PVEditor_CollectionSpreadSheet"));
const FName FPVEditor::PreviewSceneSettingsTabId(TEXT("PVE_PreviewSceneSettings"));

using namespace PV::Graph;

FPVEditor::FPVEditor()
{}

void FPVEditor::Initialize(
	const EToolkitMode::Type InMode,
	const TSharedPtr<IToolkitHost>& InToolkitHost,
	UProceduralVegetation* InProceduralVegetation
)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bForceRefreshAttributeEvenIfClosed = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	check(InProceduralVegetation);

	UE_LOGF(LogProceduralVegetationEditor, Log, "PVEditor initialized for %ls", *InProceduralVegetation->GetFName().ToString());
	ProceduralVegetationBeingEdited = InProceduralVegetation;

	UProceduralVegetationGraph* ProceduralVegetationGraph = Cast<UProceduralVegetationGraph>(InProceduralVegetation->GetGraph());
	check(ProceduralVegetationGraph);

	// Initialize widgets before calling base

	if (PV::Utilities::DebugModeEnabled())
	{
		CollectionSpreadSheetWidget = CreateCollectionSpreadSheetWidget();
	}

	FPCGEditor::Initialize(InMode, InToolkitHost, ProceduralVegetationGraph, InProceduralVegetation);

	ExecutionSource = IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>({ .GraphInterface = ProceduralVegetationGraph });

	IPCGBaseSubsystem* Subsystem = ExecutionSource->GetExecutionState().GetSubsystem();
	check(Subsystem);

	Subsystem->GetOnPCGSourceGenerationDone().AddRaw(this, &FPVEditor::OnGraphExecuted);

	// Set the stack
	FPCGStack Stack;
	Stack.PushFrame(ExecutionSource.Get());
	Stack.PushFrame(ProceduralVegetationGraph);
	SetStackBeingInspected(Stack);

	// Ask for generation
	ExecutionSource->Generate();

	// Select the first available output
	for (TObjectPtr<UEdGraphNode> EdNode : GetMainEditorGraph()->Nodes)
	{
		if (const UPCGEditorGraphNodeBase* PCGEdNode = Cast<UPCGEditorGraphNodeBase>(EdNode))
		{
			if (PCGEdNode->GetPCGNode()->GetSettings()->IsA<UPVExportSettings>())
			{
				GetMainEditorGraph()->SelectNodeSet({PCGEdNode}, true);
				break;
			}
		}
	}

	SessionStartTime = FDateTime::Now();
	if (FEngineAnalytics::IsAvailable())
	{
		PV::Analytics::SendSessionStartedEvent();
	}
}

TSubclassOf<UPCGEditorGraphSchema> FPVEditor::GetSchemaClass() const
{
	return UPVEditorSchema::StaticClass();
}

void FPVEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	FPCGEditor::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(ExecutionSource);
}

FText FPVEditor::GetToolkitName() const
{
	return GetLabelForObject(ProceduralVegetationBeingEdited);
}

FName FPVEditor::GetToolkitFName() const
{
	return TEXT("ProceduralVegetationEditor");
}

FText FPVEditor::GetBaseToolkitName() const
{
	return LOCTEXT("PVEditorToolkitName", "Procedural Vegetation Editor");
}

FText FPVEditor::GetToolkitToolTipText() const
{
	return GetToolTipTextForObject(ProceduralVegetationBeingEdited);
}

FString FPVEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("Procedural Vegetation Editor ");
}

FLinearColor FPVEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::Blue;
}

void FPVEditor::OnClose()
{
	UE_LOGF(LogProceduralVegetationEditor, Log, "PVEditor Closed");

	if (FEngineAnalytics::IsAvailable())
	{
		const double TotalSeconds = (FDateTime::Now() - SessionStartTime).GetTotalSeconds();
		PV::Analytics::SendSessionEndedEvent(TotalSeconds);
	}

	if (IPCGBaseSubsystem* Subsystem = ExecutionSource ? ExecutionSource->GetExecutionState().GetSubsystem() : nullptr)
	{
		Subsystem->GetOnPCGSourceGenerationDone().RemoveAll(this);
	}

	FPCGEditor::OnClose();

	if (ExecutionSource)
	{
		ExecutionSource->SetGraphInterface(nullptr);
	}
}

IPCGBaseSubsystem* FPVEditor::GetSubsystem() const
{
	return UPCGEngineSubsystem::Get();
}


void FPVEditor::BindCommands()
{
	FPCGEditor::BindCommands();

	const FPVEditorCommands& EditorCommands = FPVEditorCommands::Get();

	ToolkitCommands->MapAction(
		EditorCommands.Export,
		FExecuteAction::CreateSP(this, &FPVEditor::OnExport));

	ToolkitCommands->MapAction(
		EditorCommands.LockNodeInspection,
		FExecuteAction::CreateSP(this, &FPVEditor::OnLockNodeSelection));
}

void FPVEditor::OnSelectedNodesChanged(const TSet<UObject*>& InNewSelection)
{
	FPCGEditor::OnSelectedNodesChanged(InNewSelection);

	if (const UPCGNode* OldSelectedNode = GetFirstSelectedNode())
	{
		if (UPVBaseSettings* Settings = Cast<UPVBaseSettings>(OldSelectedNode->GetSettings()))
		{
			for (FStructProperty* Property : TFieldRange<FStructProperty>(Settings->GetClass(), EFieldIterationFlags::None))
			{
				if (Property->Struct == FLoopDebugStepper::StaticStruct())
				{
					FLoopDebugStepper* const LoopDebugStepper = Property->ContainerPtrToValuePtr<FLoopDebugStepper>(Settings);
					if (LoopDebugStepper->bDebug)
					{
						LoopDebugStepper->bDebug = false;
						FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
						Settings->PostEditChangeProperty(PropertyChangedEvent);
					}
				}
			}
		}
	}
	SelectedNodes.Empty();

	for (UObject* Selection : InNewSelection)
	{
		if (UPCGEditorGraphNodeBase* EdNode = Cast<UPCGEditorGraphNodeBase>(Selection))
		{
			SelectedNodes.Add(EdNode);
		}
	}

	const IPVRenderSettings* IRenderSettings = GetRenderSettings(GetNodeBeingInspected());

	const bool bInspectionLocked = IRenderSettings && IRenderSettings->IsInspectionLocked();

	if (!IRenderSettings || !bInspectionLocked)
	{
		ChangeNodeInspection(GetFirstSelectedEdNode());
	}

	// TODO: Hacky way to execute embedded subgraphs
	// change execution source when focused graph is changed.
	if (ExecutionSource && GetFocusedGraph())
	{
		ExecutionSource->SetGraphInterface(GetFocusedGraph());

		FPCGStack Stack;
		Stack.PushFrame(ExecutionSource.Get());
		Stack.PushFrame(GetFocusedGraph());
		SetStackBeingInspected(Stack);
	}
}

void FPVEditor::OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType)
{
	FPCGEditor::OnGraphChanged(InGraph, ChangeType);

	if (!!(ChangeType & EPCGChangeType::Structural) && IsInspectedNodeStale())
	{
		ClearInspection();
	}
}

TAttribute<FGraphAppearanceInfo> FPVEditor::GetAppearanceInfo() const
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("PVEditorCornerText", "Procedural Vegetation");

	return AppearanceInfo;
}

TSharedRef<FTabManager::FLayout> FPVEditor::GetDefaultLayout() const
{
	return FTabManager::NewLayout("Standalone_PVGraphEditor_DefaultLayout_v1.0.6")
		->AddArea // Main PCG Graph Editor Area
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split // Top Section - Graph, Data Viewport, HLSL Source Editor, and Details View
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->SetSizeCoefficient(0.40f)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.1f)
					->Split // Viewport
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.75f)
						->AddTab(PCGEditorTabs::ViewportID[0], ETabState::OpenedTab)
						->SetForegroundTab(FTabId(PCGEditorTabs::ViewportID[0]))
						->SetHideTabWell(true)
					)
					->Split // Embedded graph
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->AddTab(PCGEditorTabs::EmbeddedSubgraphsID, ETabState::OpenedTab)
						->SetForegroundTab(FTabId(PCGEditorTabs::EmbeddedSubgraphsID))
					)
				)
				->Split // Graph and details panel
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(PCGEditorTabs::GraphEditorID, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.4f)
						->AddTab(PCGEditorTabs::PropertyDetailsID[0], ETabState::OpenedTab)
						->AddTab(PCGEditorTabs::UserParamsID, ETabState::OpenedTab)
						->AddTab(PreviewSceneSettingsTabId, ETabState::ClosedTab)
						->SetForegroundTab(FTabId(PCGEditorTabs::PropertyDetailsID[0]))
					)
				)
			)
		);
}

void FPVEditor::RegisterExtraTabFactories(FWorkflowAllowedTabSet& TabSet)
{
	if (CollectionSpreadSheetWidget.IsValid())
	{
		FPCGGenericTabFactoryParams Params(CollectionSpreadSheetTabId, SharedThis<FPCGEditor>(this),
			[this]()
				{
					return CollectionSpreadSheetWidget.ToSharedRef();
				});
		Params.Label = LOCTEXT("CollectionSpreadsheetTab", "Collection Spreadsheet");
		TabSet.RegisterFactory(MakeShared<FPCGGenericTabFactory>(Params));
	}

	FPCGGenericTabFactoryParams PreviewParams(PreviewSceneSettingsTabId,SharedThis<FPCGEditor>(this),
		[this]() -> TSharedRef<SWidget>
		{
			if (!EditorViewport.IsValid())
			{
				return SNullWidget::NullWidget;
			}
			FAdvancedPreviewSceneModule& Module = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
			return Module.CreateAdvancedPreviewSceneSettingsWidget(EditorViewport->GetAdvancedPreviewSceneRef());
		}
	);
	PreviewParams.Label = LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings");
	TabSet.RegisterFactory(MakeShared<FPCGGenericTabFactory>(PreviewParams));
}

bool FPVEditor::IsPanelAvailable(const FName PanelID) const
{
	static TSet<FName> SupportedIDs = {
		PCGEditorTabs::FindID,
		PCGEditorTabs::GraphEditorID,
		PCGEditorTabs::LogID,
		PCGEditorTabs::PropertyDetailsID[0],
		PCGEditorTabs::UserParamsID,
		PCGEditorTabs::ViewportID[0],
		PCGEditorTabs::EmbeddedSubgraphsID
	};
	return SupportedIDs.Contains(PanelID);
}

void FPVEditor::RegisterToolbarInternal(FToolBarBuilder& ToolbarBuilder) const
{
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::Find);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::ForceRegen);
	RegisterToolbarButton(ToolbarBuilder, EPCGToolbarButtons::Graph);
	const FPVEditorCommands& Commands = FPVEditorCommands::Get();

	ToolbarBuilder.AddToolBarButton(Commands.Export, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Export"));
}

bool FPVEditor::CanToggleInspected() const
{
	return false;
}

TSharedRef<SPCGEditorViewport> FPVEditor::CreateViewportWidget(int32 Index)
{
	TSharedPtr<SPVEditorViewport> ViewportWidget = nullptr;
	if (EditorViewport == nullptr)
	{
		// ViewportModeManagers[Index] is guaranteed valid for index 0 (set by CreateEditorModeManager).
		FAssetEditorModeManager* ModeManager = GetViewportModeManager(Index);
		EditorViewport = ViewportWidget = SNew(SPVEditorViewport).ModeTools(ModeManager);

		ModeManager->SetPreviewScene(ViewportWidget->GetAdvancedPreviewScene());
		ModeManager->SetSupportsViewportITF(true);
		ModeManager->ActivateMode(UPCGAssetEditorMode::EM_PCGAssetEditorModeId);
		
		SetupModeTools(ModeManager);

		if (UPCGAssetEditorMode* const PCGAssetEditorMode = GetAssetEditorMode(Index))
		{
			PCGAssetEditorMode->SetPCGEditor(SharedThis<FPCGEditor>(this));
		}
	}
	else
	{
		// Non-primary viewports intentionally get no mode tools in the PV Editor.
		ViewportWidget = SNew(SPVEditorViewport);
	}

	return ViewportWidget.ToSharedRef();
}

void FPVEditor::SetupModeTools(FAssetEditorModeManager* InModeTools)
{
	FPCGEditor::SetupModeTools(InModeTools);
	if (const UModeManagerInteractiveToolsContext* InteractiveToolsContext = InModeTools->GetInteractiveToolsContext())
	{
		const TObjectPtr<UContextObjectStore> ContextObjectStore = InteractiveToolsContext->ContextObjectStore;
		UPVToolContextObject* PVNodeToolContext = NewObject<UPVToolContextObject>(ContextObjectStore);
		ContextObjectStore->AddContextObject(PVNodeToolContext);
	}
}

TSharedRef<SCollectionSpreadSheetWidget> FPVEditor::CreateCollectionSpreadSheetWidget()
{
	return SNew(SCollectionSpreadSheetWidget);
}

FPCGStack FPVEditor::GetStackFromPin(const UPCGPin* InPin) const
{
	FPCGStack Stack;
	if (const FPCGStack* PCGStack = GetStackBeingInspected())
	{
		Stack = CopyTemp(*PCGStack);
		if (InPin)
		{
			Stack.PushFrame(InPin->Node);
			Stack.PushFrame(InPin);
		}
	}

	return Stack;
}

const UPVData* FPVEditor::GetDataFromPin(const UPCGPin* InPin)
{
	if (!InPin)
	{
		return nullptr;
	}

	const FPCGStack Stack = GetStackFromPin(InPin);

	const UPVData* PVData = nullptr;
	ExecutionSource->GetExecutionState().GetInspection().InspectData(
		Stack,
		[&](const FPCGDataCollection& DataCollection)
			{
				const FPCGTaggedData* TaggedData = DataCollection.TaggedData.GetData();
				PVData = TaggedData
					? Cast<UPVData>(TaggedData->Data)
					: nullptr;
			}
	);

	return PVData;
}

void FPVEditor::OnGraphExecuted(IPCGBaseSubsystem* InSubsystem, IPCGGraphExecutionSource* InExecutionSource, EPCGGenerationStatus InGenerationStatus)
{
	UpdateInspectedCollection();
	UpdateStats();
}

void FPVEditor::OnNodeToolStarted(UPCGEditorGraphNodeBase* InteractiveNode)
{
	if (!InteractiveNode)
	{
		return;
	}

	UPCGNode* PCGNode = InteractiveNode->GetPCGNode();
	if (!PCGNode || !PCGNode->GetSettings() || PCGNode->GetSettings()->IsNodeToolActive())
	{
		return;
	}
	
	// Only the first viewport is open in PVE Context. So fetch the index=0 Asset Editor Mode
	if (const UPCGAssetEditorMode* Mode = GetAssetEditorMode(0))
	{
		if (const UEditorInteractiveToolsContext* ToolsContext = Mode->GetInteractiveToolsContext())
		{
			if (UPVToolContextObject* PVToolContext = ToolsContext->ContextObjectStore->FindContext<UPVToolContextObject>())
			{
				PVToolContext->InspectionStack = GetStackFromPin(GetInPinFromNode(PCGNode));
				ToolsContext->ContextObjectStore->AddContextObject(PVToolContext);
			}
			else
			{
				return;
			}
		}
	}
	
	FPCGEditor::OnNodeToolStarted(InteractiveNode);
}

void FPVEditor::ChangeNodeInspection(UPCGEditorGraphNodeBase* InNode)
{
	if (!InNode)
	{
		return;
	}

	if (IPVRenderSettings* Settings = GetRenderSettings(InNode->GetPCGNode()))
	{
		if (EditorViewport.IsValid())
		{
			EditorViewport->OnNodeInspectionChanged(Settings);
		}
	}
	else
	{
		if (EditorViewport.IsValid())
		{
			EditorViewport->ClearNodeInspection();
		}
	}

	SetNodeBeingInspected(InNode);
}

void FPVEditor::ClearInspection()
{
	if (!NodeBeingInspected)
	{
		return;
	}

	const TObjectPtr<UPCGEditorGraphNodeBase> PreviouslyInspectedNode = NodeBeingInspected;
	NodeBeingInspected = nullptr;
	CollectionBeingInspected = nullptr;

	if (PreviouslyInspectedNode && PreviouslyInspectedNode->GetPCGNode() && PreviouslyInspectedNode->GetPCGNode()->GetSettings())
	{
		SetNodeInspected(PreviouslyInspectedNode, /*bValue=*/false);
	}

	if (EditorViewport.IsValid())
	{
		EditorViewport->ClearNodeInspection();
		EditorViewport->ResetScene();
		EditorViewport->SetOverlayText(FText::GetEmpty());
		EditorViewport->PopulateStatsOverlayText({});
	}

	UpdateCollectionSpreadSheet();
}

void FPVEditor::OnLockNodeSelection()
{
	if (IPVRenderSettings* NodeBeingInspectedRenderSettings = GetRenderSettings(GetNodeBeingInspected()))
	{
		NodeBeingInspectedRenderSettings->SetInspectionLocked(!NodeBeingInspectedRenderSettings->IsInspectionLocked());
	}

	if (UPCGEditorGraphNodeBase* const SelectedNode = GetFirstSelectedEdNode())
	{
		if (IPVRenderSettings* SelectedNodeRenderSettings = GetRenderSettings(SelectedNode->GetPCGNode()))
		{
			if (NodeBeingInspected != SelectedNode)
			{
				SelectedNodeRenderSettings->SetInspectionLocked(true);
				ChangeNodeInspection(SelectedNode);
			}
		}
	}

	UpdateStats();
}

TObjectPtr<UPCGNode> FPVEditor::GetFirstSelectedNode()
{
	if (const TObjectPtr<UPCGEditorGraphNodeBase> EdNode = GetFirstSelectedEdNode())
	{
		return EdNode->GetPCGNode();
	}
	return nullptr;
}

TObjectPtr<UPCGEditorGraphNodeBase> FPVEditor::GetFirstSelectedEdNode()
{
	return SelectedNodes.Num()
		? SelectedNodes[0]
		: nullptr;
}

bool FPVEditor::IsNodeSelected(TObjectPtr<UPCGNode> InNode)
{
	return SelectedNodes.ContainsByPredicate(
		[&](const TObjectPtr<UPCGEditorGraphNodeBase>& Node)
			{
				return Node->GetPCGNode() == InNode;
			}
	);
}

TObjectPtr<UPCGNode> FPVEditor::GetNodeBeingInspected()
{
	return NodeBeingInspected
		? NodeBeingInspected->GetPCGNode()
		: nullptr;
}

bool FPVEditor::IsInspectedNodeStale() const
{
	if (!NodeBeingInspected)
	{
		return false;
	}

	const UEdGraph* OwningGraph = NodeBeingInspected->GetGraph();
	const bool bStillInGraph = OwningGraph && OwningGraph->Nodes.Contains(NodeBeingInspected);
	return !bStillInGraph || !IsValid(NodeBeingInspected->GetPCGNode());
}

void FPVEditor::SetNodeBeingInspected(const TObjectPtr<UPCGEditorGraphNodeBase> InNode)
{
	NodeBeingInspected = InNode;
	SetNodeInspected(NodeBeingInspected, true);
	
	UpdateInspectedCollection();
	UpdateStats();
}

void FPVEditor::UpdateInspectedCollection()
{
	if (NodeBeingInspected)
	{
		if (const UPVData* const Data = GetDataFromPin(GetOutPinFromNode(GetNodeBeingInspected())))
		{
			CollectionBeingInspected = Data->GetSharedCollection();
		}
	}

	UpdateCollectionSpreadSheet();
}

void FPVEditor::UpdateCollectionSpreadSheet()
{
	if (!CollectionSpreadSheetWidget)
	{
		return;
	}

	if (!NodeBeingInspected)
	{
		CollectionSpreadSheetWidget->SetData(FString());
		CollectionSpreadSheetWidget->RefreshWidget();

		return;
	}

	const TSharedPtr<SDockTab> CollectionSpreadSheetTab =
		GetTabManager()
		? GetTabManager()->FindExistingLiveTab(CollectionSpreadSheetTabId)
		: nullptr;

	const bool bIsCollectionSpreadSheetVisible =
		CollectionSpreadSheetWidget.IsValid() &&
		CollectionSpreadSheetTab.IsValid() &&
		CollectionSpreadSheetTab->GetVisibility().IsVisible();

	if (bIsCollectionSpreadSheetVisible)
	{
		const TObjectPtr<UPCGNode> InspectedNode = GetNodeBeingInspected();
		const TArray<TObjectPtr<UPCGPin>> AllPins = GetAllPVPinsFromNode(InspectedNode);

		CollectionSpreadSheetWidget->SetData(InspectedNode->GetNodeTitle(EPCGNodeTitleType::FullTitle).ToString());

		TMap<FString, SCollectionInfo>& CollectionInfoMap = CollectionSpreadSheetWidget->GetCollectionTable()->GetCollectionInfoMap();
		CollectionInfoMap.Empty();
		for (TObjectPtr<UPCGPin> Pin : AllPins)
		{
			if (const UPVData* const PinData = GetDataFromPin(Pin))
			{
				CollectionInfoMap.Emplace(
					Pin->Properties.Label.ToString(),
					{PinData->GetSharedCollection()}
				);
			}
		}


		CollectionSpreadSheetWidget->RefreshWidget();
	}
}

void FPVEditor::GatherStats(TArray<FText>& OverlayStats)
{
	if (CollectionBeingInspected == nullptr)
	{
		return;
	}

	if (CollectionBeingInspected->HasGroup(PV::GroupNames::PointGroup))
	{
		const int32 NumPoints = CollectionBeingInspected->NumElements(PV::GroupNames::PointGroup);
		OverlayStats.Add(FText::FromString("Points: " + FString::FromInt(NumPoints)));
	}
	if (CollectionBeingInspected->HasGroup(PV::GroupNames::BranchGroup))
	{
		const int32 NumBranches = CollectionBeingInspected->NumElements(PV::GroupNames::BranchGroup);
		OverlayStats.Add(FText::FromString("Branches: " + FString::FromInt(NumBranches)));
	}
	if (CollectionBeingInspected->HasGroup(FGeometryCollection::VerticesGroup))
	{
		const int32 NumVertices = CollectionBeingInspected->NumElements(FGeometryCollection::VerticesGroup);
		OverlayStats.Add(FText::FromString("Vertices: " + FString::FromInt(NumVertices)));
	}
	if (CollectionBeingInspected->HasGroup(FGeometryCollection::FacesGroup))
	{
		const int32 NumTriangles = CollectionBeingInspected->NumElements(FGeometryCollection::FacesGroup);
		OverlayStats.Add(FText::FromString("Triangles: " + FString::FromInt(NumTriangles)));
	}
	if (CollectionBeingInspected->HasGroup(PV::GroupNames::BonesGroup))
	{
		const int32 NumBones = CollectionBeingInspected->NumElements(PV::GroupNames::BonesGroup);
		OverlayStats.Add(FText::FromString("Bones: " + FString::FromInt(NumBones)));
	}
	if (CollectionBeingInspected->HasGroup(PV::GroupNames::FoliageNamesGroup))
	{
		const int32 NumUniqueFoliage = CollectionBeingInspected->NumElements(PV::GroupNames::FoliageNamesGroup);
		OverlayStats.Add(FText::FromString("Unique Foliage: " + FString::FromInt(NumUniqueFoliage)));
	}
	if (CollectionBeingInspected->HasGroup(PV::GroupNames::FoliageGroup))
	{
		const int32 NumFoliageInstances = CollectionBeingInspected->NumElements(PV::GroupNames::FoliageGroup);
		OverlayStats.Add(FText::FromString("Foliage Instances: " + FString::FromInt(NumFoliageInstances)));
	}
}

void FPVEditor::UpdateStats()
{
	if (!EditorViewport)
	{
		return;
	}

	TArray<FText> OverlayStats;
	GatherStats(OverlayStats);
	EditorViewport->PopulateStatsOverlayText(OverlayStats);

	const TObjectPtr<UPCGNode> InspectedNode = GetNodeBeingInspected();
	if (const IPVRenderSettings* NodeBeingInspectedRenderSettings = GetRenderSettings(InspectedNode))
	{
		EditorViewport->SetOverlayText(
			NodeBeingInspectedRenderSettings->IsInspectionLocked()
			? InspectedNode->GetNodeTitle(EPCGNodeTitleType::FullTitle)
			: FText::GetEmpty()
		);
	}
}

void FPVEditor::OnExport()
{
	UProceduralVegetationGraph* ProceduralVegetationGraph = Cast<UProceduralVegetationGraph>(GetMainGraph());

	TArray<TObjectPtr<UPVExportEntry>> ExportEntries;
	ProceduralVegetationGraph->ForEachNode(
		[&](UPCGNode* Node)-> bool
			{
				if (!Node->GetSettings()->IsA<UPVExportSettings>())
				{
					return true;
				}

				UPCGPin* const InPin = GetInPinFromNode(Node);
				
				if (const UPVData* const NodeData = GetDataFromPin(InPin))
				{
					if (!NodeData->IsA<UPVMeshData>())
					{
						return true;
					}

					const TObjectPtr<UPVExportEntry>& ExportEntry = ExportEntries.
						Emplace_GetRef(NewObject<UPVExportEntry>(ProceduralVegetationGraph));

					const TSharedPtr<FManagedArrayCollection>& Collection = NodeData->GetSharedCollection();
					const bool bHasFoliage = Collection && Collection->HasGroup(PV::GroupNames::FoliageGroup);
					ExportEntry->Initialize(Collection, Node, IsNodeSelected(Node), bHasFoliage);
				}

				return true;
			}
	);

	const TSharedPtr<SPVExportSelectionDialog> ExportSelectionWidget =
		SNew(SPVExportSelectionDialog)
		.Title(LOCTEXT("ExportSelectionLabel", "Modify Export Settings"))
		.ExportEntries(ExportEntries);

	if (ExportSelectionWidget->ShowModal() != EAppReturnType::Ok)
	{
		return;
	}

	UE_LOGF(LogProceduralVegetationEditor, Log, "Export Started");

	FPVExporter Exporter(ProceduralVegetationBeingEdited, ExportEntries);
	Exporter.Export();
}


#undef LOCTEXT_NAMESPACE
