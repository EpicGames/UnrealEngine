// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGEditor.h"
#include "SPVEditorViewport.h"

#include "DataTypes/PVData.h"

class SCollectionSpreadSheetWidget;
class UPCGDefaultExecutionSource;
class UPCGEditorGraphNodeBase;
class UProceduralVegetation;
struct FManagedArrayCollection;

class FPVEditor : public FPCGEditor
{
public:
	FPVEditor();

	void Initialize(
		const EToolkitMode::Type InMode,
		const TSharedPtr<IToolkitHost>& InToolkitHost,
		UProceduralVegetation* InProceduralVegetation
	);

	virtual TSubclassOf<UPCGEditorGraphSchema> GetSchemaClass() const override;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	// ~Begin FAssetEditorToolkit interface
	virtual FText GetToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;

	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	virtual void OnClose() override;
	// ~End FAssetEditorToolkit interface

	// ~Begin FPCGEditor interface
	virtual IPCGBaseSubsystem* GetSubsystem() const override;
	// ~End FPCGEditor interface

protected:
	// ~Begin FPCGEditor interface
	virtual void BindCommands() override;
	virtual void OnSelectedNodesChanged(const TSet<UObject*>& InNewSelection) override;
	virtual void OnGraphChanged(UPCGGraphInterface* InGraph, EPCGChangeType ChangeType) override;

	virtual TAttribute<FGraphAppearanceInfo> GetAppearanceInfo() const override;
	virtual TSharedRef<FTabManager::FLayout> GetDefaultLayout() const override;

	virtual void RegisterExtraTabFactories(FWorkflowAllowedTabSet& TabSet) override;
	virtual bool IsPanelAvailable(const FName PanelID) const override;
	virtual void RegisterToolbarInternal(FToolBarBuilder& ToolbarBuilder) const override;

	virtual bool CanToggleInspected() const override;

	virtual TSharedRef<SPCGEditorViewport> CreateViewportWidget(int32 Index) override;
	
	virtual void SetupModeTools(FAssetEditorModeManager* InModeTools) override;

	virtual void OnNodeToolStarted(UPCGEditorGraphNodeBase* InteractiveNode) override;
	// ~End FPCGEditor interface

	static TSharedRef<SCollectionSpreadSheetWidget> CreateCollectionSpreadSheetWidget();

private:
	FPCGStack GetStackFromPin(const UPCGPin* InPin) const;

	const UPVData* GetDataFromPin(const UPCGPin* InPin);

	void OnGraphExecuted(IPCGBaseSubsystem* InSubsystem, IPCGGraphExecutionSource* InExecutionSource,
	EPCGGenerationStatus InGenerationStatus);

	void ChangeNodeInspection(UPCGEditorGraphNodeBase* InNode);
	void OnLockNodeSelection();

	TObjectPtr<UPCGNode> GetFirstSelectedNode();
	TObjectPtr<UPCGEditorGraphNodeBase> GetFirstSelectedEdNode();
	bool IsNodeSelected(TObjectPtr<UPCGNode> InNode);
	
	TObjectPtr<UPCGNode> GetNodeBeingInspected();
	void SetNodeBeingInspected(TObjectPtr<UPCGEditorGraphNodeBase> InNode);

	/** Returns true if the currently inspected EdNode no longer exists in its owning graph (i.e. was deleted). */
	bool IsInspectedNodeStale() const;

	/**
	 * Fully releases inspection: clears base PCG inspection bookkeeping, tears down the drawn viewport scene,
	 * clears the "Previewing Output of Node" banner and the stats overlay, and resets all inspection state.
	 * Safe to call (no-op) when nothing is inspected.
	 */
	void ClearInspection();

	void UpdateInspectedCollection();

	void UpdateCollectionSpreadSheet();

	void GatherStats(TArray<FText>& OverlayStats);
	void UpdateStats();

	void OnExport();

private:
	static const FName CollectionSpreadSheetTabId;
	static const FName PreviewSceneSettingsTabId;

	TObjectPtr<UPCGDefaultExecutionSource> ExecutionSource;

	TObjectPtr<UProceduralVegetation> ProceduralVegetationBeingEdited = nullptr;

	TSharedPtr<SPVEditorViewport> EditorViewport = nullptr;

	TObjectPtr<UPCGEditorGraphNodeBase> NodeBeingInspected = nullptr;
	TArray<TObjectPtr<UPCGEditorGraphNodeBase>> SelectedNodes;

	TSharedPtr<SCollectionSpreadSheetWidget> CollectionSpreadSheetWidget;
	TSharedPtr<FManagedArrayCollection> CollectionBeingInspected = nullptr;

	FDateTime SessionStartTime;
};
