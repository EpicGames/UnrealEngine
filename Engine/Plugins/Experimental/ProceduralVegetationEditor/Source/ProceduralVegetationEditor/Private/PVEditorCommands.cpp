// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVEditorCommands.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PVEditorCommands"

FPVEditorCommands::FPVEditorCommands()
	: TCommands<FPVEditorCommands>(
		"PVPCGEditor",
		LOCTEXT("ContextDesc", "Procedural Vegetation Editor"),
		NAME_None,
		FAppStyle::GetAppStyleSetName())
{
}

void FPVEditorCommands::RegisterCommands()
{
	UI_COMMAND(Export, "Export", "Export outputs.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::E));
	UI_COMMAND(ShowMannequin, "Mannequin", "Show/Hide Mannequin", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::M));
	UI_COMMAND(ShowScaleVisualization, "Scale Visualization", "Show/Hide Scale Visualization", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::V));
	UI_COMMAND(ShowStats, "Stats Overlay", "Show/Hide Viewport Stats Overlay", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AutoFocusViewport, "Auto focus viewport", "Auto focus viewport on node selection and visualization mode switch", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(LockNodeInspection, "LockNodeInspection", "Lock node inspection.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::L));
}

#undef LOCTEXT_NAMESPACE
