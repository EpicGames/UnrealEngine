// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FPVEditorCommands : public TCommands<FPVEditorCommands>
{
public:
	FPVEditorCommands();

	// ~Begin TCommands<> interface
	virtual void RegisterCommands() override;
	// ~End TCommands<> interface

	//Editor Commands
	TSharedPtr<FUICommandInfo> Export;

	//Viewport Commands
	TSharedPtr<FUICommandInfo> ShowMannequin;
	TSharedPtr<FUICommandInfo> ShowScaleVisualization;
	TSharedPtr<FUICommandInfo> AutoFocusViewport;
	TSharedPtr<FUICommandInfo> ShowStats;
	TSharedPtr<FUICommandInfo> LockNodeInspection;
};