// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVEditorCommon.h"

namespace PV::EditorCommon
{
	const FLinearColor HoverHighlightColor    = FLinearColor(1.0f, 1.0f, 0.0f, 0.5f); // yellow
	const FLinearColor SelectedHighlightColor = FLinearColor(1.0f, 0.5f, 0.0f, 0.5f); // orange
	const FLinearColor InfluenceRadiusColor   = FLinearColor(1.0f, 1.0f, 0.0f, 0.5f); // yellow

	const PV::TDynamicMeshVertexAttributeDefinition<int32> BranchIndexAttribute(TEXT("PVBranchIndex"));
	const PV::TDynamicMeshVertexAttributeDefinition<int32> BranchPointIndexAttribute(TEXT("PVBranchPointIndex"));
}
