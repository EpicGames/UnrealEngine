// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/EngineTypes.h"

#include "Utils/PVDynamicMeshVertexAttribute.h"

namespace PV::EditorCommon
{
	// Tessellation for the tapered cylinder representing each skeleton edge.
	constexpr int32 EdgeCylinderNumSides = 6;

	// Tessellation for the wireframe-sphere highlight drawn over points.
	constexpr int32 PointHighlightNumSides = 16;

	// Tessellation for the influence-radius circles drawn in SelectByEuclideanDistance mode.
	constexpr int32 InfluenceRadiusNumSides = 32;

	// Minimum scale for the point sphere.
	constexpr float PointMinScale = 0.1f;

	// Scale biases applied to the mesh sphere, point highlight, and edge highlight.
	constexpr float HighlightPointScaleBias = 1.25f;
	constexpr float HighlightEdgeScaleBias = 1.1f;

	// Hover/selection highlight visuals. Defined in PVEditorCommon.cpp.
	extern const FLinearColor HoverHighlightColor;
	extern const FLinearColor SelectedHighlightColor;
	extern const FLinearColor InfluenceRadiusColor;

	constexpr float HighlightLineThickness = 0.0f;
	constexpr uint8 HighlightPointDepthPriority = SDPG_Foreground;
	constexpr uint8 HighlightEdgeDepthPriority = SDPG_World;

	// Per-vertex int32 attribute names. Anchored on the dynamic mesh attribute set so the
	// click ray cast can recover (BranchIndex, BranchPointIndex) for any hit triangle.
	extern const PV::TDynamicMeshVertexAttributeDefinition<int32> BranchIndexAttribute;
	extern const PV::TDynamicMeshVertexAttributeDefinition<int32> BranchPointIndexAttribute;
}
