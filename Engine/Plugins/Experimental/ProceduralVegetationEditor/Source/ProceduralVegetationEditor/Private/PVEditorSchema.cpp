// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVEditorSchema.h"
#include "PCGSubgraph.h"
#include "Elements/PCGCreateAttribute.h"
#include "Elements/ControlFlow/PCGMultiSelect.h"
#include "Elements/Metadata/PCGMetadataMathsOpElement.h"
#include "Nodes/PVBaseSettings.h"
#include "Nodes/PVPointScatterSettings.h"
#include "Nodes/PVSubgraphSettings.h"
#include "Schema/PCGEditorGraphSchemaActions.h"
#include "GraphEditorActions.h"

namespace
{
	/** Category overrides for classes that derive from PCG types but where we cannot (or do not) use the
	 *  UPVBaseSettings::GetCategoryOverride() virtual. This covers:
	 *    - PCG-engine classes whose headers we don't own (UPCGCreateAttributeSetSettings, etc.)
	 *    - PV classes that derive directly from a PCG class instead of UPVBaseSettings (e.g. UPVPointScatterSettings)
	 *  Virtual on UPVBaseSettings takes precedence; this map is the fallback. */
	static const TMap<TSubclassOf<UPCGSettings>, FText>& GetPCGNodesCategoryOverrides()
	{
		static const TMap<TSubclassOf<UPCGSettings>, FText> Map = {
			// PV class that does not derive from UPVBaseSettings
			{ UPVPointScatterSettings::StaticClass(),         NSLOCTEXT("PVNodeCategories", "CategorySeedLabel",         "Seed") },
			// PCG-exposed classes (filtered list lives in GetGraphEditorCustomization above)
			{ UPCGCreateAttributeSetSettings::StaticClass(),  NSLOCTEXT("PVNodeCategories", "CategoryParamsLabel",       "Params") },
			{ UPCGMetadataMathsSettings::StaticClass(),       NSLOCTEXT("PVNodeCategories", "CategoryMathLabel",         "Math") },
			{ UPCGMultiSelectSettings::StaticClass(),         NSLOCTEXT("PVNodeCategories", "CategoryControlFlowLabel",  "Control flow") },
		};
		return Map;
	}

	/** Single entry point for resolving the action-menu category override for a settings class.
	 *  Order:
	 *    1. UPVBaseSettings::GetCategoryOverride() virtual (preferred — class declares its own category).
	 *    2. GetPCGNodesCategoryOverrides() map fallback (for classes we don't own a virtual on).
	 *  Returns FText::GetEmpty() if neither path produces an override; the caller then leaves the
	 *  default EPCGSettingsType-derived category in place. */
	static FText ResolveCategoryOverride(TSubclassOf<UPCGSettings> SettingsClass)
	{
		if (!SettingsClass)
		{
			return FText::GetEmpty();
		}

		const UObject* DefaultObject = SettingsClass->GetDefaultObject();

		if (const UPVBaseSettings* PVDefault = Cast<UPVBaseSettings>(DefaultObject))
		{
			const FText VirtualOverride = PVDefault->GetCategoryOverride();

			if (!VirtualOverride.IsEmpty())
			{
				return VirtualOverride;
			}
		}

		if (const FText* MapEntry = GetPCGNodesCategoryOverrides().Find(SettingsClass))
		{
			return *MapEntry;
		}

		return FText::GetEmpty();
	}
}

EPCGElementType UPVEditorSchema::GetElementTypeFiltering() const
{
	return EPCGElementType::Native | EPCGElementType::Other;
}

TSubclassOf<UPCGBaseSubgraphSettings> UPVEditorSchema::GetSubgraphSettingsClass() const
{
	return UPVSubgraphSettings::StaticClass();
}

const FPCGGraphEditorCustomization& UPVEditorSchema::GetGraphEditorCustomization(const UEdGraph* InEdGraph) const
{
	static const FPCGGraphEditorCustomization GraphEditorCustomization
	{
		.bFilterSettings = true,
		.FilteredSettingsTypes =
	{
			UPVBaseSettings::StaticClass(),
			UPCGCreateAttributeSetSettings::StaticClass(),
			UPCGMetadataMathsSettings::StaticClass(),
			UPVSubgraphSettings::StaticClass(),
			UPVPointScatterSettings::StaticClass(),
			UPCGMultiSelectSettings::StaticClass()
		}
	};

	return GraphEditorCustomization;
}

void UPVEditorSchema::GetNativeElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const
{
	const int32 ActionCountBefore = ActionMenuBuilder.GetNumActions();

	Super::GetNativeElementActions(ActionMenuBuilder, ActionFilter);

	const int32 ActionCountAfter = ActionMenuBuilder.GetNumActions();

	for (int32 Index = ActionCountBefore; Index < ActionCountAfter; ++Index)
	{
		TSharedPtr<FEdGraphSchemaAction>& Action = ActionMenuBuilder.GetSchemaAction(Index);

		if (!Action.IsValid() || Action->GetTypeId() != FPCGEditorGraphSchemaAction_NewNativeElement::StaticGetTypeId())
		{
			continue;
		}

		TSharedPtr<FPCGEditorGraphSchemaAction_NewNativeElement> PCGAction = StaticCastSharedPtr<FPCGEditorGraphSchemaAction_NewNativeElement>(Action);

		const FText CategoryOverride = ResolveCategoryOverride(PCGAction->SettingsClass);

		if (!CategoryOverride.IsEmpty())
		{
			PCGAction->CosmeticUpdateCategory(CategoryOverride);
		}
	}
}
