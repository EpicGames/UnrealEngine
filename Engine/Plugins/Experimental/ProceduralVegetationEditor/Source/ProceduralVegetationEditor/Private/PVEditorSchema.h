// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGEditor.h"
#include "PVEditorSchema.generated.h"

UCLASS()
class UPVEditorSchema : public UPCGEditorGraphSchema
{
	GENERATED_BODY()

public:
	// Return the element type filtering for the current editor. By default it is all.
	virtual EPCGElementType GetElementTypeFiltering() const override;

	virtual TSubclassOf<UPCGBaseSubgraphSettings> GetSubgraphSettingsClass() const override;

	// Return the graph customization for this editor. By default it is the one provided by the graph.
	virtual const FPCGGraphEditorCustomization& GetGraphEditorCustomization(const UEdGraph* InEdGraph) const override;

protected:
	virtual void GetNativeElementActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FPCGActionsFilter& ActionFilter) const override;
};
