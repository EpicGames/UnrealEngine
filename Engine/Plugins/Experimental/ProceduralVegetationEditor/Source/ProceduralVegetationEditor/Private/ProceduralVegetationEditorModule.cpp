// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralVegetationEditorModule.h"
#include "ContentBrowserMenuContexts.h"
#include "LevelEditor.h"
#include "PCGModule.h"
#include "ProceduralVegetationLink.h"
#include "PropertyEditorModule.h"
#include "PVEditorCommands.h"
#include "PVEditorSettings.h"
#include "Utils/PVFloatRamp.h"
#include "Selection.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#include "Animation/SkeletalMeshActor.h"
#include "Animation/Skeleton.h"

#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Customizations/PVFloatRampCustomization.h"
#include "Customizations/PVOutputSettingsCustomizations.h"

#include "DataTypes/PVData.h"
#include "DataTypes/PVGrowthData.h"
#include "DataTypes/PVMeshData.h"
#include "Params/PVExportParams.h"
#include "Params/PVImportStaticMeshParams.h"
#include "Nodes/PVTrunkTextureSetupSettings.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"

#include "GameFramework/Actor.h"

#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"

#include "Customizations/PVColliderParamsCustomization.h"
#include "Customizations/PVGrowerSettingsCustomization.h"
#include "Customizations/PVImportStaticMeshParamsCustomization.h"
#include "Customizations/PVLoopDebugStepperCustomization.h"
#include "Customizations/PVTrunkTextureSetupSettingsCustomization.h"

#include "Helpers/PVUtilities.h"

#include "Nodes/PVGrowerSettings.h"
#include "Nodes/GrowerSettings/PVGrowerBaseSettings.h"
#include "Nodes/PVObjectInteractionSettings.h"

#include "Templates/SharedPointer.h"

#include "Visualizations/PVDataVisualization.h"

#define LOCTEXT_NAMESPACE "ProceduralVegetationEditorModule"

DEFINE_LOG_CATEGORY(LogProceduralVegetationEditor);

const FName PVEditor::MessageLogName = TEXT("PVEditor");

void FProceduralVegetationEditorModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	RegisterMenus();
	FPVEditorCommands::Register();
	RegisterLevelEditorMenuExtension();
	RegisterPinColorAndIcons();

	// Visualizations
	FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetMutablePCGDataVisualizationRegistry();
	DataVisRegistry.RegisterPCGDataVisualization(UPVData::StaticClass(), MakeUnique<const FPVDataVisualization>());

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditor.RegisterCustomPropertyTypeLayout(FPVExportParams::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPVOutputSettingsCustomizations::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout(FPVColliderParams::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPVColliderParamsCustomization::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout(FPVFloatRamp::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPVFloatRampCustomization::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout(FLoopDebugStepper::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPVLoopDebugStepperCustomization::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout(FPVImportStaticMeshParams::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPVImportStaticMeshParamsCustomization::MakeInstance));

	PropertyEditor.RegisterCustomClassLayout(UPVTrunkTextureSetupSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPVTrunkTextureSetupSettingsCustomization::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout(UPVGrowerSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPVGrowerSettingsCustomization::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout(UPVGrowerBaseSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPVGrowerSettingsCustomization::MakeInstance));
	
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(PVEditor::MessageLogName, LOCTEXT("PVEditorLogLabel", "Procedural Vegetation Editor"));
}

void FProceduralVegetationEditorModule::ShutdownModule()
{
	UnregisterLevelEditorMenuExtension();
	UnregisterPinColorAndIcons();
	FPVEditorCommands::Unregister();

	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		if (FPropertyEditorModule* PropertyEditor = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyEditor->UnregisterCustomPropertyTypeLayout(FPVExportParams::StaticStruct()->GetFName());
			PropertyEditor->UnregisterCustomPropertyTypeLayout(FPVColliderParams::StaticStruct()->GetFName());
			PropertyEditor->UnregisterCustomPropertyTypeLayout(FPVFloatRamp::StaticStruct()->GetFName());
			PropertyEditor->UnregisterCustomPropertyTypeLayout(FLoopDebugStepper::StaticStruct()->GetFName());
			PropertyEditor->UnregisterCustomPropertyTypeLayout(FPVImportStaticMeshParams::StaticStruct()->GetFName());
			PropertyEditor->UnregisterCustomClassLayout(UPVTrunkTextureSetupSettings::StaticClass()->GetFName());
			PropertyEditor->UnregisterCustomClassLayout(UPVGrowerSettings::StaticClass()->GetFName());
			PropertyEditor->UnregisterCustomClassLayout(UPVGrowerBaseSettings::StaticClass()->GetFName());
		}
	}

	if (FMessageLogModule* MessageLogModule = FModuleManager::LoadModulePtr<FMessageLogModule>("MessageLog"))
	{
		MessageLogModule->UnregisterLogListing(PVEditor::MessageLogName);
	}
}

void FProceduralVegetationEditorModule::RegisterMenus()
{
	// Allows cleanup when module unloads.
	FToolMenuOwnerScoped OwnerScoped(this);
	
	// Extend the content browser context menu for static meshes and skeletal meshes
	auto AddToContextMenuSection = [this](FToolMenuSection& Section)
	{
		Section.AddDynamicEntry("Open in Procedural Vegetation Editor", FNewToolMenuSectionDelegate::CreateLambda(
			[this](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					{
						const TAttribute<FText> Label = LOCTEXT("Mesh_OpenInProceduralVegetationEditorLabel", "Open in Procedural Vegetation Editor");
						const TAttribute<FText> ToolTip = LOCTEXT("Mesh_OpenInProceduralVegetationEditorToolTip", "Open the mesh in the Procedural Vegetation Editor.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit");

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateRaw(this, &FProceduralVegetationEditorModule::ExecuteOpenInPVEditor);
						UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateRaw(this, &FProceduralVegetationEditorModule::CanExecuteOpenInPVEditor);
						UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateRaw(this, &FProceduralVegetationEditorModule::CanExecuteOpenInPVEditor);
						
						InSection.AddMenuEntry("Mesh_OpenInProceduralVegetationEditor", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
	};

	UToolMenu* StaticMeshMenu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UStaticMesh::StaticClass());
	FToolMenuSection& StaticMeshSection = StaticMeshMenu->FindOrAddSection("GetAssetActions");
	AddToContextMenuSection(StaticMeshSection);

	UToolMenu* SkeletalMeshMenu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USkeletalMesh::StaticClass());
	FToolMenuSection& SkeletalMeshSection = SkeletalMeshMenu->FindOrAddSection("GetAssetActions");
	AddToContextMenuSection(SkeletalMeshSection);

	UToolMenu* SkeletonMenu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USkeleton::StaticClass());
	FToolMenuSection& SkeletonSection = SkeletonMenu->FindOrAddSection("GetAssetActions");
	AddToContextMenuSection(SkeletonSection);
}

void FProceduralVegetationEditorModule::ExecuteOpenInPVEditor(const FToolMenuContext& InContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
	{
		const TArray<UObject*> &Objects = Context->LoadSelectedObjects<UObject>();
		
		for (auto Iterator = Objects.CreateConstIterator(); Iterator; ++Iterator)
		{
			if (UProceduralVegetation* ProceduralVegetation = GetProceduralVegetationFromAsset(*Iterator))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ProceduralVegetation);
			}
		}
	}
}

bool FProceduralVegetationEditorModule::CanExecuteOpenInPVEditor(const FToolMenuContext& InContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
	{
		const TArray<UObject*> &Objects = Context->LoadSelectedObjects<UObject>();
		
		for (auto Iterator = Objects.CreateConstIterator(); Iterator; ++Iterator)
		{
			if (GetProceduralVegetationFromAsset(*Iterator))
			{
				return true;
			}
		}
	}

	return false;
}

void FProceduralVegetationEditorModule::RegisterLevelEditorMenuExtension()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked< FLevelEditorModule >( "LevelEditor" );
	TArray< FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors >& CBMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

	CBMenuExtenderDelegates.Add( FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateStatic( &FProceduralVegetationEditorModule::OnExtendLevelEditorActorSelectionMenu ) );
	LevelEditorExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FProceduralVegetationEditorModule::UnregisterLevelEditorMenuExtension()
{
	if ( LevelEditorExtenderDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded( "LevelEditor" ) )
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >( "LevelEditor" );
		TArray< FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors >& CBMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

		CBMenuExtenderDelegates.RemoveAll([ this ]( const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate )
		{
			return Delegate.GetHandle() == LevelEditorExtenderDelegateHandle;
		});
	}
}

void FProceduralVegetationEditorModule::RegisterPinColorAndIcons()
{
	FPCGDataTypeRegistry& TypeRegistry = FPCGModule::GetMutableDataTypeRegistry();

	TypeRegistry.RegisterPinColorFunction(FPVDataTypeInfoGrowth::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPVEditorSettings>()->GrowthDataPinColor; });
	TypeRegistry.RegisterPinColorFunction(FPVDataTypeInfoMesh::AsId(), [](const FPCGDataTypeIdentifier&) { return GetDefault<UPVEditorSettings>()->MeshDataPinColor; });
}

void FProceduralVegetationEditorModule::UnregisterPinColorAndIcons()
{
	FPCGDataTypeRegistry& TypeRegistry = FPCGModule::GetMutableDataTypeRegistry();

	TypeRegistry.UnregisterPinColorFunction(FPVDataTypeInfoGrowth::AsId());
	TypeRegistry.UnregisterPinColorFunction(FPVDataTypeInfoMesh::AsId());
}

TSharedRef<FExtender> FProceduralVegetationEditorModule::OnExtendLevelEditorActorSelectionMenu(const TSharedRef<FUICommandList> InCommandList, TArray<AActor*> InSelectedActors)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	bool bShouldExtendActorActions = false;

	for ( AActor* Actor : InSelectedActors )
	{
		if (UProceduralVegetation* ProceduralVegetation = GetProceduralVegetationFromActor(Actor))
		{
			bShouldExtendActorActions = true;
		}

		if ( bShouldExtendActorActions )
		{
			break;
		}
	}

	if ( bShouldExtendActorActions )
	{
		// Add the ProceduralVegetation actions
		Extender->AddMenuExtension("ActorGeneral", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
			[InSelectedActors](FMenuBuilder& MenuBuilder)
		{
			FUIAction Action_OpenInPVEditor(FExecuteAction::CreateStatic(&FProceduralVegetationEditorModule::ExecuteOpenInPVEditorOnSelectedActors));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("MenuExtensionOpenInProceduralVegetationEditor", "Open in Procedural Vegetation Editor"),
				LOCTEXT("MenuExtensionOpenInProceduralVegetationEditor_Tooltip", "Open mesh in ProceduralVegetation Editor"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
				Action_OpenInPVEditor,
				NAME_None,
				EUserInterfaceActionType::Button);
		}));
	}

	return Extender;
}

void FProceduralVegetationEditorModule::ExecuteOpenInPVEditorOnSelectedActors()
{
	// Create an array of actors to consider
	TArray<UObject*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(AActor::StaticClass(), /*out*/ SelectedActors);
	
	for (UObject* SelectedObject : SelectedActors)
	{
		if (AActor* SelectedActor = Cast<AActor>(SelectedObject))
		{
			if(UProceduralVegetation* ProceduralVegetation = GetProceduralVegetationFromActor(SelectedActor))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ProceduralVegetation);
			}
		}
	}
}

UProceduralVegetation* FProceduralVegetationEditorModule::GetProceduralVegetationFromAsset(UObject* InObject)
{
	if (IInterface_AssetUserData* IAssetUserData = Cast<IInterface_AssetUserData>(InObject))
	{
		const UProceduralVegetationLink* Data = IAssetUserData->GetAssetUserData<UProceduralVegetationLink>();

		if (Data && Data->Source)
		{
			return Data->Source;
		}
	}

	return nullptr;
}

UProceduralVegetation* FProceduralVegetationEditorModule::GetProceduralVegetationFromActor(AActor* InActor)
{
	if (const AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(InActor))
	{
		if (StaticMeshActor && StaticMeshActor->GetStaticMeshComponent())
		{
			if (UStaticMesh* StaticMesh = StaticMeshActor->GetStaticMeshComponent()->GetStaticMesh())
			{
				if (UProceduralVegetation* ProceduralVegetation = GetProceduralVegetationFromAsset(StaticMesh))
				{
					return ProceduralVegetation;
				}	
			}
		}
	}
	else if (const ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(InActor))
	{
		if (SkeletalMeshActor && SkeletalMeshActor->GetSkeletalMeshComponent())
		{
			if (USkeletalMesh* Mesh = SkeletalMeshActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset())
			{
				if (UProceduralVegetation* ProceduralVegetation = GetProceduralVegetationFromAsset(Mesh))
				{
					return ProceduralVegetation;
				}	
			}
		}
	}

	return nullptr;
}

IMPLEMENT_MODULE(FProceduralVegetationEditorModule, ProceduralVegetationEditor)

#undef LOCTEXT_NAMESPACE
