// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralVegetationFactory.h"

#include "ProceduralVegetation.h"
#include "ProceduralVegetationEditorModule.h"

#include "Widgets/SPVCreateDialog.h"

#define LOCTEXT_NAMESPACE "ProceduralVegetationFactory"

UProceduralVegetationFactory::UProceduralVegetationFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SupportedClass = UProceduralVegetation::StaticClass();
	bCreateNew = true;
	bEditAfterNew = false;
}

UObject* UProceduralVegetationFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext,
	FFeedbackContext* Warn)
{
	UProceduralVegetation* NewProceduralVegetation;
	if (SampleProceduralVegetation)
	{
		NewProceduralVegetation = NewObject<UProceduralVegetation>(InParent, InClass, InName, InFlags, SampleProceduralVegetation);

		UProceduralVegetationGraph* SampleGraph = Cast<UProceduralVegetationGraph>(SampleProceduralVegetation->GetGraph());
		NewProceduralVegetation->CreateGraph(SampleGraph);
		UE_LOGF(LogProceduralVegetationEditor, Log, "New Procedural Vegetation Asset created %ls from Preset %ls", *NewProceduralVegetation->GetFName().ToString(), *SampleProceduralVegetation.GetFName().ToString());
	}
	else
	{
		NewProceduralVegetation = NewObject<UProceduralVegetation>(InParent, InClass, InName, InFlags, InContext);
		NewProceduralVegetation->CreateGraph();
		UE_LOGF(LogProceduralVegetationEditor, Log, "New Procedural Vegetation Asset created %ls", *NewProceduralVegetation->GetFName().ToString());
	}
		
	return NewProceduralVegetation;
}

bool UProceduralVegetationFactory::ConfigureProperties()
{
	SampleProceduralVegetation = nullptr;
	
	const bool bPressedOk = SPVCreateDialog::OpenCreateModal(
		LOCTEXT("ProceduralVegetationFactoryConfigureProperties", "Create Procedural Vegetation"),
		SampleProceduralVegetation
	);
	
	if (SampleProceduralVegetation)
	{
		SampleName = SampleProceduralVegetation->GetName();
	}
	
	return bPressedOk;
}

FString UProceduralVegetationFactory::GetDefaultNewAssetName() const
{
	if (SampleName.IsSet())
	{
		return SampleName.GetValue();
	}
	return Super::GetDefaultNewAssetName();
}

#undef LOCTEXT_NAMESPACE
