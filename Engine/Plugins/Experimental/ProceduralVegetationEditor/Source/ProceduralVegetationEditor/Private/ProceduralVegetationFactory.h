// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "ProceduralVegetationFactory.generated.h"

class UProceduralVegetation;

UCLASS(hidecategories=Object)
class UProceduralVegetationFactory : public UFactory
{
	GENERATED_BODY()

public:
	UProceduralVegetationFactory(const FObjectInitializer& ObjectInitializer);
	
	//~UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override { return true; }
	virtual bool ConfigureProperties() override;
	virtual FString GetDefaultNewAssetName() const override;

private:
	UPROPERTY()
	TObjectPtr<UProceduralVegetation> SampleProceduralVegetation = nullptr;

	UPROPERTY()
	TOptional<FString> SampleName;
};