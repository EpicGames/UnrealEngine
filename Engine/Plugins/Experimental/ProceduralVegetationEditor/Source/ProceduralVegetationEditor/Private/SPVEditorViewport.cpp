// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPVEditorViewport.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "AssetEditorViewportLayout.h"
#include "EditorViewportCommands.h"
#include "PCGEditorSettings.h"
#include "PVEditorCommands.h"
#include "PVEditorSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#include "Brushes/SlateColorBrush.h"

#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/TextRenderComponent.h"

#include "Customizations/RichTextColorDecorator.h"

#include "DataTypes/PVData.h"

#include "Editor/EditorPerProjectUserSettings.h"

#include "Engine/StaticMesh.h"

#include "Nodes/PVBaseSettings.h"

#include "Styling/SlateIconFinder.h"

#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#include "Visualizations/PVScaleVisualizationComponent.h"

#include "Widgets/SPVGradientBox.h"
#include "ISinglePropertyView.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "PVEditorViewportToolbarSections"

static FTableViewStyle& GetTransparentViewStyle()
{
	static FTableViewStyle TransparentViewStyle = []
		{
			const FSlateColorBrush Brush(FLinearColor::Transparent);

			FTableViewStyle ViewStyle = FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView");
			ViewStyle.SetBackgroundBrush(Brush);

			return ViewStyle;
		}();
	return TransparentViewStyle;
}

static FTableRowStyle& GetTransparentRowStyle()
{
	static FTableRowStyle TransparentRowStyle = []
		{
			FTableRowStyle Style = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");

			const FSlateColorBrush Brush(FLinearColor::Transparent);

			Style.SetEvenRowBackgroundBrush(Brush)
			     .SetEvenRowBackgroundHoveredBrush(Brush)
			     .SetOddRowBackgroundBrush(Brush)
			     .SetOddRowBackgroundHoveredBrush(Brush)
			     .SetSelectorFocusedBrush(Brush)
			     .SetActiveBrush(Brush)
			     .SetActiveHoveredBrush(Brush)
			     .SetInactiveBrush(Brush)
			     .SetInactiveHoveredBrush(Brush)
			     .SetDropIndicator_Above(Brush)
			     .SetDropIndicator_Onto(Brush)
			     .SetDropIndicator_Below(Brush);

			return Style;
		}();
	return TransparentRowStyle;
}

SPVEditorViewport::SPVEditorViewport()
{
	FoliageComponents = MakeShared<TArray<TObjectPtr<UMeshComponent>>>();
	FoliageVisibility = MakeShared<TMap<FString, bool>>();

	// Cache the offset widget and the two context objects once. They live for the lifetime of this viewport
	MannequinOffsetWidget = CreateMannequinOffsetWidget();

	MannequinWidgetContext.Reset(NewObject<UPVMannequinWidgetContext>());
	MannequinWidgetContext->MannequinOffsetWidget = MannequinOffsetWidget;

	FoliageComponentsContext.Reset(NewObject<UPVFoliageComponentsContext>());
	FoliageComponentsContext->FoliageComponents = FoliageComponents;
	FoliageComponentsContext->FoliageVisibility = FoliageVisibility;

	PreviewNodeBackgroundBrush = MakeShared<FSlateColorBrush>(FColor{70, 100, 200});

	if (AdvancedPreviewScene)
	{
		if (!GetPreviewProfileController()->SetActiveProfile(UDefaultEditorProfiles::GreyAmbientProfileName.ToString()))
		{
			AdvancedPreviewScene->SetProfileIndex(GetDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex);
		}
		if (!AdvancedPreviewScene->IsPostProcessingEnabled())
		{
			AdvancedPreviewScene->HandleTogglePostProcessing();
		}
	}
}

void SPVEditorViewport::Construct(const FArguments& InArgs)
{
	SPCGEditorViewport::Construct(SPCGEditorViewport::FArguments().ModeTools(InArgs._ModeTools));

	UPCGEditorSettings* PCGEditorSettings = GetMutableDefault<UPCGEditorSettings>();

	if (PCGEditorSettings)
	{
		PCGEditorSettings->bAutoFocusViewport = IsAutoFocusViewportChecked();
	}
}

TSharedPtr<SWidget> SPVEditorViewport::BuildViewportToolbar()
{
	check(AdvancedPreviewScene.IsValid());

	const FName ToolbarName = "PVE.ViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ToolbarName, /*Parent=*/NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Menu->StyleName = "ViewportToolbar";
		FToolMenuSection& LeftSection = Menu->AddSection("Left");
		LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());

		FToolMenuSection& RightSection = Menu->AddSection("Right");
		RightSection.Alignment = EToolMenuSectionAlign::Last;
		RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
		RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
		RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
		RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
		UE::AdvancedPreviewScene::Menus::ExtendAdvancedPreviewSceneSettings(
			"PVE.ViewportToolbar.AssetViewerProfile",
			UE::AdvancedPreviewScene::Menus::FSettingsOptions()
		);

		RightSection.AddEntry(CreateVisualizationModeToolbarMenu());
		RightSection.AddEntry(CreateSettingsToolbarMenu());
	}

	FToolMenuContext Context;
	{
		Context.AppendCommandList(AdvancedPreviewScene->GetCommandList());
		Context.AppendCommandList(GetCommandList());
		Context.AddExtender(GetExtenders());
		Context.AddObject(UE::UnrealEd::CreateViewportToolbarDefaultContext(GetViewportWidget()));
		Context.AddObject(CreateMannequinWidgetContext());
		Context.AddObject(CreateFoliageComponentsContext());
	}

	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

void SPVEditorViewport::BindCommands()
{
	SPCGEditorViewport::BindCommands();

	FPVEditorCommands& Commands = FPVEditorCommands::Get();

	CommandList->MapAction(
		Commands.ShowMannequin,
		FExecuteAction::CreateSP(this, &SPVEditorViewport::ToggleShowMannequin),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPVEditorViewport::IsShowMannequinChecked));

	CommandList->MapAction(
		Commands.ShowScaleVisualization,
		FExecuteAction::CreateSP(this, &SPVEditorViewport::ToggleShowScaleVis),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPVEditorViewport::IsShowScaleVisChecked));

	CommandList->MapAction(
		Commands.ShowStats,
		FExecuteAction::CreateSP(this, &SPVEditorViewport::ToggleShowStats),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPVEditorViewport::IsShowStatsChecked));

	CommandList->MapAction(
		Commands.AutoFocusViewport,
		FExecuteAction::CreateSP(this, &SPVEditorViewport::ToggleAutofocusViewport),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPVEditorViewport::IsAutoFocusViewportChecked));
}

TSharedRef<FEditorViewportClient> SPVEditorViewport::MakeEditorViewportClient()
{
	Client = SPCGEditorViewport::MakeEditorViewportClient();
	Client->ExposureSettings.bFixed = true;
	Client->ExposureSettings.FixedEV100 = -0.2f;
	
	AdvancedPreviewScene->SetFloorVisibility(true);
	AdvancedPreviewScene->SetFloorOffset(1.0f);
		
	if (AdvancedPreviewScene->IsEnvironmentEnabled())
	{
		AdvancedPreviewScene->HandleToggleEnvironment();
	}
	
	return Client.ToSharedRef();
}

void SPVEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SPCGEditorViewport::PopulateViewportOverlays(Overlay);

	Overlay
		->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)
		[
			SNew(SBorder)
			.BorderImage(PreviewNodeBackgroundBrush.Get())
			.IsEnabled_Lambda([this]
				{
					return bIsPreviewingLockedNode;
				})
			.Visibility_Lambda([this]
				{
					return bIsPreviewingLockedNode
						? EVisibility::Visible
						: EVisibility::Collapsed;
				})
			[
				SAssignNew(OverlayText, SRichTextBlock)
				.Justification(ETextJustify::Center)
			]
		];

	Overlay
		->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(6.0f, 25.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
			.Visibility_Lambda([this]
				{
					return StatsOverlayText && !StatsOverlayText->GetText().IsEmpty() && GetDefault<UPVEditorSettings>()->bShowStats
						? EVisibility::Visible
						: EVisibility::Collapsed;
				})
			.Padding(4.f)
			[
				SAssignNew(StatsOverlayText, SRichTextBlock)
			]
		];

	Overlay
		->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		.Padding(6.0f, 25.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
			.Visibility_Lambda([this]
				{
					return !LegendData.IsEmpty()
						? EVisibility::Visible
						: EVisibility::Collapsed;
				})
			.Padding(4.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 4)
				[
					SNew(SRichTextBlock)
					.Text(FText::FromString(TEXT("<TextBlock.ShadowedText>Legend</>")))
					.Justification(ETextJustify::Center)
					.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("BoldFont"))
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(LegendListView, SListView<TSharedPtr<FText>>)
					.ListItemsSource(&LegendData)
					.ListViewStyle(&GetTransparentViewStyle())
					.OnGenerateRow_Lambda(
						[](const TSharedPtr<FText>& Item, const TSharedRef<STableViewBase>& OwningTable) -> TSharedRef<ITableRow>
							{
								return SNew(STableRow<TSharedPtr<FText>>, OwningTable)
									.Style(&GetTransparentRowStyle())
									.ShowSelection(false)
									.Padding(FMargin(0))
									[
										SNew(SRichTextBlock)
										.Text(*Item)
										.Decorators({FColorBlockDecorator::CreateDecorator()})
									];
							}
					)
				]
			]
		];

	Overlay
		->AddSlot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(6.0f, 25.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
			.Visibility_Lambda([this]
				{
					return SkeletonVisualizerComponent && SkeletonVisualizerComponent->GetVisualizationMode() != ESkeletonVisualizationModes::None
						? EVisibility::Visible
						: EVisibility::Collapsed;
				})
			.Padding(6.0f, 4.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text_Lambda([this]()
						{
							if (SkeletonVisualizerComponent)
							{
								return StaticEnum<ESkeletonVisualizationModes>()->GetDisplayNameTextByIndex(
									static_cast<int32>(SkeletonVisualizerComponent->GetVisualizationMode())
								);
							}
							return FText::GetEmpty();
						})
				]
				+ SVerticalBox::Slot()
				[
					SNew(SPVGradientBox)
					.BoxSize(160.0f)
				]
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Left)
						.Text(FText::FromString(TEXT("Min")))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Right)

						.Text(FText::FromString(TEXT("Max")))
					]
				]
			]
		];
}

void SPVEditorViewport::InitVisualizationScene()
{
	const FString MannequinPath = TEXT("/ProceduralVegetationEditor/Mannequin/Viewport_Mannequin_T_Pose.Viewport_Mannequin_T_Pose");
	if (UStaticMesh* MannequinMesh = LoadObject<UStaticMesh>(nullptr, MannequinPath))
	{
		MannequinComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		MannequinComponent->SetStaticMesh(MannequinMesh);
		const FBoxSphereBounds MannequinBounds = MannequinComponent->CalcLocalBounds();

		ManagedResources.Add(MannequinComponent);
		AdvancedPreviewScene->AddComponent(MannequinComponent, FTransform::Identity);
		SetMannequinOffset((-FocusBounds.BoxExtent.X - MannequinBounds.BoxExtent.X) * 1.5f, true);
	}

	ScaleVisualizationComponent = NewObject<UPVScaleVisualizationComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	ScaleVisualizationComponent->SetScaleBounds(FocusBounds);

	ManagedResources.Add(ScaleVisualizationComponent);
	AdvancedPreviewScene->AddComponent(ScaleVisualizationComponent, FTransform::Identity);
	for (const TObjectPtr<UTextRenderComponent>& TextRenderComponent : ScaleVisualizationComponent->GetTextRenderComponents())
	{
		ManagedResources.Add(TextRenderComponent);
		AdvancedPreviewScene->AddComponent(TextRenderComponent, FTransform::Identity);
	}
}

void SPVEditorViewport::ResetVisualizationScene()
{
	if (MannequinComponent)
	{
		MannequinComponent->MarkAsGarbage();
		MannequinComponent = nullptr;
	}

	if (ScaleVisualizationComponent)
	{
		ScaleVisualizationComponent->MarkAsGarbage();
		ScaleVisualizationComponent = nullptr;
	}

	if (SkeletonVisualizerComponent)
	{
		SkeletonVisualizerComponent = nullptr;
	}
}

TSharedRef<SWidget> SPVEditorViewport::CreateMannequinOffsetWidget() const
{
	return SNew(SNumericEntryBox<float>)
		.AllowSpin(true)
		.AllowWheel(true)
		.Delta(5.0f)
		.MinValue(-2000.0f)
		.MaxValue(2000.0f)
		.MinSliderValue(-2000.0f)
		.MaxSliderValue(2000.0f)
		.Value_Lambda([this] { return GetMannequinOffset(); })
		.OnValueChanged_Lambda([this](const float NewValue)
			{
				SetMannequinOffset(NewValue, false);
			})
		.OnValueCommitted_Lambda([this](const float NewValue, ETextCommit::Type)
			{
				SetMannequinOffset(NewValue, true);
			});
}

void SPVEditorViewport::ToggleShowMannequin()
{
	UPVEditorSettings* EditorSettings = GetMutableDefault<UPVEditorSettings>();
	EditorSettings->bShowMannequin = !EditorSettings->bShowMannequin;
	EditorSettings->SaveConfig();

	SetMannequinState(EditorSettings->bShowMannequin);
}

bool SPVEditorViewport::IsShowMannequinChecked() const
{
	return GetDefault<UPVEditorSettings>()->bShowMannequin;
}

void SPVEditorViewport::SetMannequinState(bool InEnable)
{
	if (MannequinComponent)
	{
		MannequinComponent->SetVisibility(InEnable, true);
		Client->Invalidate();
	}
}

void SPVEditorViewport::ToggleShowScaleVis()
{
	UPVEditorSettings* EditorSettings = GetMutableDefault<UPVEditorSettings>();
	EditorSettings->bShowScaleVisualization = !EditorSettings->bShowScaleVisualization;
	EditorSettings->SaveConfig();

	SetScaleVisState(EditorSettings->bShowScaleVisualization);
}

bool SPVEditorViewport::IsShowScaleVisChecked() const
{
	return GetMutableDefault<UPVEditorSettings>()->bShowScaleVisualization;
}

void SPVEditorViewport::SetScaleVisState(bool InEnable)
{
	if (ScaleVisualizationComponent)
	{
		ScaleVisualizationComponent->SetVisibility(InEnable, true);
		Client->Invalidate();
	}
}

void SPVEditorViewport::ToggleShowStats()
{
	UPVEditorSettings* EditorSettings = GetMutableDefault<UPVEditorSettings>();
	EditorSettings->bShowStats = !EditorSettings->bShowStats;
	EditorSettings->SaveConfig();
}

bool SPVEditorViewport::IsShowStatsChecked() const
{
	return GetDefault<UPVEditorSettings>()->bShowStats;
}

void SPVEditorViewport::ToggleAutofocusViewport()
{
	UPVEditorSettings* EditorSettings = GetMutableDefault<UPVEditorSettings>();
	EditorSettings->bAutoFocusViewport = !EditorSettings->bAutoFocusViewport;
	EditorSettings->SaveConfig();

	UPCGEditorSettings* PCGEditorSettings = GetMutableDefault<UPCGEditorSettings>();
	PCGEditorSettings->bAutoFocusViewport = IsAutoFocusViewportChecked();
}

bool SPVEditorViewport::IsAutoFocusViewportChecked() const
{
	return GetDefault<UPVEditorSettings>()->bAutoFocusViewport;
}

float SPVEditorViewport::GetMannequinOffset() const
{
	return GetDefault<UPVEditorSettings>()->MannequinOffset;
}

void SPVEditorViewport::SetMannequinOffset(const float NewValue, const bool bSaveConfig) const
{
	UPVEditorSettings* const EditorSettings = GetMutableDefault<UPVEditorSettings>();
	EditorSettings->MannequinOffset = NewValue;
	if (bSaveConfig)
	{
		EditorSettings->SaveConfig();
	}

	if (MannequinComponent)
	{
		MannequinComponent->SetRelativeLocation(FVector(EditorSettings->MannequinOffset, 0.0, 0.0));
		Client->Invalidate();
	}
}

UPVMannequinWidgetContext* SPVEditorViewport::CreateMannequinWidgetContext()
{
	return MannequinWidgetContext.Get();
}

UPVFoliageComponentsContext* SPVEditorViewport::CreateFoliageComponentsContext()
{
	return FoliageComponentsContext.Get();
}

void SPVEditorViewport::OnSetupScene()
{
	SPCGEditorViewport::OnSetupScene();

	InitVisualizationScene();

	SetMannequinState(IsShowMannequinChecked());
	SetScaleVisState(IsShowScaleVisChecked());

	if (bFocusOnNextUpdate)
	{
		OnFocusViewportToSelection();

		bFocusOnNextUpdate = false;
	}
}

void SPVEditorViewport::OnResetScene()
{
	SPCGEditorViewport::OnResetScene();

	ResetVisualizationScene();

	FoliageComponents->Empty();
	LegendData.Empty();

	if (LegendListView)
	{
		LegendListView->RequestListRefresh();
	}
}

FToolMenuEntry SPVEditorViewport::CreateSettingsToolbarMenu()
{
	return FToolMenuEntry::InitDynamicEntry(TEXT("PVE Settings"), FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
		{
			Section.AddSubMenu(
				/*Name=*/TEXT("PVESettingsSubmenu"),
				FText::GetEmpty(),
				FText::GetEmpty(),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
					{
						const UPVMannequinWidgetContext* MannequinContext = Submenu->FindContext<UPVMannequinWidgetContext>();
						const UPVFoliageComponentsContext* FoliageContext = Submenu->FindContext<UPVFoliageComponentsContext>();

						// Add preference toggles to this section.
						FToolMenuSection& OverlaySection = Submenu->FindOrAddSection(TEXT("PVE_OverlaySettings"),
							LOCTEXT("PVE_OverlaySettingsSectionLabel", "Overlay Settings"));
						OverlaySection.AddMenuEntry(FPVEditorCommands::Get().ShowStats);

						if (MannequinContext && MannequinContext->MannequinOffsetWidget.IsValid())
						{
							FToolMenuSection& MannequinSection = Submenu->FindOrAddSection(TEXT("PVE_MannequinSettings"),
								LOCTEXT("PVE_MannequinSettingsSectionLabel", "Mannequin Settings"));
							MannequinSection.AddMenuEntry(FPVEditorCommands::Get().ShowMannequin);
							MannequinSection.AddEntry(
								FToolMenuEntry::InitWidget(
									"PVE_MannequinOffset",
									MannequinContext->MannequinOffsetWidget.ToSharedRef(),
									LOCTEXT("PVE_MannequinOffsetLabel", "Mannequin Offset"),
									false,
									true,
									false,
									LOCTEXT("PVE_MannequinOffsetTooltip", "Manually Offset the Mannequin")
								)
							);
						}
						FToolMenuSection& ScaleVisSection = Submenu->FindOrAddSection(TEXT("PVE_ScaleVisSettings"),
							LOCTEXT("PVE_ScaleVisSettingsSectionLabel", "Scale Vis. Settings"));
						ScaleVisSection.AddMenuEntry(FPVEditorCommands::Get().ShowScaleVisualization);

						if (FoliageContext && FoliageContext->FoliageComponents && FoliageContext->FoliageComponents->Num())
						{
							FToolMenuSection& FoliageSection = Submenu->FindOrAddSection(TEXT("PVE_FoliageSettings"),
								LOCTEXT("PVE_FoliageSettingsSectionLabel", "Foliage Settings"));
							FoliageSection.AddSubMenu(
								"PVE_ToggleFoliage",
								LOCTEXT("PVE_ToggleFoliageSettingsLabel", "Toggle Foliage"),
								LOCTEXT("PVE_ToggleFoliageSettingsTooltip", "Toggle Individual Foliage"),
								FNewToolMenuChoice(
									FNewToolMenuDelegate::CreateLambda([](UToolMenu* FoliageSubMenu)
										{
											const UPVFoliageComponentsContext* Context = FoliageSubMenu->FindContext<UPVFoliageComponentsContext>();

											if (Context->FoliageComponents)
											{
												for (TObjectPtr<UMeshComponent> InstancedStaticMeshComponent : *Context->FoliageComponents)
												{
													const FString ObjectName = InstancedStaticMeshComponent->GetName();
													FoliageSubMenu->AddMenuEntry(
														"Toggle Foliage",
														FToolMenuEntry::InitMenuEntry(
															*ObjectName,
															FText::FromString(ObjectName),
															FText::Format(FText::FromString("Toggle visibility of: {0}"),
																FText::FromString(ObjectName)),
															FSlateIcon(),
															FUIAction(
																FExecuteAction::CreateLambda([=]()
																	{
																		InstancedStaticMeshComponent->ToggleVisibility(true);
																		Context->FoliageVisibility->FindOrAdd(ObjectName, true) =
																			InstancedStaticMeshComponent->GetVisibleFlag();
																	}
																),
																FCanExecuteAction(),
																FIsActionChecked::CreateLambda([InstancedStaticMeshComponent]
																	{
																		return InstancedStaticMeshComponent->IsVisible();
																	})
															),
															EUserInterfaceActionType::ToggleButton
														)
													);
												}
											}
										}
									)
								),
								false,
								FSlateIconFinder::FindIcon("ClassIcon.FoliageType_Actor")
							);
						}

						FToolMenuSection& AutoFocusViewportSection = Submenu->FindOrAddSection(TEXT("PVE_InteractionSettings"),
							LOCTEXT("PVE_InteractionSettingsSectionLabel", "Interaction Settings"));
						AutoFocusViewportSection.AddMenuEntry(FPVEditorCommands::Get().AutoFocusViewport);
					}),
				/*bInOpenSubMenuOnClick=*/false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings"));
		}));
}

TArray<EPVRenderType> VisualizationModeToRenderModes(EPVVisualizationMode InMode, TArray<EPVRenderType> DefautlSettings = TArray<EPVRenderType>{})
{
	switch (InMode)
	{
	case EPVVisualizationMode::Default:
		return DefautlSettings;

	case EPVVisualizationMode::FoliageMesh:
		return {EPVRenderType::Mesh, EPVRenderType::Foliage};

	case EPVVisualizationMode::FoliageAttachmentMesh:
		return {EPVRenderType::Mesh, EPVRenderType::FoliageAttachments};

	case EPVVisualizationMode::FoliageGrid:
		return {EPVRenderType::FoliageGrid};

	case EPVVisualizationMode::Bones:
		return {EPVRenderType::Bones};

	case EPVVisualizationMode::Mesh:
		return {EPVRenderType::Mesh};

	case EPVVisualizationMode::PointData:
		return {EPVRenderType::PointData};

	case EPVVisualizationMode::PointDataLeafProxy:
		return {EPVRenderType::PointData, EPVRenderType::Leaf};

	case EPVVisualizationMode::BonesMesh:
		return {EPVRenderType::Bones, EPVRenderType::Mesh};

	case EPVVisualizationMode::PointDataMesh:
		return {EPVRenderType::PointData, EPVRenderType::Mesh};

	case EPVVisualizationMode::PointDataFoliageAttachment:
		return {EPVRenderType::PointData, EPVRenderType::FoliageAttachments};

	case EPVVisualizationMode::GraftsGrid:
		return {EPVRenderType::GrafterGrid};

	case EPVVisualizationMode::BoundingBoxOnly:
		return {EPVRenderType::BoundingBoxOnly};

	default:
		return TArray<EPVRenderType>{};
	}
}

TSharedPtr<SPVEditorViewport> GetViewportFromViewportToolbarContext(const UUnrealEdViewportToolbarContext* Context)
{
	if (const TSharedPtr<SEditorViewport> Viewport = Context ? Context->Viewport.Pin() : nullptr)
	{
		return StaticCastSharedPtr<SPVEditorViewport>(Viewport);
	}
	return nullptr;
}

FToolMenuEntry SPVEditorViewport::CreateVisualizationModeToolbarMenu()
{
	return FToolMenuEntry::InitDynamicEntry(
		"VisualizationMode",
		FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InnerSection) -> void
			{
				TWeakObjectPtr ToolbarContext(InnerSection.FindContext<UUnrealEdViewportToolbarContext>());

				TAttribute<FText> LabelAttribute = UE::UnrealEd::GetViewModesSubmenuLabel(nullptr);

				LabelAttribute = TAttribute<FText>::CreateLambda([ToolbarContext]
					{
						if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
						{
							if (
								Viewport->SkeletonVisualizerComponent &&
								Viewport->SkeletonVisualizerComponent->GetVisualizationMode() != ESkeletonVisualizationModes::None
							)
							{
								const UEnum* EnumPtr = StaticEnum<ESkeletonVisualizationModes>();
								return EnumPtr->GetDisplayNameTextByIndex(
									static_cast<int32>(Viewport->SkeletonVisualizerComponent->GetVisualizationMode()));
							}
							const UEnum* EnumPtr = StaticEnum<EPVVisualizationMode>();
							return EnumPtr->GetDisplayNameTextByIndex(static_cast<int>(Viewport->CurrentVisualizationMode));
						}
						return FText::GetEmpty();
					}
				);

				FToolMenuEntry& Entry = InnerSection.AddSubMenu(
					"VisualizationModes",
					LabelAttribute,
					FText::GetEmpty(),
					FNewToolMenuDelegate::CreateLambda([ToolbarContext](UToolMenu* Submenu) -> void
						{
							FToolMenuSection& RenderModeSelectionSection = Submenu->FindOrAddSection(
								"VisualizationModeSection",
								LOCTEXT("FoliageVisualizationModeSelectionSectionLabel", "Visualization modes")
							);

							if (const UEnum* EnumPtr = StaticEnum<EPVVisualizationMode>())
							{
								for (int32 i = 0; i < EnumPtr->NumEnums() - 1; i++) // last entry is _MAX or hidden
								{
									FText ModeName = EnumPtr->GetDisplayNameTextByIndex(i);
									const EPVVisualizationMode Mode = static_cast<EPVVisualizationMode>(EnumPtr->GetValueByIndex(i));
									if (Mode == EPVVisualizationMode::BoundingBoxOnly)
									{
										continue;
									}

									const FUIAction Action = FUIAction(
										FExecuteAction::CreateLambda([ToolbarContext, Mode]
											{
												if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
												{
													Viewport->OnVisualizationModeChanged(Mode);
												}
											}
										),
										FCanExecuteAction::CreateLambda([ToolbarContext, Mode]
											{
												if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
												{
													if (Mode != EPVVisualizationMode::Default)
													{
														TArray<EPVRenderType> RenderTypes = VisualizationModeToRenderModes(Mode);
														for (EPVRenderType RenderType : RenderTypes)
														{
															if (!Viewport->SupportedRenderTypes.Contains(RenderType))
															{
																return false;
															}
														}
													}

													return true;
												}
												return false;
											}),
										FIsActionChecked::CreateLambda([ToolbarContext, Mode]
											{
												if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
												{
													return Viewport->CurrentVisualizationMode == Mode;
												}
												return false;
											}
										)
									);

									if (Mode == EPVVisualizationMode::PointData)
									{
										RenderModeSelectionSection.AddSubMenu(
											NAME_None,
											ModeName,
											FText::GetEmpty(),
											FNewToolMenuDelegate::CreateLambda([ToolbarContext](UToolMenu* SubMenu)
												{
													if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
													{
														Viewport->CreatePointDataSettingsMenu(SubMenu);
													}
												}),
											Action,
											EUserInterfaceActionType::RadioButton
										);
									}
									else
									{
										RenderModeSelectionSection.AddMenuEntry(
											NAME_None,
											ModeName,
											FText::GetEmpty(),
											FSlateIcon(),
											Action,
											EUserInterfaceActionType::RadioButton
										);
									}
								}
							}

							RenderModeSelectionSection.AddSubMenu(
								TEXT("DebugModeSection"),
								LOCTEXT("DebugModeSelectionSubSectionLabel", "Debug Visualizations"),
								LOCTEXT("DebugModeSelectionSubSectionLabelToolTip", "Toggle Debug Visualizations"),
								FNewToolMenuDelegate::CreateLambda(
									[ToolbarContext](UToolMenu* SubMenu)
										{
											if (const UEnum* EnumPtr = StaticEnum<ESkeletonVisualizationModes>())
											{
												for (int32 i = 0; i < EnumPtr->NumEnums() - 1; i++)
												{
													const FText ModeName = EnumPtr->GetDisplayNameTextByIndex(i);
													const FString Category = EnumPtr->GetMetaData(TEXT("Category"), i);
													const ESkeletonVisualizationModes Mode = static_cast<ESkeletonVisualizationModes>(EnumPtr->GetValueByIndex(i));

													FToolMenuSection& DebugModeSelectionSection = SubMenu->FindOrAddSection(
														*Category,
														FText::FromString(Category)
													);

													DebugModeSelectionSection.AddMenuEntry(
														NAME_None,
														ModeName,
														FText(),
														FSlateIcon(),
														FUIAction(
															FExecuteAction::CreateLambda(
																[ToolbarContext, Mode]
																	{
																		if (const TSharedPtr<SPVEditorViewport> Viewport =
																			GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
																		{
																			Viewport->DebugVisualizationMode = Mode;
																			if (Viewport->SkeletonVisualizerComponent)
																			{
																				Viewport->SkeletonVisualizerComponent->SetVisualizationMode(Mode);
																			}
																		}
																	}
															),
															FCanExecuteAction::CreateLambda(
																[ToolbarContext, Mode]
																	{
																		if (const TSharedPtr<SPVEditorViewport> Viewport =
																			GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
																		{
																			return Viewport->SkeletonVisualizerComponent != nullptr;
																		}
																		return false;
																	}
															),
															FIsActionChecked::CreateLambda(
																[ToolbarContext, Mode]
																	{
																		if (const TSharedPtr<SPVEditorViewport> Viewport =
																			GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
																		{
																			return Viewport->SkeletonVisualizerComponent && Viewport->
																				SkeletonVisualizerComponent->
																				GetVisualizationMode() == Mode;
																		}
																		return false;
																	}
															)
														),
														EUserInterfaceActionType::RadioButton
													);
												}
											}
										}
								)
							);
						}
					),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Visualizers")
				);
				Entry.ToolBarData.ResizeParams.ClippingPriority = 1100;
			}
		)
	);
}

void SPVEditorViewport::CreatePointDataSettingsMenu(UToolMenu* InPointDataSubMenu)
{
	TWeakObjectPtr ToolbarContext(InPointDataSubMenu->FindContext<UUnrealEdViewportToolbarContext>());

	FToolMenuSection& PointDataSettingsSection = InPointDataSubMenu->FindOrAddSection(
		"PointDataSettingsSection",
		LOCTEXT("PointDataSettingsSectionLabel", "Point Data Settings")
	);
	PointDataSettingsSection.AddMenuEntry(
		"UseMeshSkeletonPreview",
		LOCTEXT("UseMeshSkeletonPreviewLabel", "Mesh Preview"),
		LOCTEXT("UseMeshSkeletonPreviewTooltip", "Show skeleton as mesh geometry (spheres + cylinders) instead of lines"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ToolbarContext]
				{
					UPVEditorSettings* Settings = GetMutableDefault<UPVEditorSettings>();
					Settings->bUseMeshSkeletonPreview = !Settings->bUseMeshSkeletonPreview;
					Settings->SaveConfig();
					if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
					{
						if (Viewport->SkeletonVisualizerComponent)
						{
							Viewport->SkeletonVisualizerComponent->SetUseMeshPreview(Settings->bUseMeshSkeletonPreview);
						}
					}
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]
				{
					return GetDefault<UPVEditorSettings>()->bUseMeshSkeletonPreview;
				})
		),
		EUserInterfaceActionType::ToggleButton
	);
	FSinglePropertyParams Params;
	Params.NamePlacement = EPropertyNamePlacement::Hidden;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const TSharedPtr<ISinglePropertyView> ColorView = PropertyEditorModule.CreateSingleProperty(
		GetMutableDefault<UPVEditorSettings>(),
		GET_MEMBER_NAME_CHECKED(UPVEditorSettings, SkeletonDefaultColor),
		Params
	);
	if (ColorView.IsValid())
	{
		ColorView->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
			[ToolbarContext]()
				{
					UPVEditorSettings* Settings = GetMutableDefault<UPVEditorSettings>();
					Settings->SaveConfig();
					if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
					{
						if (Viewport->SkeletonVisualizerComponent)
						{
							Viewport->SkeletonVisualizerComponent->RebuildSkeleton();
						}
					}
				}
		));
		PointDataSettingsSection.AddEntry(FToolMenuEntry::InitWidget(
			"SkeletonDefaultColor",
			ColorView.IsValid() ? ColorView.ToSharedRef() : SNullWidget::NullWidget,
			LOCTEXT("SkeletonDefaultColorLabel", "Skeleton Default Color")
		));
	}

	const TSharedPtr<ISinglePropertyView> PointScaleView = PropertyEditorModule.CreateSingleProperty(
		GetMutableDefault<UPVEditorSettings>(),
		GET_MEMBER_NAME_CHECKED(UPVEditorSettings, PointScaleBias),
		Params
	);
	if (PointScaleView.IsValid())
	{
		PointScaleView->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
			[ToolbarContext]()
				{
					UPVEditorSettings* Settings = GetMutableDefault<UPVEditorSettings>();
					Settings->SaveConfig();
					if (const TSharedPtr<SPVEditorViewport> Viewport = GetViewportFromViewportToolbarContext(ToolbarContext.Get()))
					{
						if (Viewport->SkeletonVisualizerComponent)
						{
							Viewport->SkeletonVisualizerComponent->RebuildSkeleton();
						}
					}
				}
		));
		PointDataSettingsSection.AddEntry(FToolMenuEntry::InitWidget(
			"PointScale",
			PointScaleView.IsValid() ? PointScaleView.ToSharedRef() : SNullWidget::NullWidget,
			LOCTEXT("PointScaleLabel", "Point Scale")
		));
	}
}

void SPVEditorViewport::OnVisualizationModeChanged(EPVVisualizationMode InMode)
{
	CurrentVisualizationMode = InMode;

	if (InspectedNodeSettings)
	{
		const TArray<EPVRenderType> RenderTypes = VisualizationModeToRenderModes(CurrentVisualizationMode,
			InspectedNodeSettings->GetDefaultRenderType());

		InspectedNodeSettings->SetCurrentRenderType(RenderTypes);
		if (UObject* Object = Cast<UObject>(InspectedNodeSettings))
		{
			Object->PostEditChange();
		}

		Invalidate();

		if (IsAutoFocusViewportChecked())
		{
			bFocusOnNextUpdate = true;
		}
	}
}

void SPVEditorViewport::OnNodeInspectionChanged(IPVRenderSettings* InSettings)
{
	if (InspectedNodeSettings != InSettings)
	{
		SupportedRenderTypes = InSettings->GetSupportedRenderTypes();
		CurrentVisualizationMode = EPVVisualizationMode::Default;
		InspectedNodeSettings = InSettings;
		DebugVisualizationMode = ESkeletonVisualizationModes::None;

		InspectedNodeSettings->SetCurrentRenderType(InSettings->GetDefaultRenderType());

		if (IsAutoFocusViewportChecked())
		{
			bFocusOnNextUpdate = true;
		}
	}
}

void SPVEditorViewport::ClearNodeInspection()
{
	SupportedRenderTypes = {};
	CurrentVisualizationMode = EPVVisualizationMode::Default;
	InspectedNodeSettings = nullptr;
}

void SPVEditorViewport::SetOverlayText(const FText& CurrentlyLockedNodeName)
{
	bIsPreviewingLockedNode = !CurrentlyLockedNodeName.IsEmpty();
	static FText NormalTextStyle = FText::FromString(TEXT("<TextBlock.ShadowedText>Previewing Output of Node: {0}</>"));
	FTextBuilder FinalText;
	FinalText.AppendLineFormat(NormalTextStyle, CurrentlyLockedNodeName);
	OverlayText->SetText(FinalText.ToText());
}

void SPVEditorViewport::PopulateStatsOverlayText(const TArrayView<FText> TextItems)
{
	static FText NormalTextStyle = FText::FromString(TEXT("<TextBlock.ShadowedText>{0}</>"));

	FTextBuilder FinalText;
	for (const FText& TextItem : TextItems)
	{
		FinalText.AppendLineFormat(NormalTextStyle, TextItem);
	}

	StatsOverlayText->SetText(FinalText.ToText());
}

void SPVEditorViewport::PopulateLegendOverlayText(const TArrayView<TPair<FColor, FText>> LegendItems)
{
	static FText LegendStyle = FText::FromString(TEXT("<block hex=\"{0}\">          </>\t<TextBlock.ShadowedText>{1}</>"));

	LegendData.Empty();
	for (const auto& [Color, Text] : LegendItems)
	{
		LegendData.Add(MakeShared<FText>(FText::Format(LegendStyle, FText::FromString(Color.ToHex()), Text)));
	}

	if (LegendListView)
	{
		LegendListView->RequestListRefresh();
	}
}

void SPVEditorViewport::AddInstancedFoliageComponent(TObjectPtr<UMeshComponent> Component)
{
	FoliageComponents->Add(Component);
	Component->SetVisibility(FoliageVisibility->FindOrAdd(Component->GetName(), true), true);
}

void SPVEditorViewport::SetSkeletonVisualizerComponent(TObjectPtr<UPVSkeletonVisualizerComponent> InSkeletonVisualizerComponent)
{
	SkeletonVisualizerComponent = InSkeletonVisualizerComponent;
	if (SkeletonVisualizerComponent)
	{
		SkeletonVisualizerComponent->SetUseMeshPreview(GetDefault<UPVEditorSettings>()->bUseMeshSkeletonPreview);
		SkeletonVisualizerComponent->SetVisualizationMode(DebugVisualizationMode);
	}
}

#undef LOCTEXT_NAMESPACE
