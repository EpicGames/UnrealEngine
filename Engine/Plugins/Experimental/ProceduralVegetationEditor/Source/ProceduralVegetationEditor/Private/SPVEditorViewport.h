// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Visualizations/PVSkeletonVisualizerComponent.h"

#include "Widgets/AssetEditorViewport/SPCGEditorViewport.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Views/SListView.h"

#include "SPVEditorViewport.generated.h"

class IPVRenderSettings;

UENUM()
enum class EPVVisualizationMode : uint8
{
	Default							UMETA(DisplayName = "Default"),
	PointData						UMETA(DisplayName = "Point data"),
	Mesh							UMETA(DisplayName = "Mesh"),
	FoliageGrid						UMETA(DisplayName = "Foliage grid"),
	Bones							UMETA(DisplayName = "Bones"),
	PointDataLeafProxy				UMETA(DisplayName = "Point data + Leaf Proxy"),
	PointDataMesh					UMETA(DisplayName = "Point data + Mesh"),
	PointDataFoliageAttachment		UMETA(DisplayName = "Point data + Foliage attachments"),
	FoliageMesh						UMETA(DisplayName = "Foliage + Mesh"),
	FoliageAttachmentMesh			UMETA(DisplayName = "Foliage attachments + Mesh"),
	BonesMesh						UMETA(DisplayName = "Bones + Mesh"),
	GraftsGrid						UMETA(DisplayName = "Grafts grid"),
	BoundingBoxOnly					UMETA(Hidden, DisplayName = "Bounding Box Only")
};

struct FLegendData
{
	FLinearColor Color;
	FText Text;
};

class UMeshComponent;

class UPVBaseSettings;
class UPVEditorSettings;
class UPVScaleVisualizationComponent;
struct FToolMenuEntry;
enum class EPVRenderType : uint16;

UCLASS(MinimalAPI)
class UPVMannequinWidgetContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<SWidget> MannequinOffsetWidget = nullptr;
};

UCLASS(MinimalAPI)
class UPVFoliageComponentsContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<TArray<TObjectPtr<UMeshComponent>>> FoliageComponents = nullptr;
	TSharedPtr<TMap<FString, bool>> FoliageVisibility = nullptr;
};

class SPVEditorViewport : public SPCGEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SPVEditorViewport)
			: _ModeTools(nullptr)
	{}
		SLATE_ARGUMENT(FEditorModeTools*, ModeTools)
	SLATE_END_ARGS()

	SPVEditorViewport();
	
	void Construct(const FArguments& InArgs);
	FToolMenuEntry CreateSettingsToolbarMenu();
	FToolMenuEntry CreateVisualizationModeToolbarMenu();
	void CreatePointDataSettingsMenu(UToolMenu* InPointDataSubMenu);

	void OnNodeInspectionChanged(IPVRenderSettings* InSettings);
	void ClearNodeInspection();

	void SetOverlayText(const FText& CurrentlyLockedNodeName = FText::GetEmpty());
	void PopulateStatsOverlayText(const TArrayView<FText> TextItems);
	void PopulateLegendOverlayText(const TArrayView<TPair<FColor, FText>> LegendItems);

	void AddInstancedFoliageComponent(TObjectPtr<UMeshComponent> Component);
	void SetSkeletonVisualizerComponent(TObjectPtr<UPVSkeletonVisualizerComponent> InSkeletonVisualizerComponent);

	TSharedRef<FAdvancedPreviewScene> GetAdvancedPreviewSceneRef() const
	{
		return AdvancedPreviewScene.ToSharedRef();
	}
	
	TOptional<ELevelViewportType> OverriddenViewportType;

protected:
	//~ Begin SEditorViewport Interface
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual void BindCommands() override;
	//~ End SEditorViewport Interface

	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;


private:
	void InitVisualizationScene();
	void ResetVisualizationScene();

	TSharedRef<SWidget> CreateMannequinOffsetWidget() const;

	void ToggleShowMannequin();
	bool IsShowMannequinChecked() const;
	void SetMannequinState(bool InEnable);

	void ToggleShowScaleVis();
	bool IsShowScaleVisChecked() const;
	void SetScaleVisState(bool InEnable);

	void ToggleShowStats();
	bool IsShowStatsChecked() const;
	
	void ToggleAutofocusViewport();
	bool IsAutoFocusViewportChecked() const;

	float GetMannequinOffset() const;
	void SetMannequinOffset(float NewValue, bool bSaveConfig = false) const;

	UPVMannequinWidgetContext* CreateMannequinWidgetContext();
	UPVFoliageComponentsContext* CreateFoliageComponentsContext();

	TArray<EPVRenderType> SupportedRenderTypes;
	EPVVisualizationMode CurrentVisualizationMode = EPVVisualizationMode::Default;
	IPVRenderSettings* InspectedNodeSettings = nullptr;

	void OnVisualizationModeChanged(EPVVisualizationMode InMode);

	bool bFocusOnNextUpdate = true;
	
protected:
	virtual void OnSetupScene() override;
	virtual void OnResetScene() override;

private:
	TObjectPtr<UStaticMeshComponent> MannequinComponent = nullptr;
	TObjectPtr<UPVScaleVisualizationComponent> ScaleVisualizationComponent = nullptr;
	TObjectPtr<UPVSkeletonVisualizerComponent> SkeletonVisualizerComponent = nullptr;

	TSharedPtr<SWidget> MannequinOffsetWidget;
	TStrongObjectPtr<UPVMannequinWidgetContext> MannequinWidgetContext;
	TStrongObjectPtr<UPVFoliageComponentsContext> FoliageComponentsContext;

	TSharedPtr<TArray<TObjectPtr<UMeshComponent>>> FoliageComponents;
	TSharedPtr<TMap<FString, bool>> FoliageVisibility;

	bool bIsPreviewingLockedNode = false;
	TSharedPtr<SRichTextBlock> OverlayText;
	TSharedPtr<SRichTextBlock> StatsOverlayText;
	TArray<TSharedPtr<FText>> LegendData;
	ESkeletonVisualizationModes DebugVisualizationMode = ESkeletonVisualizationModes::None;

	TSharedPtr<FSlateBrush> PreviewNodeBackgroundBrush;
	TSharedPtr<SListView<TSharedPtr<FText>>> LegendListView;
};
