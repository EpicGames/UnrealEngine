// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreTypes.h"

#include "Tests/PVTestsCommon.h"
#include "PVTestScenario.h"
#include "DataTypes/PVData.h"
#include "ProceduralVegetation.h"

#include "PCGDefaultExecutionSource.h"
#include "PCGSettings.h"
#include "Subsystems/PCGEngineSubsystem.h"

#include "Misc/AutomationTest.h"
#include "AssetRegistry/AssetRegistryModule.h"

namespace PVScenarioTests
{
	template<typename TAttributeType>
	bool CompareAttributeValue(const TAttributeType& A, const TAttributeType& B) { return A == B; }

	template<typename TAttributeType>
	bool CompareAttributeValue(const TAttributeType* A, const TAttributeType* B) 
	{ 
		if (A == B)
		{
			return true;
		}

		if (A == nullptr || B == nullptr)
		{
			return false;
		}

		return CompareAttributeValue<TAttributeType>(*A, *B);
	}

	template<typename TAttributeType>
	bool CompareAttributeValue(const TUniquePtr<TAttributeType>& A, const TUniquePtr<TAttributeType>& B)
	{
		if (!A.IsValid() && !B.IsValid())
		{
			return true;
		}

		if (!A.IsValid() || !B.IsValid())
		{
			return false;
		}

		return CompareAttributeValue<TAttributeType>(*A, *B);
	}

	template<typename TAttributeType>
	bool CompareAttributeValue(const TArray<TAttributeType>& A, const TArray<TAttributeType>& B) 
	{ 
		if (A.Num() != B.Num())
		{
			return false;
		}

		for (int32 i = 0; i < A.Num(); ++i)
		{
			if (!CompareAttributeValue<TAttributeType>(A[i], B[i]))
			{
				return false;
			}
		}

		return true;
	}

	template<> bool CompareAttributeValue<FVector4f>(const FVector4f& A, const FVector4f& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FVector3f>(const FVector3f& A, const FVector3f& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FVector3d>(const FVector3d& A, const FVector3d& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FVector2f>(const FVector2f& A, const FVector2f& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FLinearColor>(const FLinearColor& A, const FLinearColor& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FQuat4f>(const FQuat4f& A, const FQuat4f& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FTransform>(const FTransform& A, const FTransform& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<FTransform3f>(const FTransform3f& A, const FTransform3f& B) { return A.Equals(B); }
	template<> bool CompareAttributeValue<float>(const float& A, const float& B) { return FMath::IsNearlyEqual(A, B); }
	template<> bool CompareAttributeValue<double>(const double& A, const double& B) { return FMath::IsNearlyEqual(A, B); }
	template<> bool CompareAttributeValue<TSet<int32>>(const TSet<int32>& A, const TSet<int32>& B) { return A.Num() == B.Num() && A.Includes(B); }

	template<typename TAttributeType>
	bool CompareAttributes_Internal(
		const FManagedArrayCollection& CollectionA,
		const FManagedArrayCollection& CollectionB,
		const FName& InAttributeName,
		const FName& InGroupName
	)
	{
		if (CollectionA.NumElements(InGroupName) != CollectionB.NumElements(InGroupName))
		{
			return false;
		}

		const TManagedArray<TAttributeType>& AttributeA = CollectionA.GetAttribute<TAttributeType>(InAttributeName, InGroupName);
		const TManagedArray<TAttributeType>& AttributeB = CollectionB.GetAttribute<TAttributeType>(InAttributeName, InGroupName);

		for (int32 i = 0; i < AttributeA.Num(); ++i)
		{
			if (!CompareAttributeValue(AttributeA[i], AttributeB[i]))
			{
				return false;
			}
		}

		return true;
	}

	bool CompareAttributes(
		const FManagedArrayCollection& CollectionA, 
		const FManagedArrayCollection& CollectionB, 
		const FName& InAttributeName, 
		const FName& InGroupName
	)
	{
		const FManagedArrayCollection::EArrayType ArrayTypeA = CollectionA.GetAttributeType(InAttributeName, InGroupName);
		const FManagedArrayCollection::EArrayType ArrayTypeB = CollectionB.GetAttributeType(InAttributeName, InGroupName);
		if (ArrayTypeA != ArrayTypeB)
		{
			return false;
		}

		switch (ArrayTypeA)
		{
		case FManagedArrayCollection::EArrayType::FVectorType:
			return CompareAttributes_Internal<FVector3f>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVectorType:
			return CompareAttributes_Internal<FIntVector>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector2DType:
			return CompareAttributes_Internal<FVector2f>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FLinearColorType:
			return CompareAttributes_Internal<FLinearColor>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FInt32Type:
			return CompareAttributes_Internal<int32>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FBoolType:
			return CompareAttributes_Internal<bool>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FTransformType:
			return CompareAttributes_Internal<FTransform>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FStringType:
			return CompareAttributes_Internal<FString>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FFloatType:
			return CompareAttributes_Internal<float>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FQuatType:
			return CompareAttributes_Internal<FQuat4f>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntArrayType:
			return CompareAttributes_Internal<TSet<int32>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FGuidType:
			return CompareAttributes_Internal<FGuid>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FUInt8Type:
			return CompareAttributes_Internal<uint8>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVectorArrayPointerType:
			return CompareAttributes_Internal<TArray<FVector3f>*>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVectorArrayUniquePointerType:
			return CompareAttributes_Internal<TUniquePtr<TArray<FVector3f>>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector2DArrayType:
			return CompareAttributes_Internal<TArray<FVector2f>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FDoubleType:
			return CompareAttributes_Internal<double>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector4Type:
			return CompareAttributes_Internal<FIntVector4>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector3dType:
			return CompareAttributes_Internal<FVector3d>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector2Type:
			return CompareAttributes_Internal<FIntVector2>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector2ArrayType:
			return CompareAttributes_Internal<TArray<FIntVector2>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FInt32ArrayType:
			return CompareAttributes_Internal<TArray<int32>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FFloatArrayType:
			return CompareAttributes_Internal<TArray<float>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector4fType:
			return CompareAttributes_Internal<FVector4f>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FFVectorArrayType:
			return CompareAttributes_Internal<TArray<FVector3f>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FTransform3fType:
			return CompareAttributes_Internal<FTransform3f>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FIntVector3ArrayType:
			return CompareAttributes_Internal<TArray<FIntVector3>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FVector4fArrayType:
			return CompareAttributes_Internal<TArray<FVector4f>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FUintVector2Type:
			return CompareAttributes_Internal<FUintVector2>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FUObjectArrayType:
			return CompareAttributes_Internal<TObjectPtr<UObject>>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FNameType:
			return CompareAttributes_Internal<FName>(CollectionA, CollectionB, InAttributeName, InGroupName);
		case FManagedArrayCollection::EArrayType::FSoftObjectPathType:
			return CompareAttributes_Internal<FSoftObjectPath>(CollectionA, CollectionB, InAttributeName, InGroupName);

		// Unused in PVE, just return true
		case FManagedArrayCollection::EArrayType::FMeshSectionType:
		case FManagedArrayCollection::EArrayType::FBoneNodeType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3PointerType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3UniquePointerType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3SerializablePtrType:
		case FManagedArrayCollection::EArrayType::FFBVHParticlesFloat3PointerType:
		case FManagedArrayCollection::EArrayType::FFBVHParticlesFloat3UniquePointerType:
		case FManagedArrayCollection::EArrayType::FTPBDRigidParticleHandle3fPtrType:
		case FManagedArrayCollection::EArrayType::FTPBDGeometryCollectionParticleHandle3fPtrType:
		case FManagedArrayCollection::EArrayType::FTGeometryParticle3fUniquePtrType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3ThreadSafeSharedPointerType:
		case FManagedArrayCollection::EArrayType::FFImplicitObject3SharedPointerType:
		case FManagedArrayCollection::EArrayType::FTPBDRigidClusteredParticleHandle3fPtrType:
		case FManagedArrayCollection::EArrayType::FFConvexUniquePtrType:
		case FManagedArrayCollection::EArrayType::FTPBDRigidParticle3fUniquePtrType:
		case FManagedArrayCollection::EArrayType::FFImplicitObjectRefCountedPtrType:
		case FManagedArrayCollection::EArrayType::FFConvexRefCountedPtrType:
		case FManagedArrayCollection::EArrayType::FPMatrix33dType:
		case FManagedArrayCollection::EArrayType::FLinearCurveType:
		case FManagedArrayCollection::EArrayType::FFVector3fNestedArrayType:
			return true;
		default:
			ensureAlwaysMsgf(false, TEXT("Unknown attribute type"));
			break;
		}

		return false;
	}

	static bool CompareManagedArrayCollections(const FManagedArrayCollection& A, const FManagedArrayCollection& B)
	{
		if (&A == &B || (A.IsEmpty() && B.IsEmpty()))
		{
			return true;
		}

		const TSet<FName> GroupsA(A.GroupNames());
		const TSet<FName> GroupsB(B.GroupNames());

		if (GroupsA.Num() != GroupsB.Num() || !GroupsA.Includes(GroupsB))
		{
			return false;
		}

		for (const FName GroupName : GroupsA)
		{
			const TSet<FName> AttributeNamesA = TSet<FName>(A.AttributeNames(GroupName));
			const TSet<FName> AttributeNamesB = TSet<FName>(B.AttributeNames(GroupName));

			if (AttributeNamesA.Num() != AttributeNamesB.Num() || !AttributeNamesA.Includes(AttributeNamesB))
			{
				return false;
			}

			for (const FName AttributeName : AttributeNamesA)
			{
				if (!CompareAttributes(A, B, AttributeName, GroupName))
				{
					return false;
				}
			}
		}

		return true;
	}

	static TArray<TWeakObjectPtr<UPVTestScenario>> LoadTestScenarios()
	{
		TArray<TWeakObjectPtr<UPVTestScenario>> TestScenarios;

		TArray<FAssetData> Assets;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().GetAssetsByClass(UPVTestScenario::StaticClass()->GetClassPathName(), Assets);

		TestScenarios.Reserve(Assets.Num());

		for (const FAssetData& Asset : Assets)
		{
			UPVTestScenario* TestScenario = Cast<UPVTestScenario>(Asset.GetAsset());
			if (TestScenario)
			{
				TestScenarios.Add(TestScenario);
			}
		}

		return TestScenarios;
	}

	struct FStrongExecutionSourcePtrArray
	{
		TArray<UPCGDefaultExecutionSource*> ExecutionSources;

		FStrongExecutionSourcePtrArray() = delete;
		FStrongExecutionSourcePtrArray(const FStrongExecutionSourcePtrArray&) = delete;
		FStrongExecutionSourcePtrArray(FStrongExecutionSourcePtrArray&&) = default;
		FStrongExecutionSourcePtrArray(TArray<UPCGDefaultExecutionSource*>&& InExecutionSources)
		{
			ExecutionSources = MoveTemp(InExecutionSources);
			for (UPCGDefaultExecutionSource* ExecutionSource : ExecutionSources)
			{
				if (ExecutionSource)
				{
					ExecutionSource->AddToRoot();
				}
			}
		}

		~FStrongExecutionSourcePtrArray()
		{
			for (UPCGDefaultExecutionSource* ExecutionSource : ExecutionSources)
			{
				if (ExecutionSource)
				{
					ExecutionSource->RemoveFromRoot();
					ExecutionSource->MarkAsGarbage();
				}
			}
		}
	};

	static TSharedRef<FStrongExecutionSourcePtrArray> CreateScenariosPCGExecutionSources(const TArray<TWeakObjectPtr<UPVTestScenario>>& InTestScenarios)
	{
		TArray<UPCGDefaultExecutionSource*> OutExecutionSources;
		OutExecutionSources.SetNum(InTestScenarios.Num());

		for (int32 i = 0; i < InTestScenarios.Num(); ++i)
		{
			const TWeakObjectPtr<UPVTestScenario> TestScenario = InTestScenarios[i];
			if (!TestScenario.IsValid())
			{
				continue;
			}

			UProceduralVegetation* ProceduralVegetation = TestScenario->ProceduralVegetationAsset.LoadSynchronous();
			if (!ProceduralVegetation)
			{
				continue;
			}

			UPCGGraphInterface* Graph = ProceduralVegetation->GetGraph();
			if (!Graph)
			{
				continue;
			}

			OutExecutionSources[i] = IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultExecutionSource>({ .GraphInterface = Graph });
			OutExecutionSources[i]->GetExecutionState().GetInspection().EnableInspection();
		}

		return MakeShared<FStrongExecutionSourcePtrArray>(MoveTemp(OutExecutionSources));
	}

	static const UPVData* GetPinPVData(const UPCGPin& Pin, UPCGDefaultExecutionSource& ExecutionSource)
	{
		FPCGStack Stack;
		Stack.PushFrame(&ExecutionSource);
		Stack.PushFrame(ExecutionSource.GetGraph());
		Stack.PushFrame(Pin.Node);
		Stack.PushFrame(&Pin);

		const UPVData* OutPVData = nullptr;
		ExecutionSource.GetExecutionState().GetInspection().InspectData(Stack, [&](const FPCGDataCollection& InDataCollection)
		{
			const FPCGTaggedData* TaggedData = InDataCollection.TaggedData.GetData();
			if (TaggedData != nullptr)
			{
				OutPVData = Cast<UPVData>(TaggedData->Data);
			}
		});

		return OutPVData;
	}

	static void ExtractGraphExecutionResults(UPCGDefaultExecutionSource& ExecutionSource, TMap<FPVPinResultKey, FManagedArrayCollection>& OutResults)
	{
		if (!ensureAlways(!ExecutionSource.GetExecutionState().IsGenerating()))
		{
			return;
		}

		const UPCGGraph* Graph = ExecutionSource.GetGraph();
		const TArray<UPCGNode*>& Nodes = Graph->GetNodes();
		for (const UPCGNode* Node : Nodes)
		{
			const TArray<TObjectPtr<UPCGPin>>& Pins = Node->GetOutputPins();
			for (const TObjectPtr<UPCGPin>& Pin : Pins)
			{
				const UPVData* ProceduralVegetationData = PVScenarioTests::GetPinPVData(*Pin, ExecutionSource);
				if (!ProceduralVegetationData)
				{
					continue;
				}
				
				OutResults.Add(FPVPinResultKey(Pin), ProceduralVegetationData->GetCollection());
			}
		}
	}

	static void CompareExecutionResults(
		const TMap<FPVPinResultKey, FManagedArrayCollection>& PrevResults, 
		const TMap<FPVPinResultKey, FManagedArrayCollection>& NewResults,
		TSet<FPVPinResultKey>& OutAddedPins,
		TSet<FPVPinResultKey>& OutRemovedPins,
		TSet<FPVPinResultKey>& OutModifiedPins
	)
	{
		TSet<FPVPinResultKey> PrevPins;
		TSet<FPVPinResultKey> NewPins;
		PrevResults.GetKeys(PrevPins);
		NewResults.GetKeys(NewPins);

		OutAddedPins = NewPins.Difference(PrevPins);
		OutRemovedPins = PrevPins.Difference(NewPins);

		const TSet<FPVPinResultKey> PinsInBoth = PrevPins.Intersect(NewPins);
		for (const FPVPinResultKey& PinKey : PinsInBoth)
		{
			const FManagedArrayCollection& PrevCollection = PrevResults[PinKey];
			const FManagedArrayCollection& NewCollection = NewResults[PinKey];
			if (!CompareManagedArrayCollections(PrevCollection, NewCollection))
			{
				OutModifiedPins.Add(PinKey);
			}
		}
	}

	static bool CompareExecutionResults(
		const TMap<FPVPinResultKey, FManagedArrayCollection>& PrevResults,
		const TMap<FPVPinResultKey, FManagedArrayCollection>& NewResults
	)
	{
		TSet<FPVPinResultKey> OutAddedPins;
		TSet<FPVPinResultKey> OutRemovedPins;
		TSet<FPVPinResultKey> OutModifiedPins;
		CompareExecutionResults(PrevResults, NewResults, OutAddedPins, OutRemovedPins, OutModifiedPins);
		return OutAddedPins.Num() == 0 && OutRemovedPins.Num() == 0 && OutModifiedPins.Num() == 0;
	}

	void RegenerateScenarioResults(TArray<TWeakObjectPtr<UPVTestScenario>> TestScenarios, TFunction<void()> OnRegenerationComplete)
	{
		TSharedRef<FStrongExecutionSourcePtrArray> ExecutionSourcePtrArray = PVScenarioTests::CreateScenariosPCGExecutionSources(TestScenarios);

		for (UPCGDefaultExecutionSource* ExecutionSource : ExecutionSourcePtrArray->ExecutionSources)
		{
			if (ExecutionSource)
			{
				ExecutionSource->Generate();
			}
		}

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
		[
			ExecutionSourcePtrArray,
			TestScenarios = MoveTemp(TestScenarios),
			OnRegenerationComplete = MoveTemp(OnRegenerationComplete)
		](float DeltaTime)->bool
		{
			bool bIsGenerating = false;
			for (UPCGDefaultExecutionSource* ExecutionSource : ExecutionSourcePtrArray->ExecutionSources)
			{
				if (ExecutionSource && ExecutionSource->GetExecutionState().IsGenerating())
				{
					return true;
				}
			}

			for (int32 i = 0; i < ExecutionSourcePtrArray->ExecutionSources.Num(); ++i)
			{
				UPCGDefaultExecutionSource* ExecutionSource = ExecutionSourcePtrArray->ExecutionSources[i];
				if (!ExecutionSource)
				{
					continue;
				}

				TMap<FPVPinResultKey, FManagedArrayCollection> ExecutionResults;
				PVScenarioTests::ExtractGraphExecutionResults(*ExecutionSource, ExecutionResults);

				TWeakObjectPtr<UPVTestScenario> TestScenario = TestScenarios[i];
				if (!TestScenario.IsValid())
				{
					UE_LOGF(LogPVTest, Warning, "Test scenario for graph unloaded during re-generation");
					continue;
				}
				
				if (!PVScenarioTests::CompareExecutionResults(TestScenario->ExecutionResults, ExecutionResults))
				{
					TestScenario->Modify(true);
					TestScenario->ExecutionResults = ExecutionResults;
				}
			}

			OnRegenerationComplete();

			return false;
		}));
	}

	static void RegenerateAllScenarioResults()
	{
		TArray<TWeakObjectPtr<UPVTestScenario>> TestScenarios = PVScenarioTests::LoadTestScenarios();
		RegenerateScenarioResults(MoveTemp(TestScenarios), [](){});
	}

	static FAutoConsoleCommand RegenerateScenarioResultsCommand(
		TEXT("PVE.TestScenarios.RegenerateAllResults"),
		TEXT("Regenerates the results data for all scenarios"),
		FConsoleCommandDelegate::CreateStatic(&RegenerateAllScenarioResults)
	);
};

PV_SIMPLE_AUTOMATION_TEST(ScenarioTests, All)
{
	using namespace PVScenarioTests;

	const TArray<TWeakObjectPtr<UPVTestScenario>> TestScenarios = PVScenarioTests::LoadTestScenarios();
	if (TestScenarios.Num() == 0)
	{
		return true;
	}

	TSharedRef<FStrongExecutionSourcePtrArray> ExecutionSourcePtrArray = PVScenarioTests::CreateScenariosPCGExecutionSources(TestScenarios);

	for (UPCGDefaultExecutionSource* ExecutionSource : ExecutionSourcePtrArray->ExecutionSources)
	{
		UTEST_NOT_NULL("Valid execution source", ExecutionSource);
		ExecutionSource->Generate();
	}

	AddCommand(new FFunctionLatentCommand([ExecutionSourcePtrArray]()->bool
	{
		for (UPCGDefaultExecutionSource* ExecutionSource : ExecutionSourcePtrArray->ExecutionSources)
		{
			check(ExecutionSource != nullptr);

			IPCGGraphExecutionState& ExecutionState = ExecutionSource->GetExecutionState();
			if (ExecutionState.IsGenerating())
			{
				return false;
			}
		}

		return true;
	}));

	AddCommand(new FFunctionLatentCommand([this, ExecutionSourcePtrArray, TestScenarios]()->bool
	{
		for (int32 i = 0; i < ExecutionSourcePtrArray->ExecutionSources.Num(); ++i)
		{
			UPCGDefaultExecutionSource* ExecutionSource = ExecutionSourcePtrArray->ExecutionSources[i];
			check(ExecutionSource != nullptr);

			TMap<FPVPinResultKey, FManagedArrayCollection> ExecutionResults;
			PVScenarioTests::ExtractGraphExecutionResults(*ExecutionSource, ExecutionResults);

			TWeakObjectPtr<UPVTestScenario> TestScenario = TestScenarios[i];
			if (!TestScenario.IsValid())
			{
				AddWarning(FString::Printf(TEXT("Test scenario unloaded during graph generation")));
				continue;
			}

			TSet<FPVPinResultKey> AddedPins;
			TSet<FPVPinResultKey> RemovedPins;
			TSet<FPVPinResultKey> ModifiedPins;
			PVScenarioTests::CompareExecutionResults(TestScenario->ExecutionResults, ExecutionResults, AddedPins, RemovedPins, ModifiedPins);

			for (const FPVPinResultKey& PinKey : AddedPins)
			{
				AddWarning(FString::Printf(TEXT("Unrecognized pin \"%s\", a new node has most likley been added"), *PinKey.ToString()));
			}
			
			for (const FPVPinResultKey& PinKey : RemovedPins)
			{
				AddWarning(FString::Printf(TEXT("Missing pin \"%s\", a new node has most likley been removed"), *PinKey.ToString()));
			}

			for (const FPVPinResultKey& PinKey : ModifiedPins)
			{
				AddWarning(FString::Printf(TEXT("Pin output value mismatch \"%s\""), *PinKey.ToString()));
			}
		}

		return true;
	}));

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS