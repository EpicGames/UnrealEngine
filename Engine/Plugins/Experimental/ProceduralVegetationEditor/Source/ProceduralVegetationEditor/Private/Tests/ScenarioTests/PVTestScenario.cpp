// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVTestScenario.h"
#include "PCGPin.h"
#include "PCGNode.h"
#include "PCGGraph.h"

FPVPinResultKey::FPVPinResultKey(const UPCGPin* InPin)
{
	if (!InPin || !InPin->Node || !InPin->Node->GetGraph())
	{
		PinPath = NAME_None;
		return;
	}

	UPCGNode* Node = InPin->Node.Get();
	UPCGGraph* PCGGraph = Node->GetGraph();

	PinPath = *(InPin->GetPathName(PCGGraph));
}

void UPVTestScenario::RegenerateResults()
{
#if WITH_DEV_AUTOMATION_TESTS
	if (bIsRegeneratingRestults)
	{
		return;
	}

	bIsRegeneratingRestults = true;
	TWeakObjectPtr<UPVTestScenario> WeakThis = this;
	PVScenarioTests::RegenerateScenarioResults(
		{ this }, 
		[WeakThis]() 
		{ 
			if (WeakThis.IsValid())
			{
				WeakThis->bIsRegeneratingRestults = false;
			}
		}
	);
#endif
}
