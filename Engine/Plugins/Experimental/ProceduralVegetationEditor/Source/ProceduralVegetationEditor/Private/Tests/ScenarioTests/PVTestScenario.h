// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "PVTestScenario.generated.h"

class UPCGPin;
class UPVTestScenario;
class UProceduralVegetation;

#if WITH_DEV_AUTOMATION_TESTS
namespace PVScenarioTests
{
	extern void RegenerateScenarioResults(TArray<TWeakObjectPtr<UPVTestScenario>> TestScenarios, TFunction<void()> OnRegenerationComplete);
};
#endif // WITH_DEV_AUTOMATION_TESTS

USTRUCT()
struct FPVPinResultKey
{
	GENERATED_BODY()

	FPVPinResultKey() = default;
	FPVPinResultKey(const UPCGPin* InPin);

	UPROPERTY(VisibleAnywhere, Category = "Pin Result Key")
	FName PinPath;

	FString ToString() const
	{
		return PinPath.ToString();
	}

	friend uint32 GetTypeHash(const FPVPinResultKey& InKey)
	{
		return GetTypeHash(InKey.PinPath);
	}

	bool operator==(const FPVPinResultKey& Other) const
	{
		return PinPath == Other.PinPath;
	}
};

UCLASS()
class UPVTestScenario : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="Test Scenario")
	FString TestName;

	UPROPERTY(EditAnywhere, Category="Test Scenario")
	TSoftObjectPtr<UProceduralVegetation> ProceduralVegetationAsset;

	UPROPERTY(VisibleAnywhere, Category="Test Scenario Result")
	TMap<FPVPinResultKey, FManagedArrayCollection> ExecutionResults;

	UFUNCTION(CallInEditor, Category = "Test Scenario Result")
	void RegenerateResults();

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

private:
	bool bIsRegeneratingRestults = false;
};