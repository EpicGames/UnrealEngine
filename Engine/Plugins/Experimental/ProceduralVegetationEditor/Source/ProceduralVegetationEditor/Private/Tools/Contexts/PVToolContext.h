// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Graph/PCGStackContext.h"

#include "PVToolContext.generated.h"

class UPVData;

/** PVE-specific context object stored alongside UPCGNodeToolContext in the context store.
 *  Carries vegetation data that PCG's generic context does not hold. */
UCLASS()
class UPVToolContextObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FPCGStack InspectionStack;
};
