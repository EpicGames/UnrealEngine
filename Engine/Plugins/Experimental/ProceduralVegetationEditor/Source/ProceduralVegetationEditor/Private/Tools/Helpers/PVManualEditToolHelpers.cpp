// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/Helpers/PVManualEditToolHelpers.h"

#include "EngineUtils.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"

#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"

#include "PVEditorCommon.h"

#include "Components/DynamicMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Visualizations/PVSkeletonVisualizerComponent.h"

namespace PV::Tools
{
	bool FManualEditAttributes::InitializeFromCollection(const FManagedArrayCollection& InCollection)
	{
		PointPosition = FPointPositionAttribute::FindAttribute(InCollection);
		PointScale = FPointScaleAttribute::FindAttribute(InCollection);
		PointLengthFromRoot = FPointLengthFromRootAttribute::FindAttribute(InCollection);
		PointBudNumber = FPointBudNumberAttribute::FindAttribute(InCollection);
		BranchPoints = FBranchPointsAttribute::FindAttribute(InCollection);
		BranchNumber = FBranchNumberAttribute::FindAttribute(InCollection);
		BranchChildren = FBranchChildrenAttribute::FindAttribute(InCollection);
		BranchParents = FBranchParentNumberAttribute::FindAttribute(InCollection);
		return IsValid();
	}

	bool FManualEditAttributes::IsValid() const
	{
		return PointPosition.IsValid() &&
			PointScale.IsValid() &&
			PointLengthFromRoot.IsValid() &&
			PointBudNumber.IsValid() &&
			BranchPoints.IsValid() &&
			BranchNumber.IsValid() &&
			BranchChildren.IsValid() &&
			BranchParents.IsValid();
	}

	bool IsPointRemoved(
		const FPointBudNumberAttributeConstView& PointBudNumberAttribute,
		const TArray<bool>& RemovedPoints,
		const int32 PointIndex
	)
	{
		const int32 BudNumber = PointBudNumberAttribute[PointIndex];
		return RemovedPoints.IsValidIndex(BudNumber) && RemovedPoints[BudNumber];
	}

	TArray<int32> GetBranchImmediateChildren(
		const FManualEditAttributes& Attributes,
		const int32 BranchIndex
	)
	{
		const int32 ParentBranchNumber = Attributes.BranchNumber[BranchIndex];
		return Attributes.BranchChildren[BranchIndex].FilterByPredicate(
			[&](const int32 Child)
				{
					const int32 ChildIndex = Attributes.BranchNumber.Find(Child);
					if (ChildIndex != INDEX_NONE && Attributes.BranchParents[ChildIndex] == ParentBranchNumber)
					{
						return true;
					}
					return false;
				}
		);
	}

	bool GetPointWorldPositionAndScale(
		const FManualEditAttributes& Attributes,
		const TArray<FVector3f>& Offsets,
		const int32 BranchIndex,
		const int32 BranchPointIndex,
		FVector& OutPosition,
		float& OutScale
	)
	{
		const TArray<int32>& BranchPoints = Attributes.BranchPoints[BranchIndex];
		if (!BranchPoints.IsValidIndex(BranchPointIndex))
		{
			return false;
		}

		const int32 PointIndex = BranchPoints[BranchPointIndex];
		const int32 BudNumber = Attributes.PointBudNumber[PointIndex];
		const FVector3f Offset = Offsets.IsValidIndex(BudNumber) ? Offsets[BudNumber] : FVector3f::ZeroVector;
		OutPosition = FVector(Attributes.PointPosition[PointIndex] + Offset);
		OutScale = Attributes.PointScale[PointIndex];
		return true;
	}

	void DrawPointHighlight(
		FPrimitiveDrawInterface* PDI,
		const FVector& Position,
		const float Scale,
		const FLinearColor& Color
	)
	{
		using namespace PV::EditorCommon;
		DrawWireSphere(
			PDI,
			Position,
			Color,
			FMath::Max(PointMinScale, Scale) * HighlightPointScaleBias,
			PointHighlightNumSides,
			HighlightPointDepthPriority,
			HighlightLineThickness
		);
	}

	void DrawEdgeHighlight(
		FPrimitiveDrawInterface* PDI,
		const FVector& PosA, const FVector& PosB,
		const float RadiusA, const float RadiusB,
		const FLinearColor& Color
	)
	{
		using namespace PV::EditorCommon;

		const FVector Dir = (PosB - PosA).GetSafeNormal();
		if (Dir.IsZero())
		{
			return;
		}

		FVector AxisX, AxisY;
		Dir.FindBestAxisVectors(AxisX, AxisY);

		for (int32 Index = 0; Index < EdgeCylinderNumSides; ++Index)
		{
			const double Angle = UE_DOUBLE_TWO_PI * static_cast<double>(Index) / static_cast<double>(EdgeCylinderNumSides);
			const FVector Radial = (FMath::Cos(Angle) * AxisX + FMath::Sin(Angle) * AxisY) * HighlightEdgeScaleBias;
			const FVector Bot = PosA + RadiusA * Radial;
			const FVector Top = PosB + RadiusB * Radial;
			PDI->DrawLine(Bot, Top, Color, HighlightEdgeDepthPriority, HighlightLineThickness);
		}

		DrawCircle(
			PDI,
			PosA, AxisX, AxisY,
			Color, RadiusA * HighlightEdgeScaleBias, EdgeCylinderNumSides,
			HighlightEdgeDepthPriority, HighlightLineThickness
		);
		DrawCircle(
			PDI,
			PosB, AxisX, AxisY,
			Color, RadiusB * HighlightEdgeScaleBias, EdgeCylinderNumSides,
			HighlightEdgeDepthPriority, HighlightLineThickness
		);
	}

	int32 GetPointIndexFromHitProxy(HHitProxy* InProxy)
	{
		if (const HInstancedStaticMeshInstance* InstanceProxy = HitProxyCast<HInstancedStaticMeshInstance>(InProxy))
		{
			return InstanceProxy->InstanceIndex;
		}
		return INDEX_NONE;
	}

	UPVSkeletonVisualizerComponent* GetVisualizerComponentFromHitProxy(HHitProxy* InProxy)
	{
		if (const HActor* ActorHitProxy = HitProxyCast<HActor>(InProxy))
		{
			return Cast<UPVSkeletonVisualizerComponent>(ActorHitProxy->PrimComponent.GetOuter());
		}
		return nullptr;
	}

	bool FindBranchSelectionFromRay(
		const UPVSkeletonVisualizerComponent* VisualizerComponent,
		const FRay& WorldRay,
		int32& OutBranchIndex,
		int32& OutBranchPointIndex
	)
	{
		check(VisualizerComponent);

		OutBranchIndex = INDEX_NONE;
		OutBranchPointIndex = INDEX_NONE;

		const UE::Geometry::FDynamicMeshAABBTree3& MeshOctree = VisualizerComponent->GetDynamicMeshOctree();
		const int HitTID = MeshOctree.FindNearestHitTriangle(WorldRay, {});
		if (HitTID == IndexConstants::InvalidID)
		{
			return false;
		}

		const TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = VisualizerComponent->GetDynamicMeshComponent();
		const FDynamicMesh3* DynamicMesh = DynamicMeshComponent->GetMesh();

		if (!DynamicMesh->IsTriangle(HitTID))
		{
			return false;
		}

		const PV::TDynamicMeshVertexAttributeExt<int32>* BranchAttr = PV::EditorCommon::BranchIndexAttribute.GetAttribute(*DynamicMesh);
		const PV::TDynamicMeshVertexAttributeExt<int32>* BranchPointAttr = PV::EditorCommon::BranchPointIndexAttribute.GetAttribute(*DynamicMesh);
		if (!BranchAttr || !BranchPointAttr)
		{
			return false;
		}

		const UE::Geometry::FIndex3i Tri = DynamicMesh->GetTriangle(HitTID);

		// All three vertices of any sphere-stamped or cylinder-side-stamped triangle carry the same
		// (BranchIndex, BranchPointIndex), so reading vertex 0 is enough.
		OutBranchIndex = BranchAttr->GetValue(Tri.A);
		OutBranchPointIndex = BranchPointAttr->GetValue(Tri.A);
		
		if (OutBranchIndex != INDEX_NONE && OutBranchPointIndex != INDEX_NONE)
		{
			return true;
		}

		return false;
	}
}
