// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryCollection/ManagedArray.h"

#include "Utils/PVAttributes.h"

struct FManagedArrayCollection;

class FPrimitiveDrawInterface;
class UPVSkeletonVisualizerComponent;

namespace PV::Facades
{
	class FPointFacade;
}


namespace PV::Tools
{
	enum class ESkeletonHoverType : uint8
	{
		None,
		Point,
		Edge
	};

	struct FManualEditAttributes
	{
		FPointPositionAttributeConstView PointPosition;
		FPointScaleAttributeConstView PointScale;
		FPointLengthFromRootAttributeConstView PointLengthFromRoot;
		FPointBudNumberAttributeConstView PointBudNumber;
		FBranchPointsAttributeConstView BranchPoints;
		FBranchNumberAttributeConstView BranchNumber;
		FBranchChildrenAttributeConstView BranchChildren;
		FBranchParentNumberAttributeConstView BranchParents;

		bool InitializeFromCollection(const FManagedArrayCollection& InCollection);

		bool IsValid() const;
	};

	// Returns true if PointIndex's BudNumber maps to a "removed" entry in RemovedPoints.
	bool IsPointRemoved(
		const FPointBudNumberAttributeConstView& PointBudNumberAttribute,
		const TArray<bool>& RemovedPoints,
		int32 PointIndex
	);

	TArray<int32> GetBranchImmediateChildren(const FManualEditAttributes& Attributes, const int32 BranchIndex);

	// Compute world-space position and per-point scale for a (BranchIndex, BranchPointIndex). Returns false on any
	// invalid input (collection invalid, branch out of range, point index out of range, etc.).
	bool GetPointWorldPositionAndScale(
		const FManualEditAttributes& Attributes,
		const TArray<FVector3f>& Offsets,
		const int32 BranchIndex,
		const int32 BranchPointIndex,
		FVector& OutPosition,
		float& OutScale
	);

	// Wireframe sphere drawn slightly larger than the mesh sphere so it visually wraps it.
	void DrawPointHighlight(
		FPrimitiveDrawInterface* PDI,
		const FVector& Position,
		float Scale,
		const FLinearColor& Color
	);

	// Six PDI lines along the longitudinal ridges of the tapered cylinder built by AppendTaperedCylinder,
	// plus two end-cap circles. Tessellation matches PV::EditorCommon::EdgeCylinderNumSides.
	void DrawEdgeHighlight(
		FPrimitiveDrawInterface* PDI,
		const FVector& PosA, const FVector& PosB,
		float RadiusA, float RadiusB,
		const FLinearColor& Color
	);

	int32 GetPointIndexFromHitProxy(HHitProxy* InProxy);
	
	UPVSkeletonVisualizerComponent* GetVisualizerComponentFromHitProxy(HHitProxy* InProxy);

	bool FindBranchSelectionFromRay(
		const UPVSkeletonVisualizerComponent* VisualizerComponent,
		const FRay& WorldRay,
		int32& OutBranchIndex,
		int32& OutBranchPointIndex
	);
}
