// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/Helpers/PVSkeletonPointsSelection.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"

#include "GeometryCollection/ManagedArrayCollection.h"

namespace
{
	// Walk one step toward the root: given a branch, find the (BranchIndex, BranchPointIndex) of the
	// parent-branch point whose length-from-root is just past this branch's root point.
	TPair<int32, int32> GetParentPointOnParentBranch(
		const PV::Facades::FPointFacade& PointFacade,
		const PV::Facades::FBranchFacade& BranchFacade,
		const int32 BranchIndex)
	{
		const int32 RootPointIndex = BranchFacade.GetRootPoint(BranchIndex);
		if (RootPointIndex != INDEX_NONE)
		{
			const float PointLFR = PointFacade.GetLengthFromRoot(RootPointIndex);
			const int32 ParentBranchIndex = BranchFacade.GetParentBranchIndex(BranchIndex);
			if (ParentBranchIndex != INDEX_NONE)
			{
				const TArray<int32>& ParentBranchPoints = BranchFacade.GetPoints(ParentBranchIndex);
				for (int32 BranchPointIndex = 0; BranchPointIndex < ParentBranchPoints.Num(); BranchPointIndex++)
				{
					if (PointFacade.GetLengthFromRoot(ParentBranchPoints[BranchPointIndex]) >= PointLFR)
					{
						return {ParentBranchIndex, BranchPointIndex};
					}
				}
			}
		}
		return {INDEX_NONE, INDEX_NONE};
	}

	// Find the immediate child branches whose root point sits in the segment between
	// (BranchIndex, BranchPointIndex) and the next point on the same branch.
	TArray<TPair<int32, int32>> GetChildPointsAtSegment(
		const PV::Facades::FPointFacade& PointFacade,
		const PV::Facades::FBranchFacade& BranchFacade,
		const int32 BranchIndex,
		const int32 BranchPointIndex,
		const TArray<int32>& ImmediateChildBranches)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);

		const float PointLFR = PointFacade.GetLengthFromRoot(BranchPoints[BranchPointIndex]);
		const float NextPointLFR = BranchPoints.IsValidIndex(BranchPointIndex + 1)
			? PointFacade.GetLengthFromRoot(BranchPoints[BranchPointIndex + 1])
			: UE_MAX_FLT;

		TArray<TPair<int32, int32>> ChildPoints;
		for (int32 ChildBranch : ImmediateChildBranches)
		{
			const int32 ChildBranchIndex = BranchFacade.GetBranchNumbers().Find(ChildBranch);
			if (ChildBranchIndex != INDEX_NONE)
			{
				const int32 ChildBranchRootPointIndex = BranchFacade.GetRootPoint(ChildBranchIndex);
				if (ChildBranchRootPointIndex != INDEX_NONE)
				{
					const float ChildLFR = PointFacade.GetLengthFromRoot(ChildBranchRootPointIndex);
					if (ChildLFR >= PointLFR && ChildLFR < NextPointLFR)
					{
						ChildPoints.Emplace(ChildBranchIndex, 0);
					}
				}
			}
		}

		return ChildPoints;
	}
}

FSkeletonPointsSelection::FSkeletonPointsSelection(const FManagedArrayCollection* const InCollection)
	: Collection(InCollection)
{
	check(Collection);
}

void FSkeletonPointsSelection::SelectPoint(const int32 InBranchIndex, const int32 InBranchPointIndex)
{
	PrimarySelectedPoint.BranchIndex = InBranchIndex;
	PrimarySelectedPoint.BranchPointIndex = InBranchPointIndex;
	PrimarySelectedPoint.Weight = 1.0f;
	PrimarySelectedPoint.NeighbourIndex = 0;

	SelectedPoints.Empty();
}

void FSkeletonPointsSelection::ExtendSelection(
	const ESkeletonSelectionMode SelectionMode,
	const float ExtensionLimit,
	const TArray<FVector3f>& Offsets,
	const EPointSelectionSmoothnessMethod SmoothnessMethod
)
{
	check(Collection);

	const PV::Facades::FPointFacade PointFacade(*Collection);
	const PV::Facades::FBranchFacade BranchFacade(*Collection);
	if (!PointFacade.IsValid() || !BranchFacade.IsValid())
	{
		return;
	}

	SelectedPoints.Empty();
	PrimarySelectedPoint.Weight = 0.0f;
	SelectedPoints.Add(PrimarySelectedPoint);

	switch (SelectionMode)
	{
	case ESkeletonSelectionMode::SelectByNeighbours:
		ExtendSelectionByNeighbours(PointFacade, BranchFacade, ExtensionLimit);
		break;

	case ESkeletonSelectionMode::SelectByTreeDistance:
		ExtendSelectionByTreeDistance(PointFacade, BranchFacade, Offsets, ExtensionLimit);
		break;

	case ESkeletonSelectionMode::SelectByEuclideanDistance:
		ExtendSelectionByEuclideanDistance(PointFacade, BranchFacade, Offsets, ExtensionLimit);
		break;

	default:
		break;
	}

	ApplySelectionWeights(SelectionMode, ExtensionLimit, SmoothnessMethod);
}

void FSkeletonPointsSelection::ExtendSelectionByNeighbours(
	const PV::Facades::FPointFacade& PointFacade,
	const PV::Facades::FBranchFacade& BranchFacade,
	const float ExtensionLimit)
{
	int32 CurrentNeighbour = 0;
	const int32 MaxNeighbours = static_cast<int32>(ExtensionLimit);
	while (CurrentNeighbour <= MaxNeighbours)
	{
		for (int32 SelectedPointIndex = 0; SelectedPointIndex < SelectedPoints.Num(); SelectedPointIndex++)
		{
			const FSelectedPoint& SelectedPoint = SelectedPoints[SelectedPointIndex];
			if (SelectedPoint.NeighbourIndex != CurrentNeighbour)
			{
				continue;
			}

			if (SelectedPoint.BranchPointIndex == 0)
			{
				const auto [ParentBranchIndex, ParentBranchPointIndex] = GetParentPointOnParentBranch(PointFacade, BranchFacade, SelectedPoint.BranchIndex);
				if (ParentBranchIndex != INDEX_NONE && ParentBranchPointIndex != INDEX_NONE)
				{
					SelectedPoints.AddUnique({ParentBranchIndex, ParentBranchPointIndex, 0.0f, CurrentNeighbour});
				}
			}
			const TArray<TPair<int32, int32>> ChildPoints = GetChildPointsAtSegment(
				PointFacade, BranchFacade,
				SelectedPoint.BranchIndex,
				SelectedPoint.BranchPointIndex,
				BranchFacade.GetImmediateChildren(SelectedPoint.BranchIndex)
			);
			for (const auto [ChildBranchIndex, ChildBranchPointIndex] : ChildPoints)
			{
				SelectedPoints.AddUnique({ChildBranchIndex, ChildBranchPointIndex, 0.0f, CurrentNeighbour});
			}

			if (CurrentNeighbour < static_cast<int32>(ExtensionLimit))
			{
				const TArray<int32>& BranchPoints = BranchFacade.GetPoints(SelectedPoint.BranchIndex);
				if (BranchPoints.IsValidIndex(SelectedPoint.BranchPointIndex - 1))
				{
					SelectedPoints.AddUnique({SelectedPoint.BranchIndex, SelectedPoint.BranchPointIndex - 1, 0.0f, CurrentNeighbour + 1});
				}
				if (BranchPoints.IsValidIndex(SelectedPoint.BranchPointIndex + 1))
				{
					SelectedPoints.AddUnique({SelectedPoint.BranchIndex, SelectedPoint.BranchPointIndex + 1, 0.0f, CurrentNeighbour + 1});
				}
			}
		}

		++CurrentNeighbour;
	}
}

void FSkeletonPointsSelection::ExtendSelectionByTreeDistance(
	const PV::Facades::FPointFacade& PointFacade,
	const PV::Facades::FBranchFacade& BranchFacade,
	const TArray<FVector3f>& Offsets,
	const float ExtensionLimit)
{
	int32 CurrentNeighbour = 0;
	bool bPointAdded = true;
	while (bPointAdded)
	{
		bPointAdded = false;
		for (int32 SelectedPointIndex = 0; SelectedPointIndex < SelectedPoints.Num(); SelectedPointIndex++)
		{
			const FSelectedPoint& SelectedPoint = SelectedPoints[SelectedPointIndex];
			if (SelectedPoint.NeighbourIndex != CurrentNeighbour)
			{
				continue;
			}

			if (SelectedPoint.BranchPointIndex == 0)
			{
				const auto [ParentBranchIndex, ParentBranchPointIndex] = GetParentPointOnParentBranch(PointFacade, BranchFacade, SelectedPoint.BranchIndex);
				if (ParentBranchIndex != INDEX_NONE && ParentBranchPointIndex != INDEX_NONE)
				{
					SelectedPoints.AddUnique({ParentBranchIndex, ParentBranchPointIndex, SelectedPoint.Weight, CurrentNeighbour});
				}
			}
			const TArray<TPair<int32, int32>> ChildPoints = GetChildPointsAtSegment(
				PointFacade, BranchFacade,
				SelectedPoint.BranchIndex,
				SelectedPoint.BranchPointIndex,
				BranchFacade.GetImmediateChildren(SelectedPoint.BranchIndex)
			);
			for (const auto [ChildBranchIndex, ChildBranchPointIndex] : ChildPoints)
			{
				SelectedPoints.AddUnique({ChildBranchIndex, ChildBranchPointIndex, SelectedPoint.Weight, CurrentNeighbour});
			}

			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(SelectedPoint.BranchIndex);
			const int32 SelectedIndex = BranchPoints[SelectedPoint.BranchPointIndex];
			const int32 SelectedBudNumber = PointFacade.GetBudNumber(SelectedIndex);
			const FVector3f SelectedPointPosition = PointFacade.GetPosition(SelectedIndex) + Offsets[SelectedBudNumber];

			if (BranchPoints.IsValidIndex(SelectedPoint.BranchPointIndex - 1))
			{
				const int32 PointIndex = BranchPoints[SelectedPoint.BranchPointIndex - 1];
				const int32 PointBudNumber = PointFacade.GetBudNumber(PointIndex);
				const FVector3f PointPosition = PointFacade.GetPosition(PointIndex) + Offsets[PointBudNumber];
				const float PointDistance = SelectedPoint.Weight + FVector3f::Distance(SelectedPointPosition, PointPosition);
				if (PointDistance < ExtensionLimit)
				{
					SelectedPoints.AddUnique({
						SelectedPoint.BranchIndex, SelectedPoint.BranchPointIndex - 1, PointDistance, CurrentNeighbour + 1
					});
					bPointAdded = true;
				}
			}
			if (BranchPoints.IsValidIndex(SelectedPoint.BranchPointIndex + 1))
			{
				const int32 PointIndex = BranchPoints[SelectedPoint.BranchPointIndex + 1];
				const int32 PointBudNumber = PointFacade.GetBudNumber(PointIndex);
				const FVector3f PointPosition = PointFacade.GetPosition(PointIndex) + Offsets[PointBudNumber];
				const float PointDistance = SelectedPoint.Weight + FVector3f::Distance(SelectedPointPosition, PointPosition);
				if (PointDistance < ExtensionLimit)
				{
					SelectedPoints.AddUnique({
						SelectedPoint.BranchIndex, SelectedPoint.BranchPointIndex + 1, PointDistance, CurrentNeighbour + 1
					});
					bPointAdded = true;
				}
			}
		}

		++CurrentNeighbour;
	}
}

void FSkeletonPointsSelection::ExtendSelectionByEuclideanDistance(
	const PV::Facades::FPointFacade& PointFacade,
	const PV::Facades::FBranchFacade& BranchFacade,
	const TArray<FVector3f>& Offsets,
	const float ExtensionLimit)
{
	// Get the primary selected point's position
	const TArray<int32>& PrimaryBranchPoints = BranchFacade.GetPoints(PrimarySelectedPoint.BranchIndex);
	const int32 PrimaryPointIndex = PrimaryBranchPoints[PrimarySelectedPoint.BranchPointIndex];
	const int32 PrimaryBudNumber = PointFacade.GetBudNumber(PrimaryPointIndex);
	const FVector3f PrimaryPointPosition = PointFacade.GetPosition(PrimaryPointIndex) + Offsets[PrimaryBudNumber];

	// Loop through all branches and their points
	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); BranchIndex++)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
		for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num(); BranchPointIndex++)
		{
			// Skip the primary selected point as it's already added
			if (BranchIndex == PrimarySelectedPoint.BranchIndex && BranchPointIndex == PrimarySelectedPoint.BranchPointIndex)
			{
				continue;
			}

			const int32 PointIndex = BranchPoints[BranchPointIndex];
			const int32 PointBudNumber = PointFacade.GetBudNumber(PointIndex);
			const FVector3f PointPosition = PointFacade.GetPosition(PointIndex) + Offsets[PointBudNumber];

			// Calculate Euclidean distance from primary selected point
			const float PointDistance = FVector3f::Distance(PrimaryPointPosition, PointPosition);

			if (PointDistance <= ExtensionLimit)
			{
				SelectedPoints.Add({BranchIndex, BranchPointIndex, PointDistance, 0});
			}
		}
	}
}

void FSkeletonPointsSelection::ApplySelectionWeights(
	const ESkeletonSelectionMode SelectionMode,
	const float ExtensionLimit,
	const EPointSelectionSmoothnessMethod SmoothnessMethod
)
{
	switch (SelectionMode)
	{
	case ESkeletonSelectionMode::SelectByNeighbours:
		{
			const float MaxNeighbours = static_cast<int32>(ExtensionLimit) + 1;
			for (FSelectedPoint& SelectedPoint : SelectedPoints)
			{
				SelectedPoint.Weight = GetWeightBySelectedSmoothness(
					(MaxNeighbours - SelectedPoint.NeighbourIndex) / MaxNeighbours,
					SmoothnessMethod
				);
			}
		}
		break;

	case ESkeletonSelectionMode::SelectByTreeDistance:
	case ESkeletonSelectionMode::SelectByEuclideanDistance:
		{
			for (FSelectedPoint& SelectedPoint : SelectedPoints)
			{
				SelectedPoint.Weight = GetWeightBySelectedSmoothness(
					1.0f - (ExtensionLimit != 0.0f ? SelectedPoint.Weight / ExtensionLimit : 0.0f),
					SmoothnessMethod
				);
			}
		}
		break;

	default:
		break;
	}
}

void FSkeletonPointsSelection::ClearSelection()
{
	SelectedPoints.Empty();
	PrimarySelectedPoint = FSelectedPoint();
}

bool FSkeletonPointsSelection::HasSelection() const
{
	return PrimarySelectedPoint.BranchIndex != INDEX_NONE && PrimarySelectedPoint.BranchPointIndex != INDEX_NONE;
}

float FSkeletonPointsSelection::GetWeightBySelectedSmoothness(const float Weight, const EPointSelectionSmoothnessMethod SmoothnessMethod) const
{
	switch (SmoothnessMethod)
	{
	case EPointSelectionSmoothnessMethod::Linear:
		return Weight;

	case EPointSelectionSmoothnessMethod::Smooth:
		return Weight * Weight * (3.0f - 2.0f * Weight);

	case EPointSelectionSmoothnessMethod::Sphere:
		return FMath::Sqrt(1 - FMath::Square(Weight - 1));

	case EPointSelectionSmoothnessMethod::Root:
		return FMath::Sqrt(Weight);

	case EPointSelectionSmoothnessMethod::Sharp:
		return FMath::Square(Weight);

	case EPointSelectionSmoothnessMethod::Sine:
		return FMath::Sin(Weight * UE_HALF_PI);

	case EPointSelectionSmoothnessMethod::Constant:
		return 1.0f;

	default:
		return Weight;
	}
}

uint32 GetTypeHash(const FSkeletonPointsSelection::FSelectedPoint& InPoint)
{
	return static_cast<uint32>(InPoint.BranchIndex ^ InPoint.BranchPointIndex);
}
