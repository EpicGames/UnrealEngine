// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Nodes/PVManualEditSettings.h"

struct FManagedArrayCollection;

namespace PV::Facades
{
	class FPointFacade;
	class FBranchFacade;
}


// Selection state for the skeleton edit tool: a primary "anchor" point plus an extended set
// reachable from the anchor via one of three modes (neighbour count, tree distance, or
// Euclidean distance). Owned exclusively by UPVManualEditTool.
struct FSkeletonPointsSelection
{
	struct FSelectedPoint
	{
		int32 BranchIndex = INDEX_NONE;
		int32 BranchPointIndex = INDEX_NONE;
		float Weight = 0.0f;

		int32 NeighbourIndex = INDEX_NONE;

		bool operator==(const FSelectedPoint& Other) const
		{
			return BranchIndex == Other.BranchIndex && BranchPointIndex == Other.BranchPointIndex;
		}
	};
	
	explicit FSkeletonPointsSelection(const FManagedArrayCollection* const InCollection);

	void SelectPoint(int32 InBranchIndex, int32 InBranchPointIndex);

	void ExtendSelection(
		ESkeletonSelectionMode SelectionMode,
		float ExtensionLimit,
		const TArray<FVector3f>& Offsets,
		EPointSelectionSmoothnessMethod SmoothnessMethod = EPointSelectionSmoothnessMethod::Linear
	);

	void ApplySelectionWeights(
		ESkeletonSelectionMode SelectionMode,
		float ExtensionLimit,
		EPointSelectionSmoothnessMethod SmoothnessMethod
	);

	void ClearSelection();

	bool HasSelection() const;

	int32 GetSelectedBranchIndex() const { return PrimarySelectedPoint.BranchIndex; }
	int32 GetSelectedBranchPointIndex() const { return PrimarySelectedPoint.BranchPointIndex; }

	const FSelectedPoint& GetPrimarySelected() const { return PrimarySelectedPoint; }
	bool HasPrimarySelected() const { return PrimarySelectedPoint.BranchIndex != INDEX_NONE; }

	const TArray<FSelectedPoint>& GetSelectedPoints() const { return SelectedPoints; }
	int32 NumSelectedPoints() const { return SelectedPoints.Num(); }

private:
	// Mode-specific extension helpers, dispatched from ExtendSelection.
	void ExtendSelectionByNeighbours(
		const PV::Facades::FPointFacade& PointFacade,
		const PV::Facades::FBranchFacade& BranchFacade,
		float ExtensionLimit);

	void ExtendSelectionByTreeDistance(
		const PV::Facades::FPointFacade& PointFacade,
		const PV::Facades::FBranchFacade& BranchFacade,
		const TArray<FVector3f>& Offsets,
		float ExtensionLimit);

	void ExtendSelectionByEuclideanDistance(
		const PV::Facades::FPointFacade& PointFacade,
		const PV::Facades::FBranchFacade& BranchFacade,
		const TArray<FVector3f>& Offsets,
		float ExtensionLimit);

	float GetWeightBySelectedSmoothness(
		float Weight,
		EPointSelectionSmoothnessMethod SmoothnessMethod = EPointSelectionSmoothnessMethod::Linear
	) const;

	const FManagedArrayCollection* const Collection = nullptr;

	FSelectedPoint PrimarySelectedPoint;
	TArray<FSelectedPoint> SelectedPoints;
};

uint32 GetTypeHash(const FSkeletonPointsSelection::FSelectedPoint& InPoint);
