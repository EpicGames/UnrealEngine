// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVColliderTransformTool.h"

#include "ContextObjectStore.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "EngineUtils.h"
#include "Selection.h"

#include "BaseBehaviors/SingleClickBehavior.h"

#include "BaseGizmos/TransformGizmoUtil.h"

#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/Viewport.h"

#include "Tools/EdModeInteractiveToolsContext.h"

#define LOCTEXT_NAMESPACE "PVColliderTransformTool"

UPVColliderTransformTool::UPVColliderTransformTool()
{
	SetFlags(RF_Transactional);
}

void UPVColliderTransformTool::Setup()
{
	Super::Setup();

	SetupColliderPreviews();

	USingleClickInputBehavior* Behavior = NewObject<USingleClickInputBehavior>(this);
	Behavior->Initialize(this);
	AddInputBehavior(Behavior);

	TransformProxy = NewObject<UTransformProxy>(this);

	CombinedGizmo = UE::TransformGizmoUtil::Create3AxisTransformGizmo(GetToolManager(), this);
	CombinedGizmo->SetActiveTarget(TransformProxy, GetToolManager());
	CombinedGizmo->SetVisibility(false);
	CombinedGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
	CombinedGizmo->bUseContextGizmoMode = false;
	CombinedGizmo->bUseContextCoordinateSystem = false;

	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UPVColliderTransformTool::OnGizmoTransformBegin);
	TransformProxy->OnTransformChanged.AddUObject(this, &UPVColliderTransformTool::OnGizmoTransformChanged);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UPVColliderTransformTool::OnGizmoTransformEnd);
}

void UPVColliderTransformTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
	
	ClearColliderPreviews();

	if (CombinedGizmo)
	{
		CombinedGizmo->ClearActiveTarget();
		CombinedGizmo->Shutdown();
		CombinedGizmo = nullptr;
	}

	TransformProxy = nullptr;

	Super::Shutdown(ShutdownType);
}

void UPVColliderTransformTool::OnNodeSettingsChanged(UPCGSettings* InNodeSettings, EPCGChangeType InChangeType)
{
	if (InChangeType == EPCGChangeType::Settings)
	{
		ClearColliderPreviews();
		SetupColliderPreviews();

		if (MeshComponents.IsValidIndex(SelectedComponentIndex) && CombinedGizmo && MeshComponents[SelectedComponentIndex])
		{
			CombinedGizmo->SetVisibility(true);
			CombinedGizmo->ReinitializeGizmoTransform(MeshComponents[SelectedComponentIndex]->GetComponentTransform());
		}
		else if (CombinedGizmo)
		{
			SelectedComponentIndex = INDEX_NONE;
			CombinedGizmo->SetVisibility(false);
		}
	}
}

FInputRayHit UPVColliderTransformTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	return FInputRayHit(1.0f);
}

void UPVColliderTransformTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	const HHitProxy* HitProxy = FocusedViewport ? FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y) : nullptr;
	if (!HitProxy || !HitProxy->IsA(HActor::StaticGetType()))
	{
		SelectedComponentIndex = INDEX_NONE;
		CombinedGizmo->SetVisibility(false);
		return;
	}

	const HActor* ActorHitProxy = static_cast<const HActor*>(HitProxy);
	if (ActorHitProxy->Actor == PreviewActor)
	{
		const TObjectPtr<const UPrimitiveComponent> HitComponent = ActorHitProxy->PrimComponent;
		SelectedComponentIndex = MeshComponents.IndexOfByPredicate([&](const auto& Comp)
			{
				return Comp == HitComponent;
			}
		);
		if (ensure(SelectedComponentIndex != INDEX_NONE))
		{
			CombinedGizmo->SetVisibility(true);
			CombinedGizmo->SetNewGizmoTransform(ActorHitProxy->PrimComponent->GetComponentTransform());
		}
		else
		{
			CombinedGizmo->SetVisibility(false);
		}
	}
}

void UPVColliderTransformTool::SetupColliderPreviews()
{
	check(PreviewActor);

	const TObjectPtr<UPVObjectInteractionSettings> ObjectInteractionSettings = GetNodeSettings<UPVObjectInteractionSettings>();

	int32 Index = 0;
	for (const FPVColliderParams& Collider : ObjectInteractionSettings->ObjectInteractionSettings.Colliders)
	{
		if (UStaticMesh* ColliderMesh = Collider.Mesh.LoadSynchronous())
		{
			UStaticMeshComponent* PreviewMeshComponent = NewObject<UStaticMeshComponent>(PreviewActor,
				*("ToolPreviewStaticMesh" + FString::FromInt(Index)), RF_Transient);
			PreviewMeshComponent->SetMobility(EComponentMobility::Movable);
			PreviewMeshComponent->SetupAttachment(PreviewActor->GetRootComponent());
			PreviewMeshComponent->SetStaticMesh(ColliderMesh);
			PreviewMeshComponent->SetWorldTransform(Collider.Transform);
			PreviewMeshComponent->RegisterComponent();

			MeshComponents.Add(PreviewMeshComponent);
		}
		++Index;
	}
}

void UPVColliderTransformTool::ClearColliderPreviews()
{
	for (const TObjectPtr<UStaticMeshComponent>& MeshComponent : MeshComponents)
	{
		MeshComponent->DestroyComponent();
	}
	MeshComponents.Empty();
}

void UPVColliderTransformTool::OnGizmoTransformBegin(UTransformProxy* Proxy)
{
	GEditor->BeginTransaction(NSLOCTEXT("PVColliderTransformTool", "MoveComponent", "Move Component"));
	if (NodeSettings)
	{
		NodeSettings->Modify();
	}
}

void UPVColliderTransformTool::OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform NewTransform)
{
	if (SelectedComponentIndex != INDEX_NONE && MeshComponents.IsValidIndex(SelectedComponentIndex))
	{
		MeshComponents[SelectedComponentIndex]->SetWorldTransform(NewTransform);
		GetNodeSettings<UPVObjectInteractionSettings>()->ObjectInteractionSettings.Colliders[SelectedComponentIndex].Transform = NewTransform;
	}
}

void UPVColliderTransformTool::OnGizmoTransformEnd(UTransformProxy* Proxy)
{
	GEditor->EndTransaction();
	if (NodeSettings)
	{
		NodeSettings->PostEditChange();
	}
}

#undef LOCTEXT_NAMESPACE
