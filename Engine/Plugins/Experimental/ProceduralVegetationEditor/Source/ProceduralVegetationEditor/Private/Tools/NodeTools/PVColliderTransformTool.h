// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/PVBaseInteractiveTool.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "Components/StaticMeshComponent.h"

#include "Nodes/PVObjectInteractionSettings.h"

#include "PVColliderTransformTool.generated.h"

class UTransformProxy;
class UCombinedTransformGizmo;

UCLASS()
class UPVColliderTransformTool
	: public UPVBaseInteractiveTool,
	  public IClickBehaviorTarget
{
	GENERATED_BODY()

	PCG_DECLARE_SUPPORTED_NODES(UPVObjectInteractionSettings)

public:
	UPVColliderTransformTool();

	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

protected:
	virtual void OnNodeSettingsChanged(UPCGSettings* InNodeSettings, EPCGChangeType InChangeType) override;
	
	void SetupColliderPreviews();
	void ClearColliderPreviews();

	void OnGizmoTransformBegin(UTransformProxy* Proxy);
	void OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform NewTransform);
	void OnGizmoTransformEnd(UTransformProxy* Proxy);

private:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UStaticMeshComponent>> MeshComponents;

	int32 SelectedComponentIndex = INDEX_NONE;

	UPROPERTY(Transient)
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UCombinedTransformGizmo> CombinedGizmo = nullptr;
};
