// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVExtractFromImageTool.h"

#include "Nodes/PVExtractFromImageSettings.h"
#include "Implementations/PVImporter_Texture2D.h"
#include "Helpers/PVImportHelpers.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "HitProxies.h"
#include "UnrealClient.h"
#include "InteractiveToolManager.h"
#include "PrimitiveDrawInterface.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "CanvasItem.h"
#include "SceneView.h"

#define LOCTEXT_NAMESPACE "PVTexture2DImportTool"

namespace PVExtractFromImageTool
{
	class HPVRootPointProxy : public HHitProxy
	{
		DECLARE_HIT_PROXY();

	public:
		const int32 PlantIndex = INDEX_NONE;
		const bool bIsRotation = false;

		HPVRootPointProxy(const int32 InPlantIndex, const bool bInIsRotation)
			: HHitProxy(HPP_Wireframe)
			, PlantIndex(InPlantIndex)
			, bIsRotation(bInIsRotation)
		{
		}

		virtual EMouseCursor::Type GetMouseCursor() override
		{
			return EMouseCursor::GrabHand;
		}
	};

	IMPLEMENT_HIT_PROXY(HPVRootPointProxy, HHitProxy)
}

UPVExtractFromImageTool::UPVExtractFromImageTool()
{
	SetFlags(RF_Transactional);
}

void UPVExtractFromImageTool::Setup()
{
	Super::Setup();

	UClickDragInputBehavior* DragBehavior = NewObject<UClickDragInputBehavior>(this);
	DragBehavior->Initialize(this);
	AddInputBehavior(DragBehavior);
}

void UPVExtractFromImageTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace PVExtractFromImageTool;

	const TObjectPtr<UPVExtractFromImageSettings> Texture2DImporterSettings = GetNodeSettings<UPVExtractFromImageSettings>();
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

	const FPVImportTexture2DParams& ImportTexture2DParams = Texture2DImporterSettings->Params;

	const float GizmoScale = 10.f;
	for (int32 PlantIndex = 0; PlantIndex < ImportTexture2DParams.PlantSettings.Num(); ++PlantIndex)
	{
		const FPVImportTexture2DPlantSettings& PlantSettings = ImportTexture2DParams.PlantSettings[PlantIndex];

		const FMatrix44f UVToWorld = PVTexture2DImporterVisualization::GetUVToWorldMatrix();
		const FVector RootPointPosition = FVector(UVToWorld.TransformPosition(FVector3f(PlantSettings.RootPosition, 0.f)));

		PDI->SetHitProxy(new HPVRootPointProxy(PlantIndex, false));
		PDI->DrawPoint(RootPointPosition, FColor::Blue, 1.25f * GizmoScale, SDPG_Foreground);
		PDI->SetHitProxy(nullptr);

		const float RotationRad = FMath::DegreesToRadians(PlantSettings.Rotation);
		const FVector RotationDir(0.f, -FMath::Sin(RotationRad), FMath::Cos(RotationRad));
		const FVector RotationGizmoEndPoint = RootPointPosition + RotationDir * GizmoScale;

		PDI->DrawLine(RootPointPosition, RotationGizmoEndPoint, FColor::Orange, SDPG_Foreground, 0.05f * GizmoScale);
		PDI->SetHitProxy(new HPVRootPointProxy(PlantIndex, true));
		PDI->DrawPoint(RotationGizmoEndPoint, FColor::Orange, 1.1f * GizmoScale, SDPG_Foreground);
		PDI->SetHitProxy(nullptr);
	}
}

void UPVExtractFromImageTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	const FSceneView* SceneView = RenderAPI->GetSceneView();
	if (!Canvas || !SceneView)
	{
		return;
	}

	const TObjectPtr<UPVExtractFromImageSettings> Texture2DImporterSettings = GetNodeSettings<UPVExtractFromImageSettings>();
	const FPVImportTexture2DParams& ImportTexture2DParams = Texture2DImporterSettings->Params;
	const float DPIScale = Canvas->GetDPIScale();

	for (int32 PlantIndex = 0; PlantIndex < ImportTexture2DParams.PlantSettings.Num(); ++PlantIndex)
	{
		const FPVImportTexture2DPlantSettings& PlantSettings = ImportTexture2DParams.PlantSettings[PlantIndex];

		const FMatrix44f UVToWorld = PVTexture2DImporterVisualization::GetUVToWorldMatrix();
		const FVector RootPointPosition = FVector(UVToWorld.TransformPosition(FVector3f(PlantSettings.RootPosition, 0.f)));

		FVector2D PixelLocation;
		if (SceneView->WorldToPixel(RootPointPosition, PixelLocation))
		{
			FCanvasTextItem TextItem(
				PixelLocation / DPIScale,
				FText::AsNumber(PlantIndex),
				GEngine->GetLargeFont(),
				FLinearColor::White
			);
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.Scale = FVector2D(2.0);
			TextItem.Draw(Canvas);
		}
	}
}

FInputRayHit UPVExtractFromImageTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	using namespace PVExtractFromImageTool;

	if (FViewport* Viewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport())
	{
		const HHitProxy* HitProxy = Viewport->GetHitProxy(PressPos.ScreenPosition.X, PressPos.ScreenPosition.Y);
		if (HitProxy && HitProxy->IsA(HPVRootPointProxy::StaticGetType()))
		{
			return FInputRayHit(1.f);
		}
	}
	return FInputRayHit();
}

void UPVExtractFromImageTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	using namespace PVExtractFromImageTool;

	FViewport* Viewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!Viewport)
	{
		return;
	}

	const HHitProxy* HitProxy = Viewport->GetHitProxy(PressPos.ScreenPosition.X, PressPos.ScreenPosition.Y);
	if (HitProxy && HitProxy->IsA(HPVRootPointProxy::StaticGetType()))
	{
		const HPVRootPointProxy* RootPointProxy = static_cast<const HPVRootPointProxy*>(HitProxy);
		DraggedPlantIndex = RootPointProxy->PlantIndex;
		bDraggingRotation = RootPointProxy->bIsRotation;
		GetToolManager()->BeginUndoTransaction(bDraggingRotation ? LOCTEXT("RotatePlant", "Rotate Plant") : LOCTEXT("MoveRootPoint", "Move Root Point"));
		GetNodeSettings<UPVExtractFromImageSettings>()->Modify();
	}
}

void UPVExtractFromImageTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	using namespace PVExtractFromImageTool;

	const TObjectPtr<UPVExtractFromImageSettings> Settings = GetNodeSettings<UPVExtractFromImageSettings>();

	if (DraggedPlantIndex == INDEX_NONE || !Settings->Params.PlantSettings.IsValidIndex(DraggedPlantIndex))
	{
		return;
	}

	// All root points lie in the plane at X=0 in world space (UVToVec3 always produces X=0).
	// Intersect the drag ray with that plane to find where the cursor is in 3D.
	const FRay& WorldRay = DragPos.WorldRay;
	if (FMath::IsNearlyZero(WorldRay.Direction.X))
	{
		return;
	}

	const float T = -WorldRay.Origin.X / WorldRay.Direction.X;
	if (T < 0.f)
	{
		return;
	}

	const FVector3f HitPos3f = FVector3f(WorldRay.PointAt(T));
	const FVector3f UVPos = PVTexture2DImporterVisualization::GetWorldToUVMatrix().TransformPosition(HitPos3f);
	const FVector2f HitPointUV(UVPos.X, UVPos.Y);

	FPVImportTexture2DPlantSettings& PlantSettings = Settings->Params.PlantSettings[DraggedPlantIndex];

	if (bDraggingRotation)
	{
		const FVector2f Offset = PlantSettings.RootPosition - HitPointUV;
		PlantSettings.Rotation = FMath::RadiansToDegrees(FMath::Atan2(Offset.X, Offset.Y));
	}
	else
	{
		PlantSettings.RootPosition = FVector2f(
			FMath::Clamp(HitPointUV.X, 0.f, 1.f),
			FMath::Clamp(HitPointUV.Y, 0.f, 1.f)
		);
	}
}

void UPVExtractFromImageTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	EndClickDragBehavior();
}

void UPVExtractFromImageTool::OnTerminateDragSequence()
{
	EndClickDragBehavior();
}

void UPVExtractFromImageTool::EndClickDragBehavior()
{
	if (DraggedPlantIndex != INDEX_NONE)
	{
		GetNodeSettings()->PostEditChange();
		GetToolManager()->EndUndoTransaction();
		DraggedPlantIndex = INDEX_NONE;
		bDraggingRotation = false;
	}
}

#undef LOCTEXT_NAMESPACE
