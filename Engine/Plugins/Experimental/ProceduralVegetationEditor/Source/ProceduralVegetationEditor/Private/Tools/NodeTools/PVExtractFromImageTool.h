// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/PVBaseInteractiveTool.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Nodes/PVExtractFromImageSettings.h"
#include "Params/PVImportTexture2DParams.h"
#include "PVExtractFromImageTool.generated.h"

class UTransformProxy;
class IToolsContextRenderAPI;

UCLASS()
class UPVExtractFromImageTool : public UPVBaseInteractiveTool, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

	PCG_DECLARE_SUPPORTED_NODES(UPVExtractFromImageSettings)

public:
	UPVExtractFromImageTool();

	// Begin UInteractiveTool interface
	virtual void Setup() override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	// End UInteractiveTool interface

	// Begin IClickDragBehaviorTarget interface
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;
	// End IClickDragBehaviorTarget interface

private:
	void EndClickDragBehavior();

private:
	int32 DraggedPlantIndex = INDEX_NONE;
	bool bDraggingRotation = false;
};
