// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVManualEditTool.h"

#include "EditorCategoryUtils.h"
#include "EditorModeManager.h"
#include "EngineUtils.h"
#include "PreviewMesh.h"
#include "PrimitiveDrawingUtils.h"
#include "PVEditorCommon.h"
#include "ScopedTransaction.h"

#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseBehaviors/SingleClickBehavior.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Components/Viewport.h"

#include "DynamicMesh/DynamicMesh3.h"

#include "Generators/SphereGenerator.h"

#include "Materials/MaterialInterface.h"

#include "Misc/TransactionObjectEvent.h"

#include "Nodes/PVManualEditSettings.h"

#include "Subsystems/IPCGBaseSubsystem.h"

#include "Tools/EdModeInteractiveToolsContext.h"
#include "Tools/Helpers/PVManualEditToolHelpers.h"

#include "Visualizations/PVSkeletonVisualizerComponent.h"


UPVManualEditTool::UPVManualEditTool()
{
	SetFlags(RF_Transactional);
}

void UPVManualEditTool::Setup()
{
	Super::Setup();

	if (!HasCollection())
	{
		RequestShutdown(EToolShutdownType::Cancel);
		return;
	}

	if (!Attributes.InitializeFromCollection(GetCollection()))
	{
		RequestShutdown(EToolShutdownType::Cancel);
		return;
	}

	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>(this);
	ClickBehavior->Initialize(this);
	AddInputBehavior(ClickBehavior);

	UMouseHoverBehavior* MouseHoverBehavior = NewObject<UMouseHoverBehavior>(this);
	MouseHoverBehavior->Initialize(this);
	AddInputBehavior(MouseHoverBehavior);

	TransformProxy = NewObject<UTransformProxy>(this);

	CombinedGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(), ETransformGizmoSubElements::StandardTranslateRotate, this);
	CombinedGizmo->SetActiveTarget(TransformProxy, GetToolManager());
	CombinedGizmo->SetVisibility(false);
	CombinedGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
	CombinedGizmo->bUseContextGizmoMode = false;
	CombinedGizmo->bUseContextCoordinateSystem = false;
	for (UActorComponent* Component : CombinedGizmo->GetGizmoActor()->GetComponents())
	{
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
		{
			PrimitiveComponent->HitProxyPriority = HPP_Foreground;
		}
	}

	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UPVManualEditTool::OnGizmoTransformBegin);
	TransformProxy->OnTransformChanged.AddUObject(this, &UPVManualEditTool::OnGizmoTransformChanged);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UPVManualEditTool::OnGizmoTransformEnd);

	Selection = MakeUnique<FSkeletonPointsSelection>(&GetCollection());

	PreviewComponent = NewObject<UPVSkeletonVisualizerComponent>(PreviewActor, TEXT("PVVisualizerComponent"), RF_Transient);
	PreviewComponent->SetupAttachment(PreviewActor->GetRootComponent());
	PreviewComponent->RegisterComponent();
	PreviewComponent->GetDynamicMeshComponent()->RegisterComponentWithWorld(TargetWorld);
	PreviewComponent->GetPointMeshInstancerComponent()->RegisterComponentWithWorld(TargetWorld);
	PreviewComponent->SetUseMeshPreview(true);
	
	InitializeToolData();
}

void UPVManualEditTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (bIsTransforming || !CombinedGizmo || !CombinedGizmo->IsVisible())
	{
		return;
	}

	bHoveringRotationGizmo = false;

	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!FocusedViewport)
	{
		return;
	}

	HHitProxy* HitProxy = FocusedViewport->GetHitProxy(FocusedViewport->GetMouseX(), FocusedViewport->GetMouseY());
	if (!HitProxy)
	{
		return;
	}

	if (const HActor* ActorProxy = HitProxyCast<HActor>(HitProxy))
	{
		if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ActorProxy->PrimComponent))
		{
			if (const UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh())
			{
				if (Mesh->GetName().Contains("GizmoQuarterCircleHandle"))
				{
					bHoveringRotationGizmo = true;
				}
			}
		}
	}
}

void UPVManualEditTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FPrimitiveDrawInterface* PDI = RenderAPI ? RenderAPI->GetPrimitiveDrawInterface() : nullptr;
	if (!PDI || !HasCollection())
	{
		return;
	}

	const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();
	const FPVSkeletonSelectionParams& SelectionParams = ManualEditSettings->ManualEditSettings.SkeletonSelectionParams;

	const int32 SelectedBranchIndex = Selection->GetSelectedBranchIndex();
	const int32 SelectedBranchPointIndex = Selection->GetSelectedBranchPointIndex();

	RenderSelectionHighlights(PDI, ManualEditSettings, SelectedBranchIndex, SelectedBranchPointIndex);
	RenderHoverHighlight(PDI);
	if (SelectionParams.SkeletonSelectionMode == ESkeletonSelectionMode::SelectByEuclideanDistance)
	{
		RenderInfluenceRadius(PDI, SelectionParams.SelectionDistance);
	}
}

void UPVManualEditTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
	
	if (CombinedGizmo)
	{
		CombinedGizmo->ClearActiveTarget();
		CombinedGizmo->Shutdown();
		CombinedGizmo = nullptr;
	}

	TransformProxy = nullptr;

	Super::Shutdown(ShutdownType);
}

FInputRayHit UPVManualEditTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	if (FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport())
	{
		if (FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y))
		{
			return FInputRayHit(1.0f);
		}
	}

	Selection->ClearSelection();
	CombinedGizmo->SetVisibility(false);
	return FInputRayHit();
}

void UPVManualEditTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!FocusedViewport)
	{
		return;
	}

	HHitProxy* HitProxy = FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y);
	if (!HitProxy)
	{
		return;
	}

	int32 HitBranchIndex = INDEX_NONE;
	int32 HitBranchPointIndex = INDEX_NONE;

	if (const int32 InstanceIndex = PV::Tools::GetPointIndexFromHitProxy(HitProxy); InstanceIndex != INDEX_NONE)
	{
		PreviewComponent->GetPointDataFromInstanceIndex(InstanceIndex, HitBranchIndex, HitBranchPointIndex);
	}

	if (const UPVSkeletonVisualizerComponent* SkeletonVisualizerComponent = PV::Tools::GetVisualizerComponentFromHitProxy(HitProxy))
	{
		if (!PV::Tools::FindBranchSelectionFromRay(SkeletonVisualizerComponent, ClickPos.WorldRay, HitBranchIndex, HitBranchPointIndex))
		{
			PreviousRotation = FQuat4f::Identity;
			Selection->ClearSelection();
			CombinedGizmo->SetVisibility(false);
			return;
		}
	}
	
	if (HitBranchIndex == INDEX_NONE || HitBranchPointIndex == INDEX_NONE)
	{
		return;
	}

	const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();
	if (ManualEditSettings->ManualEditSettings.ManualEditMode == EManualEditMode::TRS)
	{
		ApplySelectionAtClick(HitBranchIndex, HitBranchPointIndex);
	}
	else if (ManualEditSettings->ManualEditSettings.ManualEditMode == EManualEditMode::BranchRemoval)
	{
		ApplyBranchRemovalAtClick(HitBranchIndex, HitBranchPointIndex);
	}
}

FInputRayHit UPVManualEditTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	static const FInputRayHit NullHit;

	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!FocusedViewport)
	{
		return NullHit;
	}

	HHitProxy* HitProxy = FocusedViewport->GetHitProxy(PressPos.ScreenPosition.X, PressPos.ScreenPosition.Y);
	if (!HitProxy)
	{
		return NullHit;
	}

	return HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()) || PV::Tools::GetVisualizerComponentFromHitProxy(HitProxy)
		? FInputRayHit(1.0f)
		: NullHit;
}

void UPVManualEditTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{}

bool UPVManualEditTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// Reset every frame so an off-target sample clears the highlight even if the viewport / hit-proxy checks pass.
	HoveredBranchIndex = INDEX_NONE;
	HoveredBranchPointIndex = INDEX_NONE;
	HoveredType = PV::Tools::ESkeletonHoverType::None;

	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!FocusedViewport)
	{
		return false;
	}

	HHitProxy* HitProxy = FocusedViewport->GetHitProxy(DevicePos.ScreenPosition.X, DevicePos.ScreenPosition.Y);
	if (!HitProxy)
	{
		return false;
	}

	if (const int32 InstanceIndex = PV::Tools::GetPointIndexFromHitProxy(HitProxy); InstanceIndex != INDEX_NONE)
	{
		if (PreviewComponent->GetPointDataFromInstanceIndex(InstanceIndex, HoveredBranchIndex, HoveredBranchPointIndex))
		{
			HoveredType = PV::Tools::ESkeletonHoverType::Point;
			return true;
		}
	}

	if (HoveredBranchIndex == INDEX_NONE || HoveredBranchPointIndex == INDEX_NONE)
	{
		if (const UPVSkeletonVisualizerComponent* SkeletonVisualizerComponent = PV::Tools::GetVisualizerComponentFromHitProxy(HitProxy))
		{
			if (PV::Tools::FindBranchSelectionFromRay(SkeletonVisualizerComponent, DevicePos.WorldRay, HoveredBranchIndex, HoveredBranchPointIndex))
			{
				HoveredType = PV::Tools::ESkeletonHoverType::Edge;
				return true;
			}
		}
	}

	return false;
}

void UPVManualEditTool::OnEndHover()
{
	HoveredBranchIndex = INDEX_NONE;
	HoveredBranchPointIndex = INDEX_NONE;
	HoveredType = PV::Tools::ESkeletonHoverType::None;
}

bool UPVManualEditTool::MatchesContext(const FTransactionContext& InContext,
                                         const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	if (!NodeToolContext)
	{
		return false;
	}

	for (const auto& [Object, Context] : TransactionObjectContexts)
	{
		if (Object == NodeSettings)
		{
			return true;
		}
	}

	return false;
}

void UPVManualEditTool::PostUndo(bool bSuccess)
{
	if (!bSuccess)
	{
		return;
	}

	GetNodeSettings()->PostEditChange();
	PreviewComponent->RebuildSkeleton();
}

void UPVManualEditTool::OnNodeSettingsChanged(UPCGSettings* InNodeSettings, EPCGChangeType InChangeType)
{
	if (InChangeType == EPCGChangeType::Settings)
	{
		if (Selection->HasSelection())
		{
			const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();
			const FPVSkeletonSelectionParams& SelectionParams = ManualEditSettings->ManualEditSettings.SkeletonSelectionParams;
			TArray<FVector3f>& Offsets = ManualEditSettings->ManualEditSettings.RelativeOffsets;

			Selection->ExtendSelection(
				SelectionParams.SkeletonSelectionMode,
				SelectionParams.SkeletonSelectionMode == ESkeletonSelectionMode::SelectByNeighbours
				? static_cast<float>(SelectionParams.NumPointsSelection)
				: SelectionParams.SelectionDistance,
				Offsets,
				SelectionParams.SelectionFalloff
			);

			const int32 SelectedPointIndex = Attributes.BranchPoints[Selection->GetSelectedBranchIndex()][Selection->GetSelectedBranchPointIndex()];
			const int32 SelectedPointBudBumber = Attributes.PointBudNumber[SelectedPointIndex];
			const FVector Position = FVector(Attributes.PointPosition[SelectedPointIndex] + Offsets[SelectedPointBudBumber]);
			CombinedGizmo->SetVisibility(true);
			CombinedGizmo->ReinitializeGizmoTransform(FTransform(Position));
		}
		else
		{
			CombinedGizmo->SetVisibility(false);
		}
	}
	else if (InChangeType == EPCGChangeType::ExternalModification)
	{
		PreviewComponent->RebuildSkeleton();
	}
}

void UPVManualEditTool::InitializeToolData()
{
	const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();

	TArray<FVector3f>& Offsets = ManualEditSettings->ManualEditSettings.RelativeOffsets;
	TArray<bool>& RemovedPoints = ManualEditSettings->ManualEditSettings.RemovedPoints;

	if (const int32* MaxBudNumber = Algo::MaxElement(Attributes.PointBudNumber))
	{
		if (Offsets.Num() < (*MaxBudNumber + 1))
		{
			Offsets.SetNumZeroed(*MaxBudNumber + 1);
		}
		if (RemovedPoints.Num() < (*MaxBudNumber + 1))
		{
			RemovedPoints.SetNumZeroed(*MaxBudNumber + 1);
		}
	}

	PreviewComponent->OnDrawPoint.BindLambda(
		[Settings = GetNodeSettings<UPVManualEditSettings>()](const int32 PointBudNumber, FVector& PointLocation)
			{
				const bool bIsRemoved = Settings->ManualEditSettings.RemovedPoints[PointBudNumber];
				if (!bIsRemoved)
				{
					PointLocation += FVector(Settings->ManualEditSettings.RelativeOffsets[PointBudNumber]);
				}
				return !bIsRemoved;
			}
	);
	PreviewComponent->SetCollection(&GetCollection());
	PreviewComponent->SetBuildOctree(true);
	PreviewComponent->SetUseMeshPreview(true);
}

void UPVManualEditTool::OnGizmoTransformBegin(UTransformProxy* InTransformProxy)
{
	GEditor->BeginTransaction(NSLOCTEXT("PVManualEditTool", "MoveTransform", "Move Transform"));
	bIsTransforming = true;
	NodeSettings->Modify();
}

void UPVManualEditTool::OnGizmoTransformChanged(UTransformProxy* InTransformProxy, FTransform InTransform)
{
	if (Selection->HasSelection())
	{
		const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();
		TArray<FVector3f>& Offsets = ManualEditSettings->ManualEditSettings.RelativeOffsets;

		const int32 PrimarySelectedBranch = Selection->GetSelectedBranchIndex();
		const int32 PrimarySelectedBranchPointIndex = Selection->GetSelectedBranchPointIndex();

		const TArray<int32>& PrimaryBranchPoints = Attributes.BranchPoints[PrimarySelectedBranch];
		const int32 PrimarySelectedPointIndex = PrimaryBranchPoints[PrimarySelectedBranchPointIndex];
		const int32 PrimaryPointBudNumber = Attributes.PointBudNumber[PrimarySelectedPointIndex];

		const FVector3f OriginalPosition = Attributes.PointPosition[PrimarySelectedPointIndex] + Offsets[PrimaryPointBudNumber];
		const FVector3f NewPosition = FVector3f(InTransform.GetLocation());
		const FVector3f Offset = NewPosition - OriginalPosition;
		if (!Offset.IsZero())
		{
			for (const FSkeletonPointsSelection::FSelectedPoint& SelectedPoint : Selection->GetSelectedPoints())
			{
				// Skip the root point of a branch (index 0) as it's shared with the parent branch's point
				// and will be transformed via the parent branch
				const bool bIsTrunk = Attributes.BranchParents[SelectedPoint.BranchIndex] == 0;
				if (!bIsTrunk && SelectedPoint.BranchPointIndex == 0)
				{
					continue;
				}

				const TArray<int32>& BranchPoints = Attributes.BranchPoints[SelectedPoint.BranchIndex];
				const int32 SelectedPointIndex = BranchPoints[SelectedPoint.BranchPointIndex];
				const int32 SelectedPointBudNumber = Attributes.PointBudNumber[SelectedPointIndex];
				Offsets[SelectedPointBudNumber] += Offset * SelectedPoint.Weight;
			}
		}

		const FQuat4f CurrentRotation = FQuat4f(InTransform.GetRotation());
		const FQuat4f DeltaRotation = CurrentRotation * PreviousRotation.Inverse();
		if (!DeltaRotation.IsIdentity())
		{
			for (int32 BranchPointIndex = PrimarySelectedBranchPointIndex + 1; BranchPointIndex < PrimaryBranchPoints.Num(); BranchPointIndex++)
			{
				const int32 PointIndex = PrimaryBranchPoints[BranchPointIndex];
				const int32 PointBudNumber = Attributes.PointBudNumber[PointIndex];
				const FVector3f PointPosition = Attributes.PointPosition[PointIndex] + Offsets[PointBudNumber];
				const FVector3f UpdatedPosition = OriginalPosition + DeltaRotation.RotateVector(PointPosition - OriginalPosition);
				const FVector3f PointOffset = UpdatedPosition - PointPosition;
				Offsets[PointBudNumber] += PointOffset;
			}

			const float PointLFR = Attributes.PointLengthFromRoot[PrimarySelectedPointIndex];

			for (int32 BranchChildNumber : PV::Tools::GetBranchImmediateChildren(Attributes, PrimarySelectedBranch))
			{
				if (const int32 BranchIndex = Attributes.BranchNumber.Find(BranchChildNumber); BranchIndex != INDEX_NONE)
				{
					const TArray<int32>& ChildBranchPoints = Attributes.BranchPoints[BranchIndex];
					if (Attributes.PointLengthFromRoot[ChildBranchPoints[0]] > PointLFR)
					{
						// Start from index 1 to skip the root point (index 0) of the child branch,
						// as it's shared with the parent branch's point
						for (int32 ChildBranchPointIndexInArray = 1; ChildBranchPointIndexInArray < ChildBranchPoints.Num();
						     ChildBranchPointIndexInArray++)
						{
							const int32 ChildBranchPointIndex = ChildBranchPoints[ChildBranchPointIndexInArray];
							const int32 ChildPointBudNumber = Attributes.PointBudNumber[ChildBranchPointIndex];
							const FVector3f PointPosition = Attributes.PointPosition[ChildBranchPointIndex] + Offsets[ChildPointBudNumber];
							const FVector3f UpdatedPosition = OriginalPosition + DeltaRotation.RotateVector(PointPosition - OriginalPosition);
							const FVector3f PointOffset = UpdatedPosition - PointPosition;
							Offsets[ChildPointBudNumber] += PointOffset;
						}

						for (int32 SubBranchChildNumber : Attributes.BranchChildren[BranchIndex])
						{
							if (const int32 SubBranchIndex = Attributes.BranchNumber.Find(SubBranchChildNumber); SubBranchIndex != INDEX_NONE)
							{
								const TArray<int32>& SubBranchPoints = Attributes.BranchPoints[SubBranchIndex];
								// Start from index 1 to skip the root point (index 0) of the sub-branch,
								// as it's shared with the parent branch's point
								for (int32 SubBranchPointIndexInArray = 1; SubBranchPointIndexInArray < SubBranchPoints.Num();
								     SubBranchPointIndexInArray++)
								{
									const int32 ChildSubBranchPointIndex = SubBranchPoints[SubBranchPointIndexInArray];
									const int32 SubPointBudNumber = Attributes.PointBudNumber[ChildSubBranchPointIndex];
									const FVector3f PointPosition = Attributes.PointPosition[ChildSubBranchPointIndex] + Offsets[SubPointBudNumber];
									const FVector3f UpdatedPosition = OriginalPosition + DeltaRotation.RotateVector(PointPosition - OriginalPosition);
									const FVector3f PointOffset = UpdatedPosition - PointPosition;
									Offsets[SubPointBudNumber] += PointOffset;
								}
							}
						}
					}
				}
			}
		}

		PreviousRotation = CurrentRotation;
	}
}

void UPVManualEditTool::OnGizmoTransformEnd(UTransformProxy* InTransformProxy)
{
	bIsTransforming = false;
	GEditor->EndTransaction();
	GetNodeSettings()->PostEditChange();
	PreviewComponent->RebuildSkeleton();
}

void UPVManualEditTool::RenderSelectionHighlights(
	FPrimitiveDrawInterface* PDI,
	const UPVManualEditSettings* ManualEditSettings,
	const int32 SelectedBranchIndex,
	const int32 SelectedBranchPointIndex
)
{
	if (!Selection || !Selection->HasSelection())
	{
		return;
	}

	const TSet SelectedSet(Selection->GetSelectedPoints());
	const TArray<bool>& RemovedPoints = ManualEditSettings->ManualEditSettings.RemovedPoints;

	const int32 NumBranches = Attributes.BranchPoints.Num();
	for (int32 BranchIndex = 0; BranchIndex < NumBranches; ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = Attributes.BranchPoints[BranchIndex];

		FVector PrevPos = FVector::ZeroVector;
		float PrevScale = 0.0f;
		bool bHasPrev = false;
		bool bPrevSelected = false;

		for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
		{
			const int32 PointIndex = BranchPoints[BranchPointIndex];
			if (
				PV::Tools::IsPointRemoved(Attributes.PointBudNumber, RemovedPoints, PointIndex) ||
				(
					BranchPointIndex == 0 &&
					BranchPoints.IsValidIndex(1) &&
					PV::Tools::IsPointRemoved(Attributes.PointBudNumber, RemovedPoints, BranchPoints[1])
				)
			)
			{
				break;
			}

			FVector CurPos;
			float CurScale = 0.0f;
			if (!GetPointWorldPositionAndScale(BranchIndex, BranchPointIndex, CurPos, CurScale))
			{
				bHasPrev = false;
				continue;
			}

			CurScale = FMath::Max(CurScale, PV::EditorCommon::PointMinScale);

			bool bPointSelected;
			if (bHoveringRotationGizmo)
			{
				bPointSelected = SelectedBranchIndex == BranchIndex && SelectedBranchPointIndex < BranchPointIndex;
			}
			else
			{
				bPointSelected = SelectedSet.Contains({BranchIndex, BranchPointIndex});
			}

			if (bPointSelected)
			{
				PV::Tools::DrawPointHighlight(PDI, CurPos, CurScale, PV::EditorCommon::SelectedHighlightColor);
			}
			if (bHasPrev && (bPointSelected || bPrevSelected))
			{
				PV::Tools::DrawEdgeHighlight(PDI, PrevPos, CurPos, PrevScale, CurScale, PV::EditorCommon::SelectedHighlightColor);
			}

			PrevPos = CurPos;
			PrevScale = CurScale;
			bPrevSelected = bPointSelected;
			bHasPrev = true;
		}
	}
}

void UPVManualEditTool::RenderHoverHighlight(FPrimitiveDrawInterface* PDI)
{
	using namespace PV::EditorCommon;

	// Hover highlight pass last so the cursor cue stays on top of any selection highlight.
	switch (HoveredType)
	{
	case PV::Tools::ESkeletonHoverType::Point:
		{
			FVector Pos;
			float Scale = 0.0f;
			if (GetPointWorldPositionAndScale(HoveredBranchIndex, HoveredBranchPointIndex, Pos, Scale))
			{
				PV::Tools::DrawPointHighlight(PDI, Pos, FMath::Max(Scale, PV::EditorCommon::PointMinScale), HoverHighlightColor);
			}
		}
		break;

	case PV::Tools::ESkeletonHoverType::Edge:
		{
			FVector PosA, PosB;
			float ScaleA, ScaleB;
			if (
				GetPointWorldPositionAndScale(HoveredBranchIndex, HoveredBranchPointIndex, PosA, ScaleA) &&
				GetPointWorldPositionAndScale(HoveredBranchIndex, HoveredBranchPointIndex + 1, PosB, ScaleB)
			)
			{
				PV::Tools::DrawEdgeHighlight(
					PDI,
					PosA, PosB,
					FMath::Max(ScaleA, PV::EditorCommon::PointMinScale), FMath::Max(ScaleB, PV::EditorCommon::PointMinScale),
					HoverHighlightColor
				);
			}
		}
		break;

	case PV::Tools::ESkeletonHoverType::None:
	default:
		break;
	}
}

void UPVManualEditTool::RenderInfluenceRadius(FPrimitiveDrawInterface* PDI, const float Radius)
{
	using namespace PV::EditorCommon;

	if (!CombinedGizmo || !CombinedGizmo->IsVisible())
	{
		return;
	}
	
	const FTransform GizmoTransform = TransformProxy->GetTransform();

	FTransform Transform = FTransform(FRotator(), GizmoTransform.GetLocation());
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::X), Transform.GetScaledAxis(EAxis::Y), InfluenceRadiusColor, Radius,
		InfluenceRadiusNumSides, 0, HighlightLineThickness, 0.001f, false);
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::X), Transform.GetScaledAxis(EAxis::Z), InfluenceRadiusColor, Radius,
		InfluenceRadiusNumSides, 0, HighlightLineThickness, 0.001f, false);
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::Y), Transform.GetScaledAxis(EAxis::Z), InfluenceRadiusColor, Radius,
		InfluenceRadiusNumSides, 0, HighlightLineThickness, 0.001f, false);

	Transform = FTransform(FRotator(0, 45, 0), GizmoTransform.GetLocation());
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::X), Transform.GetScaledAxis(EAxis::Z), InfluenceRadiusColor, Radius,
		InfluenceRadiusNumSides, 0, HighlightLineThickness, 0.001f, false);
	DrawCircle(PDI, Transform.GetLocation(), Transform.GetScaledAxis(EAxis::Y), Transform.GetScaledAxis(EAxis::Z), InfluenceRadiusColor, Radius,
		InfluenceRadiusNumSides, 0, HighlightLineThickness, 0.001f, false);
}


void UPVManualEditTool::ApplySelectionAtClick(const int32 HitBranchIndex, const int32 HitBranchPointIndex)
{
	const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();
	const FPVSkeletonSelectionParams& SelectionParams = ManualEditSettings->ManualEditSettings.SkeletonSelectionParams;

	Selection->SelectPoint(HitBranchIndex, HitBranchPointIndex);

	Selection->ExtendSelection(
		SelectionParams.SkeletonSelectionMode,
		SelectionParams.SkeletonSelectionMode == ESkeletonSelectionMode::SelectByNeighbours
		? static_cast<float>(SelectionParams.NumPointsSelection)
		: SelectionParams.SelectionDistance,
		ManualEditSettings->ManualEditSettings.RelativeOffsets,
		SelectionParams.SelectionFalloff
	);

	TArray<FVector3f>& Offsets = ManualEditSettings->ManualEditSettings.RelativeOffsets;

	const int32 SelectedPointIndex = Attributes.BranchPoints[HitBranchIndex][HitBranchPointIndex];
	const int32 SelectedPointBudNumber = Attributes.PointBudNumber[SelectedPointIndex];
	const FVector Position = FVector(Attributes.PointPosition[SelectedPointIndex] + Offsets[SelectedPointBudNumber]);
	PreviousRotation = FQuat4f::Identity;
	CombinedGizmo->SetVisibility(true);
	CombinedGizmo->ReinitializeGizmoTransform(FTransform(Position));
}

void UPVManualEditTool::ApplyBranchRemovalAtClick(int32 HitBranchIndex, int32 HitBranchPointIndex)
{
	const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();

	PreviousRotation = FQuat4f::Identity;
	Selection->ClearSelection();
	CombinedGizmo->SetVisibility(false);
	
	FScopedTransaction RemovalTransaction(NSLOCTEXT("PVManualEditTool", "RemovalTransaction", "Branch Removal"));
	NodeSettings->Modify();

	TArray<bool>& RemovedPoints = ManualEditSettings->ManualEditSettings.RemovedPoints;

	const TArray<int32>& BranchPoints = Attributes.BranchPoints[HitBranchIndex];

	int32 StartIndex =
		ManualEditSettings->ManualEditSettings.BranchRemovalParams.BranchRemovalMode == EBranchRemovalMode::TrimBranch
		? HitBranchPointIndex
		: 0;

	bool bNextPointRemoved = false;
	if (BranchPoints.IsValidIndex(StartIndex + 1))
	{
		const int32 NextPointBudNumber = Attributes.PointBudNumber[BranchPoints[StartIndex + 1]];
		bNextPointRemoved = RemovedPoints[NextPointBudNumber];
	}
	if (StartIndex > 0 && StartIndex < BranchPoints.Num() - 1 && !bNextPointRemoved)
	{
		// Remove from the next point onwards from currently selected point
		// unless it's the first or the last point, we want to remove the first or last point if selected
		StartIndex += 1;
	}
	if (StartIndex <= 1)
	{
		StartIndex = 1;
	}

	for (int32 BranchPointIndex = StartIndex; BranchPointIndex < BranchPoints.Num(); BranchPointIndex++)
	{
		const int32 PointIndex = BranchPoints[BranchPointIndex];
		const int32 PointBudNumber = Attributes.PointBudNumber[PointIndex];
		RemovedPoints[PointBudNumber] = true;
	}

	const int32 RemovedPointIndex = BranchPoints[StartIndex];
	const float PointLFR = Attributes.PointLengthFromRoot[RemovedPointIndex];
	for (int32 BranchChildNumber : PV::Tools::GetBranchImmediateChildren(Attributes, HitBranchIndex))
	{
		if (const int32 BranchIndex = Attributes.BranchNumber.Find(BranchChildNumber); BranchIndex != INDEX_NONE)
		{
			const TArray<int32>& ChildBranchPoints = Attributes.BranchPoints[BranchIndex];
			if (Attributes.PointLengthFromRoot[ChildBranchPoints[0]] >= PointLFR)
			{
				for (const int32 ChildBranchPointIndex : ChildBranchPoints)
				{
					const int32 ChildPointBudNumber = Attributes.PointBudNumber[ChildBranchPointIndex];
					RemovedPoints[ChildPointBudNumber] = true;
				}

				for (int32 SubBranchChildNumber : Attributes.BranchChildren[BranchIndex])
				{
					if (const int32 SubBranchIndex = Attributes.BranchNumber.Find(SubBranchChildNumber); SubBranchIndex != INDEX_NONE)
					{
						for (const int32 ChildSubBranchPointIndex : Attributes.BranchPoints[SubBranchIndex])
						{
							const int32 SubBranchChildPointBudNumber = Attributes.PointBudNumber[ChildSubBranchPointIndex];
							RemovedPoints[SubBranchChildPointBudNumber] = true;
						}
					}
				}
			}
		}
	}

	GetNodeSettings()->PostEditChange();
	PreviewComponent->RebuildSkeleton();
}

bool UPVManualEditTool::GetPointWorldPositionAndScale(
	const int32 BranchIndex,
	const int32 BranchPointIndex,
	FVector& OutPosition,
	float& OutScale
)
{
	if (!HasCollection())
	{
		return false;
	}
	const TObjectPtr<UPVManualEditSettings> ManualEditSettings = GetNodeSettings<UPVManualEditSettings>();
	const TArray<FVector3f>& Offsets = ManualEditSettings->ManualEditSettings.RelativeOffsets;
	return PV::Tools::GetPointWorldPositionAndScale(Attributes, Offsets, BranchIndex, BranchPointIndex, OutPosition, OutScale);
}
