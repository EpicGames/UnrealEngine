// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "DynamicMesh/DynamicMesh3.h"

#include "Nodes/PVManualEditSettings.h"

#include "Tools/PVBaseInteractiveTool.h"
#include "Tools/Helpers/PVManualEditToolHelpers.h"
#include "Tools/Helpers/PVSkeletonPointsSelection.h"

#include "PVManualEditTool.generated.h"

class FPrimitiveDrawInterface;
class UCombinedTransformGizmo;
class UMaterialInterface;
class UPVSkeletonVisualizerComponent;
class UPreviewMesh;
class UTransformProxy;

UCLASS()
class UPVManualEditTool
	: public UPVBaseInteractiveTool,
	  public IClickBehaviorTarget,
	  public IHoverBehaviorTarget,
	  public FSelfRegisteringEditorUndoClient
{
	GENERATED_BODY()

	PCG_DECLARE_SUPPORTED_NODES(
		UPVManualEditSettings
	)

public:
	UPVManualEditTool();

	// ~Begin UInteractiveTool Interface
	virtual void Setup() override;
	virtual void OnTick(float DeltaTime) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	// ~Begin UInteractiveTool Interface

	// ~Begin IClickBehaviorTarget Interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	// ~End IClickBehaviorTarget Interface

	// ~Begin IHoverBehaviorTarget Interface
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override;
	// ~End IHoverBehaviorTarget Interface

	// ~Begin FEditorUndoClient Interface
	virtual bool MatchesContext(
		const FTransactionContext& InContext,
		const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts
	) const override;
	virtual void PostUndo(bool bSuccess) override;
	// ~End FEditorUndoClient Interface

protected:
	// ~Begin UPCGAssetEditorInteractiveTool Interface
	virtual void OnNodeSettingsChanged(UPCGSettings* InNodeSettings, EPCGChangeType InChangeType) override;
	// ~End UPCGAssetEditorInteractiveTool Interface

private:
	void InitializeToolData();

	void OnGizmoTransformBegin(UTransformProxy* TransformProxy);
	void OnGizmoTransformChanged(UTransformProxy* TransformProxy, FTransform Transform);
	void OnGizmoTransformEnd(UTransformProxy* TransformProxy);

	// --- Render passes ---
	void RenderSelectionHighlights(
		FPrimitiveDrawInterface* PDI,
		const UPVManualEditSettings* ManualEditSettings,
		int32 SelectedBranchIndex,
		int32 SelectedBranchPointIndex
	);
	void RenderHoverHighlight(FPrimitiveDrawInterface* PDI);
	void RenderInfluenceRadius(FPrimitiveDrawInterface* PDI, const float Radius);

	// --- OnClicked mode dispatch ---
	void ApplySelectionAtClick(int32 HitBranchIndex, int32 HitBranchPointIndex);
	void ApplyBranchRemovalAtClick(int32 HitBranchIndex, int32 HitBranchPointIndex);

	/** Compute the world-space position and per-point scale for (BranchIndex, BranchPointIndex). Mirrors RebuildPreviewMesh's lookup. */
	bool GetPointWorldPositionAndScale(
		int32 BranchIndex,
		int32 BranchPointIndex,
		FVector& OutPosition,
		float& OutScale
	);

protected:
	UPROPERTY(Transient)
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UCombinedTransformGizmo> CombinedGizmo = nullptr;
	
	UPROPERTY(Transient)
	TObjectPtr<UPVSkeletonVisualizerComponent> PreviewComponent = nullptr;

	TUniquePtr<FSkeletonPointsSelection> Selection;

	PV::Tools::FManualEditAttributes Attributes;

private:
	FQuat4f PreviousRotation = FQuat4f::Identity;

	/** Last hovered (BranchIndex, BranchPointIndex) and whether the hit was a sphere or a cylinder triangle. */
	int32 HoveredBranchIndex = INDEX_NONE;
	int32 HoveredBranchPointIndex = INDEX_NONE;
	PV::Tools::ESkeletonHoverType HoveredType = PV::Tools::ESkeletonHoverType::None;

	bool bHoveringRotationGizmo = false;

	bool bIsTransforming = false;
};
