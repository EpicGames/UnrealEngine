// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVBaseInteractiveTool.h"

#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"
#include "PCGGraphExecutionInspection.h"

#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"

#include "Contexts/PVToolContext.h"

#include "DataTypes/PVData.h"

#include "Engine/World.h"

#include "Nodes/PVBaseSettings.h"

#include "Tools/EdModeInteractiveToolsContext.h"

void UPVBaseInteractiveTool::Setup()
{
	Super::Setup();

	InteractiveToolsContext = Cast<UEditorInteractiveToolsContext>(GetToolManager()->GetOuter());
	TargetWorld = GetToolManager()->GetWorld();

	// Retrieve NodeSettings from PCG's generic context object
	NodeSettings = Cast<UPVBaseSettings>(NodeToolContext->NodeSettings);

	if (ensureMsgf(TargetWorld, TEXT("Target World should be valid when setting up a PV tool")))
	{
		FActorSpawnParameters Params;
		Params.ObjectFlags = RF_Transient;
		Params.bHideFromSceneOutliner = false;
		PreviewActor = TargetWorld->SpawnActor<AActor>(Params);

		USceneComponent* Root = NewObject<USceneComponent>(PreviewActor, TEXT("PVToolPreviewRoot"), RF_Transient);
		Root->RegisterComponent();
		PreviewActor->SetRootComponent(Root);
	}

	CacheInspectedCollection();
}

void UPVBaseInteractiveTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (PreviewActor)
	{
		if (USceneComponent* const RootComponent = PreviewActor->GetRootComponent())
		{
			RootComponent->DestroyComponent();
		}
		PreviewActor->Destroy();
		PreviewActor = nullptr;
	}
	
	if (NodeSettings)
	{
		NodeSettings->OnSettingsChangedDelegate.RemoveAll(this);
	}

	Super::Shutdown(ShutdownType);
}

void UPVBaseInteractiveTool::RequestShutdown(const EToolShutdownType ShutdownType)
{
	GetToolManager()->PostActiveToolShutdownRequest(this, ShutdownType);
}

TObjectPtr<UWorld> UPVBaseInteractiveTool::GetTargetWorld()
{
	return TargetWorld;
}

bool UPVBaseInteractiveTool::HasCollection() const
{
	return Collection.IsValid();
}

const FManagedArrayCollection& UPVBaseInteractiveTool::GetCollection()
{
	check(HasCollection());
	return *Collection.Get();
}

void UPVBaseInteractiveTool::CacheInspectedCollection()
{
	// Retrieve vegetation collection from PVE-specific context object
	const UPVToolContextObject* PVToolContext = GetToolManager()->GetContextObjectStore()->FindContext<UPVToolContextObject>();
	if (!PVToolContext)
	{
		return;
	}

	const FPCGGraphExecutionInspection& Inspection = ExecutionSource->GetExecutionState().GetInspection();
	if (const TSharedPtr<FPCGInspectionData> InspectionData = Inspection.GetInspectionDataPtr(PVToolContext->InspectionStack))
	{
		if (const FPCGTaggedData* TaggedData = InspectionData->Data ? InspectionData->Data->TaggedData.GetData() : nullptr)
		{
			if (const UPVData* PVData = Cast<UPVData>(TaggedData->Data.Get()))
			{
				Collection = PVData->GetSharedCollection();
			}
		}
	}
}
