// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetEditorMode/Tools/PCGAssetEditorInteractiveTool.h"

#include "GeometryCollection/ManagedArrayCollection.h"

#include "PVBaseInteractiveTool.generated.h"

class UPVBaseSettings;
class UEditorInteractiveToolsContext;

UCLASS(Abstract)
class UPVBaseInteractiveTool : public UPCGAssetEditorInteractiveTool
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	
	virtual void RequestShutdown(EToolShutdownType ShutdownType);

	TObjectPtr<UWorld> GetTargetWorld();

	bool HasCollection() const;
	const FManagedArrayCollection& GetCollection();

	template <typename T = UPVBaseSettings>
	TObjectPtr<T> GetNodeSettings() { return CastChecked<T>(NodeSettings); }

	template <typename T = UPVBaseSettings>
	TObjectPtr<const T> GetNodeSettings() const { return CastChecked<const T>(NodeSettings); }

private:
	void CacheInspectedCollection();

protected:
	UPROPERTY(Transient)
	TObjectPtr<UWorld> TargetWorld = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<AActor> PreviewActor = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UPVBaseSettings> NodeSettings = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UEditorInteractiveToolsContext> InteractiveToolsContext;

private:
	TSharedPtr<FManagedArrayCollection> Collection;
};
