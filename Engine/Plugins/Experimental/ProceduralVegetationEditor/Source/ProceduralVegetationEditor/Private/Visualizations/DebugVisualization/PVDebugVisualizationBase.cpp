// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDebugVisualizationBase.h"

#include "AdvancedPreviewScene.h"
#include "PVDebugVisualizer.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Components/TextRenderComponent.h"

#include "DataTypes/PVData.h"

#include "Engine/StaticMesh.h"

#include "Materials/MaterialInstance.h"

#include "UObject/Package.h"

#include "Visualizations/PVLineBatchComponent.h"

#define LOCTEXT_NAMESPACE "PVDebugVisualization"

DEFINE_LOG_CATEGORY(LogPVDebugVisualization);

namespace PVE::AttributeToString
{
	FString AttributeValueToString(const float& Value) { return FString::SanitizeFloat(Value, 2); }
	FString AttributeValueToString(const double& Value) { return FString::SanitizeFloat(Value, 2); }
	FString AttributeValueToString(const uint8& Value) { return FString::FromInt((int32)Value); }
	FString AttributeValueToString(const int32& Value) { return FString::FromInt(Value); }
	FString AttributeValueToString(const FString& Value) { return Value; }
	FString AttributeValueToString(const FName& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FLinearColor& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FVector3f& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FVector3d& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FVector4f& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FVector4d& Value) { return Value.ToString(); }
	FString AttributeValueToString(const bool& Value) { return Value ? FString("true") : FString("false"); }
	FString AttributeValueToString(const FTransform3f& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FTransform& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FBox& Value) { return Value.ToString(); } 
	FString AttributeValueToString(const FVector2f& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FIntVector& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FIntVector2& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FIntVector4& Value) { return Value.ToString(); }
	FString AttributeValueToString(const FSoftObjectPath& Value) { return Value.ToString(); }

	template<typename T>
	FString AttributeValueToString(const TArray<T>& Array)
	{
		FString Out;

		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
		{
			Out += AttributeValueToString(Array[Idx]);

			if (Idx != Array.Num() - 1)
			{
				Out += "; ";
			}
		}

		return Out;
	}

	template<typename T>
	FString AttributeValueToString(const TSet<T>& Value)
	{
		return AttributeValueToString(Value.Array());
	}

	template<typename T>
	TArray<FString> AttributeToString(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName)
	{
		const TManagedArray<T>& AttributesArray = InCollection.GetAttribute<T>(InAttributeName, InGroupName);

		TArray<FString> OutStringArray;
		OutStringArray.Reserve(AttributesArray.Num());

		for (int32 i = 0; i < AttributesArray.Num(); ++i)
		{
			FString ValueAsString = AttributeValueToString(AttributesArray[i]);
			
			// Clip really long strings
			constexpr int32 MaxStringLength = 1000;
			if (ValueAsString.Len() > MaxStringLength)
			{
				ValueAsString.LeftInline(MaxStringLength);
				ValueAsString += "...";
			}
			
			OutStringArray.Add(ValueAsString);
		}

		return OutStringArray;
	}

	TArray<FString> ConvertAttributeToString(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName)
	{
		const FManagedArrayCollection::EArrayType ArrayType = InCollection.GetAttributeType(InAttributeName, InGroupName);

		TArray<FString> AttributeAsString;

		switch (ArrayType)
		{
		case FManagedArrayCollection::EArrayType::FFloatType:
			AttributeAsString = AttributeToString<float>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FDoubleType:
			AttributeAsString = AttributeToString<double>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FUInt8Type:
			AttributeAsString = AttributeToString<uint8>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FInt32Type:
			AttributeAsString = AttributeToString<int32>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FBoolType:
			AttributeAsString = AttributeToString<bool>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FStringType:
			AttributeAsString = AttributeToString<FString>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FNameType:
			AttributeAsString = AttributeToString<FName>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FSoftObjectPathType:
			AttributeAsString = AttributeToString<FSoftObjectPath>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FLinearColorType:
			AttributeAsString = AttributeToString<FLinearColor>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVectorType:
			AttributeAsString = AttributeToString<FVector3f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DType:
			AttributeAsString = AttributeToString<FVector2f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector3dType:
			AttributeAsString = AttributeToString<FVector3d>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector4fType:
			AttributeAsString = AttributeToString<FVector4f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVectorType:
			AttributeAsString = AttributeToString<FIntVector>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector2Type:
			AttributeAsString = AttributeToString<FIntVector2>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FFVectorArrayType:
			AttributeAsString = AttributeToString<TArray<FVector3f>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector3ArrayType:
			AttributeAsString = AttributeToString<TArray<FIntVector3>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector2ArrayType:
			AttributeAsString = AttributeToString<TArray<FIntVector2>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector4fArrayType:
			AttributeAsString = AttributeToString<TArray<FVector4f>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FTransformType:
			AttributeAsString = AttributeToString<FTransform>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DArrayType:
			AttributeAsString = AttributeToString<TArray<FVector2f>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntArrayType:
			AttributeAsString = AttributeToString<TSet<int32>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FInt32ArrayType:
			AttributeAsString = AttributeToString<TArray<int32>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FFloatArrayType:
			AttributeAsString = AttributeToString<TArray<float>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FBoxType:
			AttributeAsString = AttributeToString<FBox>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FTransform3fType:
			AttributeAsString = AttributeToString<FTransform3f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector4Type:
			AttributeAsString = AttributeToString<FIntVector4>(InCollection, InAttributeName, InGroupName);
			break;
		default:
			break;
		}

		return AttributeAsString;
	}
};

namespace PVE::AttributeToScalar
{
	double AttributeValueToScalar(const float& Value) { return Value; }
	double AttributeValueToScalar(const double& Value) { return Value; }
	double AttributeValueToScalar(const uint8& Value) { return double(Value); }
	double AttributeValueToScalar(const int32& Value) { return double(Value); }
	double AttributeValueToScalar(const bool& Value) { return Value ? 1 : 0; }
	double AttributeValueToScalar(const FString& Value) { return FCString::IsNumeric(*Value) ? FCString::Atof(*Value) : 0; }
	double AttributeValueToScalar(const FName& Value) { return AttributeValueToScalar(Value.ToString()); }
	double AttributeValueToScalar(const FVector2f& Value) { return Value.Size(); }
	double AttributeValueToScalar(const FVector3f& Value) { return Value.Size(); }
	double AttributeValueToScalar(const FVector3d& Value) { return Value.Size(); }
	double AttributeValueToScalar(const FVector4f& Value) { return Value.Size(); }
	double AttributeValueToScalar(const FIntVector& Value) { return Value.Size(); }

	template<typename T>
	TArray<double> AttributeToScalar(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName)
	{
		const TManagedArray<T>& AttributesArray = InCollection.GetAttribute<T>(InAttributeName, InGroupName);

		TArray<double> OutScalarArray;
		OutScalarArray.Reserve(AttributesArray.Num());

		for (int32 i = 0; i < AttributesArray.Num(); ++i)
		{
			const double ValueAsScalar = AttributeValueToScalar(AttributesArray[i]);
			OutScalarArray.Add(ValueAsScalar);
		}

		return OutScalarArray;
	}

	TArray<double> ConvertAttributeToScalar(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName)
	{
		const FManagedArrayCollection::EArrayType ArrayType = InCollection.GetAttributeType(InAttributeName, InGroupName);

		TArray<double> AttributeAsScalar;

		switch (ArrayType)
		{
		case FManagedArrayCollection::EArrayType::FFloatType:
			AttributeAsScalar = AttributeToScalar<float>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FDoubleType:
			AttributeAsScalar = AttributeToScalar<double>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FUInt8Type:
			AttributeAsScalar = AttributeToScalar<uint8>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FInt32Type:
			AttributeAsScalar = AttributeToScalar<int32>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FBoolType:
			AttributeAsScalar = AttributeToScalar<bool>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FStringType:
			AttributeAsScalar = AttributeToScalar<FString>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FNameType:
			AttributeAsScalar = AttributeToScalar<FName>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DType:
			AttributeAsScalar = AttributeToScalar<FVector2f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVectorType:
			AttributeAsScalar = AttributeToScalar<FVector3f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector3dType:
			AttributeAsScalar = AttributeToScalar<FVector3d>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector4fType:
			AttributeAsScalar = AttributeToScalar<FVector4f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVectorType:
			AttributeAsScalar = AttributeToScalar<FIntVector>(InCollection, InAttributeName, InGroupName);
			break;
		default:
			break;
		}

		return AttributeAsScalar;
	}
};

namespace PVE::AttributeToVector3
{
	typedef TArray<FVector, TInlineAllocator<1>> FVectorValue;

	FVectorValue AttributeValueToVector3(const float& Value) { return { FVector(Value) }; }
	FVectorValue AttributeValueToVector3(const double& Value) { return { FVector(Value) }; }
	FVectorValue AttributeValueToVector3(const uint8& Value) { return { FVector(Value) }; }
	FVectorValue AttributeValueToVector3(const int32& Value) { return { FVector(Value) }; }
	FVectorValue AttributeValueToVector3(const FString& Value) { FVector OutVec(EForceInit::ForceInitToZero); OutVec.InitFromString(Value); return { OutVec }; }
	FVectorValue AttributeValueToVector3(const FName& Value) { return AttributeValueToVector3(Value.ToString()); }
	FVectorValue AttributeValueToVector3(const FLinearColor& Value) { return { FVector(Value.R, Value.G, Value.B) }; }
	FVectorValue AttributeValueToVector3(const FVector3f& Value) { return { FVector(Value) }; }
	FVectorValue AttributeValueToVector3(const FVector3d& Value) { return { Value }; }
	FVectorValue AttributeValueToVector3(const FVector4f& Value) { return { FVector(Value.X, Value.Y, Value.Z) }; }
	FVectorValue AttributeValueToVector3(const FVector4d& Value) { return { FVector(Value.X, Value.Y, Value.Z) }; }
	FVectorValue AttributeValueToVector3(const bool& Value) { return { FVector(double(Value)) }; }
	FVectorValue AttributeValueToVector3(const FTransform3f& Value) { return { FVector(Value.GetLocation()) }; }
	FVectorValue AttributeValueToVector3(const FTransform& Value) { return { Value.GetLocation() }; }
	FVectorValue AttributeValueToVector3(const FVector2f& Value) { return { FVector(Value.X, Value.Y, 0) }; }
	FVectorValue AttributeValueToVector3(const FIntVector& Value) { return { FVector(Value.X, Value.Y, Value.Z) }; }
	FVectorValue AttributeValueToVector3(const FIntVector2& Value) { return { FVector(Value.X, Value.Y, 0) }; }
	FVectorValue AttributeValueToVector3(const FIntVector4& Value) { return { FVector(Value.X, Value.Y, Value.Z) }; }

	template<typename T>
	FVectorValue AttributeValueToVector3(const TArray<T>& Array)
	{
		FVectorValue Out;
		Out.Reserve(Array.Num());

		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
		{
			Out.Append(AttributeValueToVector3(Array[Idx]));
		}

		return Out;
	}

	template<typename T>
	FVectorValue AttributeValueToVector3(const TSet<T>& Value)
	{
		return AttributeValueToVector3(Value.Array());
	}

	template<typename T>
	TArray<FVectorValue> AttributeToVector3(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName)
	{
		const TManagedArray<T>& AttributesArray = InCollection.GetAttribute<T>(InAttributeName, InGroupName);

		TArray<FVectorValue> OutVectorArray;
		OutVectorArray.Reserve(AttributesArray.Num());

		for (int32 i = 0; i < AttributesArray.Num(); ++i)
		{
			FVectorValue ValueAsString = AttributeValueToVector3(AttributesArray[i]);
			OutVectorArray.Add(MoveTemp(ValueAsString));
		}

		return OutVectorArray;
	}

	TArray<FVectorValue> ConvertAttributeToVector3(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InGroupName)
	{
		const FManagedArrayCollection::EArrayType ArrayType = InCollection.GetAttributeType(InAttributeName, InGroupName);

		TArray<FVectorValue> AttributeAsVector;

		switch (ArrayType)
		{
		case FManagedArrayCollection::EArrayType::FFloatType:
			AttributeAsVector = AttributeToVector3<float>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FDoubleType:
			AttributeAsVector = AttributeToVector3<double>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FUInt8Type:
			AttributeAsVector = AttributeToVector3<uint8>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FInt32Type:
			AttributeAsVector = AttributeToVector3<int32>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FBoolType:
			AttributeAsVector = AttributeToVector3<bool>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FStringType:
			AttributeAsVector = AttributeToVector3<FString>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FNameType:
			AttributeAsVector = AttributeToVector3<FName>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FLinearColorType:
			AttributeAsVector = AttributeToVector3<FLinearColor>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVectorType:
			AttributeAsVector = AttributeToVector3<FVector3f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DType:
			AttributeAsVector = AttributeToVector3<FVector2f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector3dType:
			AttributeAsVector = AttributeToVector3<FVector3d>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector4fType:
			AttributeAsVector = AttributeToVector3<FVector4f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVectorType:
			AttributeAsVector = AttributeToVector3<FIntVector>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector2Type:
			AttributeAsVector = AttributeToVector3<FIntVector2>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FFVectorArrayType:
			AttributeAsVector = AttributeToVector3<TArray<FVector3f>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector3ArrayType:
			AttributeAsVector = AttributeToVector3<TArray<FIntVector3>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector2ArrayType:
			AttributeAsVector = AttributeToVector3<TArray<FIntVector2>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector4fArrayType:
			AttributeAsVector = AttributeToVector3<TArray<FVector4f>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FTransformType:
			AttributeAsVector = AttributeToVector3<FTransform>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DArrayType:
			AttributeAsVector = AttributeToVector3<TArray<FVector2f>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntArrayType:
			AttributeAsVector = AttributeToVector3<TSet<int32>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FInt32ArrayType:
			AttributeAsVector = AttributeToVector3<TArray<int32>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FFloatArrayType:
			AttributeAsVector = AttributeToVector3<TArray<float>>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FTransform3fType:
			AttributeAsVector = AttributeToVector3<FTransform3f>(InCollection, InAttributeName, InGroupName);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector4Type:
			AttributeAsVector = AttributeToVector3<FIntVector4>(InCollection, InAttributeName, InGroupName);
			break;
		default:
			break;
		}

		return AttributeAsVector;
	}
};

void FPVDebugVisualizationBase::Draw(FVisualizerDrawContext& InContext)
{
	DrawPivotPoints(InContext);
	DrawAttributes(InContext);
}

void FPVDebugVisualizationBase::DrawPivotPoints(const FVisualizerDrawContext& InContext)
{
	const auto& Settings = InContext.VisualizationSettings;

	if (Settings.bShowAnchorPoints)
	{
		const TArray<FVector3f> Positions = GetPivotPositions(InContext.Collection);
	
		for (int32 i = 0; i < Positions.Num(); i++)
		{
			const FVector Pos(Positions[i]);
			AddDrawAsPoint(Pos, Settings.AnchorPointsScale, Settings.Color);
		}	
	}
}

void FPVDebugVisualizationBase::DrawAttributes(FVisualizerDrawContext& InContext)
{
	const auto& Collection = InContext.Collection;
	
	const auto& VisualizationSettings = InContext.VisualizationSettings;
	const FName AttributeName = VisualizationSettings.AttributeToFilter;
	const FName GroupName = VisualizationSettings.GetGroupToFilter();

	if (!VisualizationSettings.bShow)
	{
		return;
	}
	
	if (GroupName.IsNone() || AttributeName.IsNone() || !Collection.HasAttribute(AttributeName, GroupName) || !Collection.HasGroup(GroupName))
	{
		return;
	}
	
	const int32 NumElements = Collection.NumElements(GroupName);

	switch (VisualizationSettings.VisualizationMode)
	{
		case EPVDebugValueVisualizationMode::Text:
		{
			using namespace PVE::AttributeToString;

			TArray<FString> AttributeAsString = ConvertAttributeToString(Collection, AttributeName, GroupName);
			if (AttributeAsString.Num() != NumElements)
			{
				break;
			}

			for (int32 i = 0; i < NumElements; i++)
			{
				FVector3f OutPos;
				float OutScale;
				GetPivot(Collection, i, OutPos, OutScale);

				const FColor DrawColor = VisualizationSettings.bRandomizeColors ? FColor::MakeRandomColor() : VisualizationSettings.Color;
				DrawAsText(InContext.SceneSetupParams, FText::FromString(AttributeAsString[i]), OutPos, OutScale, VisualizationSettings.TextScale, DrawColor, VisualizationSettings.TextOffset);
			}
			break;
		}
		case EPVDebugValueVisualizationMode::Direction:
		{
			using namespace PVE::AttributeToVector3;

			const TArray<FVectorValue> AttributeAsVector = ConvertAttributeToVector3(Collection, AttributeName, GroupName);
			if (AttributeAsVector.Num() != NumElements)
			{
				break;
			}

			TObjectPtr<UPVLineBatchComponent> LineBatchComponent = GetOrCreateLineComponent(InContext.SceneSetupParams);

			for (int32 i = 0; i < NumElements; i++)
			{
				FVector3f OutPos;
				float OutScale;
				GetPivot(Collection, i, OutPos, OutScale);

				const FColor DrawColor = VisualizationSettings.bRandomizeColors ? FColor::MakeRandomColor() : VisualizationSettings.Color;

				for (int32 j = 0; j < AttributeAsVector[i].Num(); ++j)
				{
					const float BaseDirectionStrength = 8.f;

					const FVector& DirectionVector = AttributeAsVector[i][j];
					const FVector StartPos(OutPos);
					const FVector EndPos = StartPos + DirectionVector * BaseDirectionStrength * VisualizationSettings.GizmoScale;

					const FColor LineColor = FLinearColor::LerpUsingHSV(DrawColor.ReinterpretAsLinear(), FLinearColor::Black, j / (float)AttributeAsVector[i].Num()).ToFColor(true);
					DrawAsLine(LineBatchComponent, StartPos, EndPos, LineColor, VisualizationSettings.DepthPriorityGroup);

					if (AttributeAsVector[i].Num() > 1)
					{
						DrawAsText(InContext.SceneSetupParams, FText::FromString(FString::Printf(TEXT("%d"), j)), FVector3f(EndPos), 0.f, VisualizationSettings.TextScale, DrawColor, VisualizationSettings.TextOffset);
					}
				}
			}
			break;
		}
		case EPVDebugValueVisualizationMode::Point:
		{
			using namespace PVE::AttributeToVector3;

			if (VisualizationSettings.bUsePivotAsPosition)
			{
				TArray<double> AttributeAsScalar = PVE::AttributeToScalar::ConvertAttributeToScalar(Collection, AttributeName, GroupName);
				if (AttributeAsScalar.Num() != NumElements)
				{
					break;
				}

				for (int32 i = 0; i < NumElements; i++)
				{
					FVector3f OutPos;
					float OutScale;
					GetPivot(Collection, i, OutPos, OutScale);

					const FColor DrawColor = VisualizationSettings.bRandomizeColors ? FColor::MakeRandomColor() : VisualizationSettings.Color;
					AddDrawAsPoint(FVector(OutPos), AttributeAsScalar[i] * VisualizationSettings.GizmoScale * OutScale, VisualizationSettings.Color);
				}
			}
			else 
			{
				const TArray<FVectorValue> AttributeAsVector = ConvertAttributeToVector3(Collection, AttributeName, GroupName);
				if (AttributeAsVector.Num() != NumElements)
				{
					break;
				}

				for (int32 i = 0; i < NumElements; i++)
				{
					for (int32 j = 0; j < AttributeAsVector[i].Num(); ++j)
					{
						AddDrawAsPoint(AttributeAsVector[i][j], VisualizationSettings.GizmoScale, VisualizationSettings.Color);
					}
				}	
			}

			break;
		}
		case EPVDebugValueVisualizationMode::Sphere:
		{
			TArray<double> AttributeAsScalar = PVE::AttributeToScalar::ConvertAttributeToScalar(Collection, AttributeName, GroupName);
			if (AttributeAsScalar.Num() != NumElements)
			{
				break;
			}

			if (VisualizationSettings.bDrawSphereAsMesh)
			{
				TArray<FSphere> SpheresToDraw;
				SpheresToDraw.Reserve(NumElements);
				for (int32 i = 0; i < NumElements; i++)
				{
					FVector3f OutPos;
					float OutScale;
					GetPivot(Collection, i, OutPos, OutScale);
					const FLinearColor DrawColor = VisualizationSettings.bRandomizeColors ? FLinearColor(FColor::MakeRandomColor()) : FLinearColor(VisualizationSettings.Color);
					SpheresToDraw.Add({ FVector(OutPos), static_cast<float>(AttributeAsScalar[i]) * VisualizationSettings.GizmoScale * OutScale, DrawColor });
				}
				DrawAsMeshSpheres(GetOrCreateSphereComponent(InContext.SceneSetupParams), SpheresToDraw);
			}
			else
			{
				TObjectPtr<UPVLineBatchComponent> LineBatchComponent = GetOrCreateLineComponent(InContext.SceneSetupParams);
				for (int32 i = 0; i < NumElements; i++)
				{
					FVector3f OutPos;
					float OutScale;
					GetPivot(Collection, i, OutPos, OutScale);
					const FColor DrawColor = VisualizationSettings.bRandomizeColors ? FColor::MakeRandomColor() : VisualizationSettings.Color;
					DrawAsSphere(LineBatchComponent, FVector(OutPos), AttributeAsScalar[i] * VisualizationSettings.GizmoScale * OutScale, DrawColor, VisualizationSettings.DepthPriorityGroup);
				}
			}
			break;
		}
		case EPVDebugValueVisualizationMode::Curve:
		{
			using namespace PVE::AttributeToVector3;

			const TArray<FVectorValue> AttributeAsVector = ConvertAttributeToVector3(Collection, AttributeName, GroupName);
			if (AttributeAsVector.Num() != NumElements)
			{
				break;
			}

			TObjectPtr<UPVLineBatchComponent> LineBatchComponent = GetOrCreateLineComponent(InContext.SceneSetupParams);

			for (int32 i = 0; i < NumElements; i++)
			{
				const FColor DrawColor = VisualizationSettings.bRandomizeColors ? FColor::MakeRandomColor() : VisualizationSettings.Color;
				for (int32 j = 0; j < AttributeAsVector[i].Num() - 1; ++j)
				{
					const FVector StartPos = AttributeAsVector[i][j];
					const FVector EndPos = AttributeAsVector[i][j + 1];
					DrawAsLine(LineBatchComponent, StartPos, EndPos, DrawColor, VisualizationSettings.DepthPriorityGroup);
				}
			}
			break;
		}
		case EPVDebugValueVisualizationMode::ColorRamp:
		{
			using namespace PVE::AttributeToScalar;

			TArray<double> AttributeAsScalar = ConvertAttributeToScalar(Collection, AttributeName, GroupName);
			if (AttributeAsScalar.Num() != NumElements || NumElements == 0)
			{
				break;
			}

			const double MaxValue = *Algo::MaxElement(AttributeAsScalar);
			const double MinValue = *Algo::MinElement(AttributeAsScalar);
			const double Range = MaxValue - MinValue;
			if (FMath::IsNearlyZero(Range))
			{
				break;
			}

			for (int32 i = 0; i < NumElements; i++)
			{
				FVector3f OutPos;
				float OutScale;
				GetPivot(Collection, i, OutPos, OutScale);

				AddDrawAsPoint(static_cast<FVector>(OutPos), OutScale * VisualizationSettings.GizmoScale, PV::Utilities::GetRandomHueColor(AttributeAsScalar[i] - MinValue) / Range);
			}
			break;
		}
		case EPVDebugValueVisualizationMode::Vector:
			ensureMsgf(false, TEXT("Unexpected visualization mode"));
			return;
	}

	if (PointsToDraw.Num() > 0)
	{
		const bool bCanDrawAsMeshPoints = VisualizationSettings.VisualizationMode != EPVDebugValueVisualizationMode::ColorRamp;
		if (VisualizationSettings.bDrawPointAsMesh && bCanDrawAsMeshPoints)
		{
			DrawAsMeshPoints(GetOrCreateSphereComponent(InContext.SceneSetupParams), PointsToDraw);
		}
		else
		{
			DrawAsPoints(GetOrCreateLineComponent(InContext.SceneSetupParams), PointsToDraw, VisualizationSettings.DepthPriorityGroup);
		}

		PointsToDraw.Reset();
	}
}

void FPVDebugVisualizationBase::AddDrawAsPoint(const FVector& InPos, float PointScale, const FLinearColor& InColor)
{
	if (!FMath::IsNearlyZero(PointScale))
	{
		PointsToDraw.Add({ InPos, PointScale, InColor });
	}
}

void FPVDebugVisualizationBase::DrawAsText(FPCGSceneSetupParams& InOutParams, const FText& InTextToDraw, const FVector3f& InPos, float InScale, float TextSize, const FLinearColor& InColor, const FVector3f TextOffset)
{
	const FVector3f& Pos = InPos + (FVector3f::RightVector * -1 * InScale);

	TObjectPtr<UTextRenderComponent> Component = NewObject<UTextRenderComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	Component->SetText(InTextToDraw);
	Component->SetTextRenderColor(InColor.ToFColor(false));
	Component->SetWorldSize(TextSize);
	Component->SetGenerateOverlapEvents(false);
	Component->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	
	InOutParams.ManagedResources.Add(Component);
	InOutParams.Scene->AddComponent(Component, FTransform(FVector(Pos.X + TextOffset.X, Pos.Y + TextOffset.Y, Pos.Z + TextOffset.Z)));
}

void FPVDebugVisualizationBase::DrawAsLine(UPVLineBatchComponent* LineBatchComponent, const FVector& InStartPos, const FVector& InEndPos, const FLinearColor& InColor, ESceneDepthPriorityGroup InDepthPriorityGroup)
{
	check(LineBatchComponent);
	LineBatchComponent->AddLine(InStartPos, InEndPos, InColor, InDepthPriorityGroup);
}

void FPVDebugVisualizationBase::DrawAsSphere(UPVLineBatchComponent* LineBatchComponent, const FVector& InPos, float Radius, const FLinearColor& InColor, ESceneDepthPriorityGroup InDepthPriorityGroup)
{
	check(LineBatchComponent);
	LineBatchComponent->AddSphere(InPos, Radius, InColor, InDepthPriorityGroup);
}

void FPVDebugVisualizationBase::DrawAsPoints(UPVLineBatchComponent* LineBatchComponent, const TArray<FPoint>& InPoints, ESceneDepthPriorityGroup InDepthPriorityGroup)
{
	check(LineBatchComponent);
	TArray<FPVPointInfo> PointsToDraw;
	PointsToDraw.Reserve(InPoints.Num());

	for (const FPoint& Point : InPoints)
	{
		PointsToDraw.Emplace(Point.Position, Point.Scale, Point.Color, InDepthPriorityGroup);
	}

	LineBatchComponent->AddPoints(PointsToDraw);
}

void FPVDebugVisualizationBase::DrawAsMeshPoints(UInstancedStaticMeshComponent* InstancedStaticMeshComponent, const TArray<FPoint>& InPoints)
{
	check(InstancedStaticMeshComponent != nullptr);
	// BasicShapes/Sphere has a radius of 50 units in mesh space, so scale = desired_radius / 50
	constexpr float SphereMeshRadius = 50.f;
	for (const auto& [Position, Size, Color] : InPoints)
	{
		if (Size <= 0)
		{
			continue;
		}

		const int32 InstanceIndex = InstancedStaticMeshComponent->AddInstance(FTransform(FRotator::ZeroRotator, Position, FVector(Size / SphereMeshRadius)));
		InstancedStaticMeshComponent->SetCustomDataValue(InstanceIndex, 0, Color.R, false);
		InstancedStaticMeshComponent->SetCustomDataValue(InstanceIndex, 1, Color.G, false);
		InstancedStaticMeshComponent->SetCustomDataValue(InstanceIndex, 2, Color.B, false);
	}
	InstancedStaticMeshComponent->MarkRenderStateDirty();
}

UPVLineBatchComponent* FPVDebugVisualizationBase::GetOrCreateLineComponent(FPCGSceneSetupParams& InOutParams)
{
	UPVLineBatchComponent* LineComponent = nullptr;
	InOutParams.ManagedResources.FindItemByClass<UPVLineBatchComponent>(&LineComponent);

	if (!LineComponent)
	{
		LineComponent = NewObject<UPVLineBatchComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		LineComponent->InitBounds();

		AddToScene(InOutParams, LineComponent);
	}

	return LineComponent;
}

void FPVDebugVisualizationBase::DrawAsMeshSpheres(UInstancedStaticMeshComponent* SphereComponent, const TArray<FSphere>& InSpheres)
{
	check(SphereComponent != nullptr);
	// BasicShapes/Sphere has a radius of 50 units in mesh space, so scale = desired_radius / 50
	constexpr float SphereMeshRadius = 50.f;
	for (const auto& [Position, Radius, Color] : InSpheres)
	{
		if (Radius <= 0)
		{
			continue;
		}

		const int32 InstanceIndex = SphereComponent->AddInstance(FTransform(FRotator::ZeroRotator, Position, FVector(Radius / SphereMeshRadius)));
		SphereComponent->SetCustomDataValue(InstanceIndex, 0, Color.R, false);
		SphereComponent->SetCustomDataValue(InstanceIndex, 1, Color.G, false);
		SphereComponent->SetCustomDataValue(InstanceIndex, 2, Color.B, false);
	}
	SphereComponent->MarkRenderStateDirty();
}

UInstancedStaticMeshComponent* FPVDebugVisualizationBase::GetOrCreateSphereComponent(FPCGSceneSetupParams& InOutParams)
{
	UInstancedStaticMeshComponent* InstancedStaticMeshComponent = nullptr;
	InOutParams.ManagedResources.FindItemByClass<UInstancedStaticMeshComponent>(&InstancedStaticMeshComponent);

	if (!InstancedStaticMeshComponent)
	{
		UStaticMesh* PointSphere = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Sphere"));
		
		InstancedStaticMeshComponent = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage(), NAME_None);
		InstancedStaticMeshComponent->SetStaticMesh(PointSphere);
		InstancedStaticMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		InstancedStaticMeshComponent->SetMaterial(0, LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/FoliageAttachmentPointMaterial.FoliageAttachmentPointMaterial")));

		AddToScene(InOutParams, InstancedStaticMeshComponent);
	}

	return InstancedStaticMeshComponent;
}

void FPVDebugVisualizationBase::AddToScene(FPCGSceneSetupParams& InOutParams, UPrimitiveComponent* InComponent)
{
	InOutParams.ManagedResources.Add(InComponent);
	InOutParams.Scene->AddComponent(InComponent, FTransform::Identity);
}

#undef LOCTEXT_NAMESPACE