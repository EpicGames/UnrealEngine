// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PVUtilities.h"

#include "GeometryCollection/ManagedArray.h"
#include "Templates/SharedPointer.h"

class UPVLineBatchComponent;
struct FVisualizerDrawContext;
class UPVSkeletonVisualizerComponent;
enum class EManagedArrayType : uint8;
class FPVAttributeValueInterface;
struct FPCGSceneSetupParams;
class UPVData;

class FPVDebugVisualizationBase
{
public:
	virtual ~FPVDebugVisualizationBase() = default;
	FPVDebugVisualizationBase(){};
	virtual void Draw(FVisualizerDrawContext& InContext);
protected:
	void DrawPivotPoints(const FVisualizerDrawContext& InContext);
	void DrawAttributes(FVisualizerDrawContext& InContext);
	void AddDrawAsPoint(const FVector& InPos, float PointScale, const FLinearColor& InColor = FLinearColor::White);
	virtual TArray<FVector3f> GetPivotPositions(const FManagedArrayCollection& InCollection) = 0;
	virtual void GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale) = 0;
	
	struct FPoint
	{
		FVector Position;
		float Scale;
		FLinearColor Color;
	};

	struct FSphere
	{
		FVector Position;
		float Radius;
		FLinearColor Color;
	};

	static void DrawAsText(FPCGSceneSetupParams& InOutParams, const FText& InTextToDraw, const FVector3f& InPos, float InScale, float TextSize, const FLinearColor& InColor = FLinearColor::White, const FVector3f TextOffset = FVector3f::Zero());
	static void DrawAsLine(UPVLineBatchComponent* LineBatchComponent, const FVector& InStartPos, const FVector& InEndPos, const FLinearColor& InColor, ESceneDepthPriorityGroup InDepthPriorityGroup);
	static void DrawAsSphere(UPVLineBatchComponent* LineBatchComponent, const FVector& InPos, float Radius, const FLinearColor& InColor, ESceneDepthPriorityGroup InDepthPriorityGroup);
	static void DrawAsPoints(UPVLineBatchComponent* LineBatchComponent, const TArray<FPoint>& InPoints, ESceneDepthPriorityGroup InDepthPriorityGroup);
	static void DrawAsMeshPoints(UInstancedStaticMeshComponent* InstancedStaticMeshComponent, const TArray<FPoint>& InPoints);
	static void DrawAsMeshSpheres(UInstancedStaticMeshComponent* SphereComponent, const TArray<FSphere>& InSpheres);

	static UPVLineBatchComponent* GetOrCreateLineComponent(FPCGSceneSetupParams& InOutParams);
	static UInstancedStaticMeshComponent* GetOrCreateSphereComponent(FPCGSceneSetupParams& InOutParams);

	static void AddToScene(FPCGSceneSetupParams& InOutParams, UPrimitiveComponent* InComponent);
	
private:
	TArray<FPoint> PointsToDraw;
};
typedef TSharedPtr<FPVDebugVisualizationBase> FPVDebugVisualizationPtr;

DECLARE_LOG_CATEGORY_EXTERN(LogPVDebugVisualization, Log, All);