// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDebugVisualizations.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVAttributesNames.h"

namespace PV::DebugVisualization
{
	const TArray<FVector3f>& GetVectorAttribute(const FManagedArrayCollection& InCollection, FName AttributeName, FName GroupName)
	{
		const TManagedArrayAccessor<FVector3f> Attribute(InCollection, AttributeName, GroupName);
		if (Attribute.IsValid())
		{
			return Attribute.Get().GetConstArray();
		}
		
		static TArray<FVector3f> EmptyArray;
		return EmptyArray;
	}

	const TArray<float>& GetFloatAttribute(const FManagedArrayCollection& InCollection, FName AttributeName, FName GroupName)
	{
		const TManagedArrayAccessor<float> Attribute(InCollection, AttributeName, GroupName);
		if (Attribute.IsValid())
		{
			return Attribute.Get().GetConstArray();
		}

		static TArray<float> EmptyArray;
		return EmptyArray;
	}
};

TArray<FVector3f> FPVPointDebugVisualization::GetPivotPositions(const FManagedArrayCollection& InCollection)
{
	return PV::DebugVisualization::GetVectorAttribute(InCollection, PV::AttributeNames::PointPosition, PV::GroupNames::PointGroup);
}

void FPVPointDebugVisualization::GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale)
{
	const TArray<FVector3f>& PointPositions = PV::DebugVisualization::GetVectorAttribute(InCollection, PV::AttributeNames::PointPosition, PV::GroupNames::PointGroup);
	const TArray<float>& PointScales = PV::DebugVisualization::GetFloatAttribute(InCollection, PV::AttributeNames::PointScale, PV::GroupNames::PointGroup);

	OutPos = PointPositions.IsValidIndex(InIndex) ? PointPositions[InIndex] : FVector3f::ZeroVector;
	OutScale = PointScales.IsValidIndex(InIndex) ? PointScales[InIndex] : 1.f;
}

TArray<FVector3f> FPVFoliageDebugVisualization::GetPivotPositions(const FManagedArrayCollection& InCollection)
{
	return PV::DebugVisualization::GetVectorAttribute(InCollection, PV::AttributeNames::FoliagePivotPoint, PV::GroupNames::FoliageGroup);
}

void FPVFoliageDebugVisualization::GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale)
{
	const PV::Facades::FFoliageFacade FoliageFacade(InCollection);
	const PV::Facades::FBranchFacade BranchFacade(InCollection);
	const PV::Facades::FPointFacade PointFacade(InCollection);

	OutPos = FVector3f::ZeroVector;
	OutScale = 1.f;

	if (FoliageFacade.IsValid())
	{
		OutPos = FoliageFacade.GetPivotPoint(InIndex);
	}

	if (FoliageFacade.IsValid() && BranchFacade.IsValid() && PointFacade.IsValid())
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(FoliageFacade.GetFoliageBranchId(InIndex));
		if (BranchPoints.Num() > 0)
		{
			OutScale = PointFacade.GetPointScale(BranchPoints[0]);
		}
	}
}

TArray<FVector3f> FPVBranchDebugVisualization::GetPivotPositions(const FManagedArrayCollection& InCollection)
{
	const PV::Facades::FBranchFacade BranchFacade(InCollection);
	const PV::Facades::FPointFacade PointFacade(InCollection);

	TArray<FVector3f> Positions;
	for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
	{
		const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);

		if (BranchPoints.Num() > 0)
		{
			FVector3f RootBranchPointPos = PointFacade.GetPosition(BranchPoints[0]);
			Positions.Add(RootBranchPointPos);
		}
	}

	return Positions;
}

void FPVBranchDebugVisualization::GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale)
{
	const PV::Facades::FBranchFacade BranchFacade(InCollection);
	const PV::Facades::FPointFacade PointFacade(InCollection);

	OutPos = FVector3f::ZeroVector;
	OutScale = 1.f;

	const TArray<int32>& BranchPoints = BranchFacade.GetPoints(InIndex);
	if (BranchPoints.Num() > 0)
	{
		const int RootPointIndex = BranchPoints[0];

		if (PointFacade.GetElementCount() > RootPointIndex)
		{
			OutPos = PointFacade.GetPosition(RootPointIndex);
			OutScale = PointFacade.GetPointScale(RootPointIndex);
		}
	}
}

TArray<FVector3f> FPVCustomDebugVisualization::GetPivotPositions(const FManagedArrayCollection& InCollection)
{
	return PV::DebugVisualization::GetVectorAttribute(InCollection, PivotPositionAttributeName, PivotGroupName);
}

void FPVCustomDebugVisualization::GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale)
{
	const TArray<FVector3f>& PointPositions = PV::DebugVisualization::GetVectorAttribute(InCollection, PivotPositionAttributeName, PivotGroupName);
	const TArray<float>& PointScales = PV::DebugVisualization::GetFloatAttribute(InCollection, PivotScaleAttributeName, PivotGroupName);

	OutPos = PointPositions.IsValidIndex(InIndex) ? PointPositions[InIndex] : FVector3f::ZeroVector;
	OutScale = PointScales.IsValidIndex(InIndex) ? PointScales[InIndex] : 1.f;
}