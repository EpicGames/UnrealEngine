// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers/PVUtilities.h"

#include "PVDebugVisualizationBase.h"
#include "GeometryCollection/ManagedArray.h"

struct FManagedArrayCollection;

class FPVPointDebugVisualization final : public FPVDebugVisualizationBase
{
public:
	FPVPointDebugVisualization(){};
protected:
	virtual TArray<FVector3f> GetPivotPositions(const FManagedArrayCollection& InCollection) override;
	virtual void GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale) override;
};

class FPVFoliageDebugVisualization final : public FPVDebugVisualizationBase
{
public:
	FPVFoliageDebugVisualization(){};
protected:
	virtual TArray<FVector3f> GetPivotPositions(const FManagedArrayCollection& InCollection) override;
	virtual void GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale) override;
};

class FPVBranchDebugVisualization final : public FPVDebugVisualizationBase
{
public:
	FPVBranchDebugVisualization(){};
protected:
	virtual TArray<FVector3f> GetPivotPositions(const FManagedArrayCollection& InCollection) override;
	virtual void GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale) override;
};

class FPVCustomDebugVisualization final : public FPVDebugVisualizationBase
{
private:
	FName PivotPositionAttributeName = NAME_None;
	FName PivotScaleAttributeName = NAME_None;
	FName PivotGroupName = NAME_None;

public:
	FPVCustomDebugVisualization(FName InPivotPositionAttributeName, FName InPivotScaleAttributeName, FName InPivotGroupName) 
		: PivotPositionAttributeName(InPivotPositionAttributeName)
		, PivotScaleAttributeName(InPivotScaleAttributeName)
		, PivotGroupName(InPivotGroupName)
	{
	}

protected:
	virtual TArray<FVector3f> GetPivotPositions(const FManagedArrayCollection& InCollection) override;
	virtual void GetPivot(const FManagedArrayCollection& InCollection, const int InIndex, FVector3f& OutPos, float& OutScale) override;
};
