// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVDebugVisualizer.h"

#include "AdvancedPreviewScene.h"
#include "PVDebugVisualizationBase.h"
#include "PVDebugVisualizations.h"
#include "SPVEditorViewport.h"

#include "Components/TextRenderComponent.h"

#include "DataTypes/PVData.h"

#include "Materials/MaterialInterface.h"

#include "Visualizations/PVLineBatchComponent.h"

FVisualizerDrawContext::FVisualizerDrawContext(
	const FManagedArrayCollection& InCollection,
	const FPVVisualizationSettings& InSettings,
	FPCGSceneSetupParams& InSceneParams
)
	: Collection(InCollection)
	, VisualizationSettings(InSettings)
	, SceneSetupParams(InSceneParams)
{}

void FPVDebugVisualizer::DrawDebugVisualizations(
	const TArray<FPVVisualizationSettings>& VisualizationSettings,
	const FManagedArrayCollection& Collection,
	FPCGSceneSetupParams& InOutParams
)
{
	for (const FPVVisualizationSettings& VisualizationSetting : VisualizationSettings)
	{
		FVisualizerDrawContext Context(Collection, VisualizationSetting, InOutParams);
		if (const FPVDebugVisualizationPtr VisualizationPtr = CreateVisualizer(VisualizationSetting))
		{
			VisualizationPtr->Draw(Context);
		}
	}
}

void FPVDebugVisualizer::DrawDebugParams(const TArray<FPVParamDebuggerSettings>& ParamDebugVisualizationSettings, bool bAutoFocusLoopDebug, FPCGSceneSetupParams& InOutParams)
{
	TArray<FString> UniqueNames;
	UniqueNames.Reserve(ParamDebugVisualizationSettings.Num());
	for (const FPVParamDebuggerSettings& DebugVisualizationSetting : ParamDebugVisualizationSettings)
	{
		UniqueNames.AddUnique(DebugVisualizationSetting.ParamName.ToString());
	}

	TArray<TPair<FColor, FText>> LegendItems;
	LegendItems.Reserve(UniqueNames.Num());

	float Alpha = 0.0f;
	const float AlphaStep = 1.0f / static_cast<float>(UniqueNames.Num());
	TArray<FLinearColor> UniqueColors;
	for (const FString& UniqueName : UniqueNames) 
	{
		const FLinearColor Color = PV::Utilities::GetRandomHueColor(Alpha);
		UniqueColors.Add(Color);
		LegendItems.Emplace(Color.ToFColorSRGB(), FText::FromString(UniqueName));
		Alpha += AlphaStep;
	}
	
	UPVLineBatchComponent* LineBatchComponent = NewObject<UPVLineBatchComponent>();

	FBox BoundingBox(ForceInit);
	for (const FPVParamDebuggerSettings& ParamDebugVisualizationSetting : ParamDebugVisualizationSettings)
	{
		if (const int32 ColorIndex = UniqueNames.Find(ParamDebugVisualizationSetting.ParamName.ToString()); ColorIndex != INDEX_NONE)
		{
			const FLinearColor Color = UniqueColors[ColorIndex];
			VisualizeDebuggerParam(InOutParams, ParamDebugVisualizationSetting, LineBatchComponent, Color, BoundingBox);
		}
	}

	InOutParams.ManagedResources.Add(LineBatchComponent);
	InOutParams.Scene->AddComponent(LineBatchComponent, FTransform());

	const TSharedPtr<SPVEditorViewport> ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(
		InOutParams.EditorViewportClient->GetEditorViewportWidget()
	);
	ViewportWidget->PopulateLegendOverlayText(LegendItems);

	if (!BoundingBox.IsValid)
	{
		return;
	}

	if (bAutoFocusLoopDebug)
	{
		if (const TSharedPtr<FEditorViewportClient> ViewportClient = ViewportWidget->GetViewportClient())
		{
			ViewportClient->FocusViewportOnBox(BoundingBox.ExpandBy(0.25f));
		}
	}
}

void FPVDebugVisualizer::VisualizeDebuggerParam(
	FPCGSceneSetupParams& InOutParams,
	const FPVParamDebuggerSettings& Settings,
	UPVLineBatchComponent* LineBatcher,
	FLinearColor Color,
	FBox& BoundingBox
)
{
	constexpr float DefaultTextScale = 0.05f;
	constexpr ESceneDepthPriorityGroup DepthGroup = SDPG_Foreground;

	if (Settings.VisualizationMode == EPVDebugValueVisualizationMode::Point)
	{
		LineBatcher->AddPoint(Settings.Data.Pivot, 5, Color, DepthGroup);
		BoundingBox += Settings.Data.Pivot;
	}
	else if (Settings.VisualizationMode == EPVDebugValueVisualizationMode::Direction && Settings.Data.Direction.IsSet())
	{
		const FVector StartPoint = Settings.Data.Pivot;
		const FVector EndPoint = Settings.Data.Pivot + Settings.Data.Direction.GetValue();
		BoundingBox += StartPoint;
		// BoundingBox += EndPoint;
		const float ArrowSize = 0.0005f * Settings.Data.Direction->Length();
		{
			const FVector Dir = (EndPoint - StartPoint).GetUnsafeNormal();
			FVector Up = FVector::UpVector;
			FVector Right = Dir ^ Up;
			if (!Right.IsNormalized())
			{
				Dir.FindBestAxisVectors(Up, Right);
			}
			const FVector Origin = FVector::ZeroVector;
			FMatrix TM;
			TM.SetAxes(&Dir, &Right, &Up, &Origin);

			const float ArrowSqrt = FMath::Sqrt(ArrowSize);

			LineBatcher->AddLine(StartPoint, EndPoint, Color, DepthGroup);
			LineBatcher->AddLine(EndPoint, EndPoint + TM.TransformPosition(FVector(-ArrowSqrt, ArrowSqrt, 0)), Color, DepthGroup);
			LineBatcher->AddLine(EndPoint, EndPoint + TM.TransformPosition(FVector(-ArrowSqrt, -ArrowSqrt, 0)), Color, DepthGroup);
		}
	}
	else if (Settings.VisualizationMode == EPVDebugValueVisualizationMode::Vector && Settings.Data.Direction.IsSet())
	{
		LineBatcher->AddLine(
			Settings.Data.Pivot,
			Settings.Data.Direction.GetValue(),
			Color,
			DepthGroup
		);
		BoundingBox += Settings.Data.Pivot;
		// BoundingBox += Settings.Data.Direction.GetValue();
	}
	else if (Settings.VisualizationMode == EPVDebugValueVisualizationMode::Text && Settings.Data.TextData.IsSet())
	{
		const TObjectPtr<UTextRenderComponent> Component = NewObject<UTextRenderComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		Component->SetText(Settings.Data.TextData.GetValue());
		Component->SetTextRenderColor(Color.ToFColorSRGB());
		Component->SetWorldSize(DefaultTextScale);
		Component->SetHorizontalAlignment(EHTA_Center);
		Component->SetVerticalAlignment(EVRTA_TextCenter);
		Component->SetGenerateOverlapEvents(false);
		Component->SetMaterial(0, LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/LoopDebugTextMaterial.LoopDebugTextMaterial")));

		InOutParams.ManagedResources.Add(Component);
		InOutParams.Scene->AddComponent(Component, FTransform(Settings.Data.Pivot));
		BoundingBox += Settings.Data.Pivot;
	}
}

FPVDebugVisualizationPtr FPVDebugVisualizer::CreateVisualizer(const FPVVisualizationSettings& InSettings)
{
	switch (InSettings.DebugType)
	{
	case EPVDebugType::Point:
		return MakeShared<FPVPointDebugVisualization>();

	case EPVDebugType::Foliage:
		return MakeShared<FPVFoliageDebugVisualization>();

	case EPVDebugType::Branches:
		return MakeShared<FPVBranchDebugVisualization>();

	case EPVDebugType::Custom:
		return MakeShared<FPVCustomDebugVisualization>(InSettings.CustomPivotPositionAttributeName, InSettings.CustomPivotScaleAttributeName, InSettings.CustomGroupToFilter);

	default:
		return nullptr;
	}
}