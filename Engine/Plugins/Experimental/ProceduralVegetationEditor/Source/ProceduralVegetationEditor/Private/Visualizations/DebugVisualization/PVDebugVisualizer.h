// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Helpers/PVUtilities.h"

#include "PVDebugVisualizationBase.h"

#include "DataTypes/PVData.h"

struct FManagedArrayCollection;
struct FPVVisualizationSettings;
class FPVDebugVisualizationBase;
struct FPCGSceneSetupParams;

struct FVisualizerDrawContext
{
	FVisualizerDrawContext(const FManagedArrayCollection& InCollection, const FPVVisualizationSettings& InSettings, FPCGSceneSetupParams& InSceneParams);
	const FManagedArrayCollection& Collection;
	const FPVVisualizationSettings& VisualizationSettings;
	FPCGSceneSetupParams& SceneSetupParams;
};

class FPVDebugVisualizer
{
public:
	static void DrawDebugVisualizations(const TArray<FPVVisualizationSettings>& VisualizationSettings, const FManagedArrayCollection& Collection, FPCGSceneSetupParams& InOutParams);
	static void DrawDebugParams(const TArray<FPVParamDebuggerSettings>& ParamDebugVisualizationSettings, bool bAutoFocusLoopDebug, FPCGSceneSetupParams& InOutParams);

private:
	static void VisualizeDebuggerParam(
		FPCGSceneSetupParams& InOutParams,
		const FPVParamDebuggerSettings& Settings,
		UPVLineBatchComponent* LineBatcher,
		FLinearColor Color,
		FBox& BoundingBox
	);

	static FPVDebugVisualizationPtr CreateVisualizer(const FPVVisualizationSettings& InSettings);
};
