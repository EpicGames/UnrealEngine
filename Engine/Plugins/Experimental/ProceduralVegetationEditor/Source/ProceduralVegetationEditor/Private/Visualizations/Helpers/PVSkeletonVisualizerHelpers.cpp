// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSkeletonVisualizerHelpers.h"

#include "PVEditorCommon.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "Generators/SphereGenerator.h"

namespace PV::Visualizations
{
	FPVEdgeMeshGenerator::FPVEdgeMeshGenerator(UE::Geometry::FDynamicMesh3* InMesh)
		: Mesh(InMesh)
	{
		check(Mesh);

		if (!Mesh->HasAttributes())
		{
			Mesh->EnableAttributes();
		}
		UE::Geometry::FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
		if (!Attributes->HasPrimaryColors())
		{
			Attributes->EnablePrimaryColors();
		}
		ColorOverlay = Attributes->PrimaryColors();

		for (int32 Index = 0; Index < PV::EditorCommon::EdgeCylinderNumSides; ++Index)
		{
			const double Angle = UE_DOUBLE_TWO_PI * static_cast<double>(Index) / static_cast<double>(PV::EditorCommon::EdgeCylinderNumSides);
			const FVector Radial = FMath::Cos(Angle) * FVector::ForwardVector + FMath::Sin(Angle) * FVector::RightVector;
			CircleVertices.Add(Radial);
		}
	}

	FPVAppendMeshResult FPVEdgeMeshGenerator::AddToMesh(
		const FTransform& InBottomTransform,
		const FTransform& InTopTransform,
		FLinearColor InBottomColor,
		FLinearColor InTopColor
	)
	{
		const int32 NumSides = CircleVertices.Num();
		const int32 NumVertices = NumSides * 2;
		const int32 NumTriangles = NumSides * 2;

		FPVAppendMeshResult Result;
		Result.VertexIDs.SetNumUninitialized(NumVertices);
		Result.TriangleIDs.SetNumUninitialized(NumTriangles);

		TArray<int32, TInlineAllocator<16>> BottomVertices;
		TArray<int32, TInlineAllocator<16>> TopVertices;
		TArray<int32, TInlineAllocator<16>> BottomPointColors;
		TArray<int32, TInlineAllocator<16>> TopPointColors;

		BottomVertices.SetNumUninitialized(NumSides);
		TopVertices.SetNumUninitialized(NumSides);
		BottomPointColors.SetNumUninitialized(NumSides);
		TopPointColors.SetNumUninitialized(NumSides);

		for (int32 VtxID = 0; VtxID < NumSides; ++VtxID)
		{
			const FVector CircleVertex = CircleVertices[VtxID];

			const FVector BottomVertexPosition = InBottomTransform.TransformPosition(CircleVertex);
			const int32 BottomVID = Mesh->AppendVertex(BottomVertexPosition);
			BottomVertices[VtxID] = BottomVID;
			const FVector TopVertexPosition = InTopTransform.TransformPosition(CircleVertex);
			const int32 TopVID = Mesh->AppendVertex(TopVertexPosition);
			TopVertices[VtxID] = TopVID;

			BottomPointColors[VtxID] = ColorOverlay->AppendElement(InBottomColor);
			TopPointColors[VtxID] = ColorOverlay->AppendElement(InTopColor);

			Result.VertexIDs[VtxID * 2] = BottomVID;
			Result.VertexIDs[VtxID * 2 + 1] = TopVID;
		}

		for (int32 Index = 0; Index < NumSides; ++Index)
		{
			const int32 NextIndex = (Index + 1) % NumSides;
			const int32 Tri1 = Mesh->AppendTriangle(BottomVertices[Index], TopVertices[Index], BottomVertices[NextIndex]);
			const int32 Tri2 = Mesh->AppendTriangle(BottomVertices[NextIndex], TopVertices[Index], TopVertices[NextIndex]);

			if (Tri1 == IndexConstants::InvalidID || Tri2 == IndexConstants::InvalidID)
			{
				continue;
			}

			ColorOverlay->SetTriangle(
				Tri1,
				UE::Geometry::FIndex3i(BottomPointColors[Index], TopPointColors[Index], BottomPointColors[NextIndex])
			);
			ColorOverlay->SetTriangle(
				Tri2,
				UE::Geometry::FIndex3i(BottomPointColors[NextIndex], TopPointColors[Index], TopPointColors[NextIndex])
			);

			Result.TriangleIDs[Index * 2] = Tri1;
			Result.TriangleIDs[Index * 2 + 1] = Tri2;
		}

		return Result;
	}
}
