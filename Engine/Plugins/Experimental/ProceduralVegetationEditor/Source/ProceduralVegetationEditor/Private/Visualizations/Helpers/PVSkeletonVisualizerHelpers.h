// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Utils/PVDynamicMeshVertexAttribute.h"

namespace UE::Geometry
{
	class FDynamicMesh3;
	struct FIndex3i;
}

namespace PV::Visualizations
{
	/**
	 * @brief Holder struct for PVMeshGenerators Vertices and Indices IDs.
	 * Returned from MeshGenerators AddToMesh method
	 */
	struct FPVAppendMeshResult
	{
		TArray<int32, TInlineAllocator<16>> VertexIDs;
		TArray<int32, TInlineAllocator<32>> TriangleIDs;
	};

	struct FPVEdgeMeshGenerator
	{
		FPVEdgeMeshGenerator(UE::Geometry::FDynamicMesh3* InMesh);
		~FPVEdgeMeshGenerator() = default;

		/**
		 * @param InBottomTransform Transform of the bottom of the edge mesh in the dynamic mesh
		 * @param InTopTransform Transform of the top of the edge mesh in the dynamic mesh
		 * @param InBottomColor Vertex Color to add to the bottom edge vertices, given the mesh has color overlay enabled
		 * @param InTopColor Vertex Color to add to the top edge vertices, given the mesh has color overlay enabled
		 * 
		 * @returns FPVAppendMeshResult with all the added vertices and indices added to the InMesh
		 */
		FPVAppendMeshResult AddToMesh(
			const FTransform& InBottomTransform,
			const FTransform& InTopTransform,
			FLinearColor InBottomColor = FLinearColor::White,
			FLinearColor InTopColor = FLinearColor::White
		);

	private:
		UE::Geometry::FDynamicMesh3* Mesh = nullptr;
		
		UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay =  nullptr;
		
		TArray<FVector> CircleVertices;
	};
}
