// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVBoneComponent.h"
#include "MeshElementCollector.h"
#include "SkeletalDebugRendering.h"
#include "Facades/PVBoneFacade.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"

FPVBoneSceneProxy::FPVBoneSceneProxy(const UPVBoneComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, BoneComponent(InComponent)
{}

FPrimitiveViewRelevance FPVBoneSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance ViewRelevance;
	ViewRelevance.bDrawRelevance = IsShown(View);
	ViewRelevance.bDynamicRelevance = true;
	ViewRelevance.bSeparateTranslucency = ViewRelevance.bNormalTranslucency = true;
	return ViewRelevance;
}

void FPVBoneSceneProxy::GetDynamicMeshElements(
	const TArray<const FSceneView*>& Views,
	const FSceneViewFamily& ViewFamily,
	uint32 VisibilityMap,
	FMeshElementCollector& Collector
) const
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

#if UE_ENABLE_DEBUG_DRAWING
			FPrimitiveDrawInterface* PDI = Collector.GetDebugPDI(ViewIndex);
#else	
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
#endif

			BoneComponent->Draw(View, PDI);
		}
	}
}

SIZE_T FPVBoneSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

uint32 FPVBoneSceneProxy::GetMemoryFootprint() const { return sizeof *this + GetAllocatedSize(); }

bool FPVBoneSceneProxy::CanBeOccluded() const
{
	return false;
}

UPVBoneComponent::UPVBoneComponent()
{
	bAutoActivate = true;
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
	SetIgnoreStreamingManagerUpdate(true);
	
	BBox = FBox(ForceInit);
	Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector::OneVector, 1);
}

void UPVBoneComponent::InitBounds()
{
	BBox = FBox(ForceInit);
}

void UPVBoneComponent::AddBone(const FVector& InStartPos, const FVector& InEndPos, const FLinearColor& InColor)
{
	Bones.Emplace(InStartPos, InEndPos, InColor);

	BBox += InStartPos;
	BBox += InEndPos;
}

void UPVBoneComponent::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	for (const FPVBoneInfo& Bone : Bones)
	{
		SkeletalDebugRendering::DrawWireBone(PDI,Bone.StartPos, Bone.EndPos, Bone.Color,SDPG_MAX,2);
	}
}

FPrimitiveSceneProxy* UPVBoneComponent::CreateSceneProxy()
{
	FPVBoneSceneProxy* Proxy = new FPVBoneSceneProxy(this);
	return Proxy;
}

FBoxSphereBounds UPVBoneComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(BBox).TransformBy(LocalToWorld); 
}

void UPVBoneComponent::SetCollection(const FManagedArrayCollection* const InSkeletonCollection)
{
	if (InSkeletonCollection == nullptr)
	{
		return;
	}

	Bones.Empty();
	InitBounds();
	
	const PV::Facades::FBoneFacade BoneFacade = PV::Facades::FBoneFacade(*InSkeletonCollection);
	TArray<PV::Facades::FBoneNode> BoneNodes = BoneFacade.GetBoneDataFromCollection();

	for (int i=0; i < BoneNodes.Num(); i++)
	{
		PV::Facades::FBoneNode* ParentNode = BoneNodes.FindByPredicate([&](const PV::Facades::FBoneNode& BoneNode)
		{
			return BoneNode.BoneIndex == BoneNodes[i].ParentBoneIndex;
		});

		FVector StartPos = ParentNode ? ParentNode->AbsolutePosition : BoneNodes[i].AbsolutePosition;
		AddBone(StartPos,BoneNodes[i].AbsolutePosition, SkeletalDebugRendering::GetSemiRandomColorForBone(i));
	}
}

int UPVBoneComponent::GetBoneCount()
{
	return Bones.Num();
}
