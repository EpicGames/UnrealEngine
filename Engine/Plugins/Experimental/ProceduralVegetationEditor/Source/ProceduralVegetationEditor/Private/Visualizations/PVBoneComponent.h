// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "SceneView.h"
#include "Components/PrimitiveComponent.h"
#include "PVBoneComponent.generated.h"

struct FManagedArrayCollection;
class UPVBoneComponent;

class FPVBoneSceneProxy final : public FPrimitiveSceneProxy
{
public:
	FPVBoneSceneProxy(const UPVBoneComponent* InComponent);

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual void GetDynamicMeshElements(
		const TArray<const FSceneView*>& Views,
		const FSceneViewFamily& ViewFamily,
		uint32 VisibilityMap,
		FMeshElementCollector& Collector
	) const override;

	virtual SIZE_T GetTypeHash() const override;
	virtual uint32 GetMemoryFootprint(void) const override;

	virtual bool CanBeOccluded() const override;
private:
	const TObjectPtr<const UPVBoneComponent> BoneComponent;
};

USTRUCT()
struct FPVBoneInfo
{
	GENERATED_BODY()

	FVector StartPos;
	FVector EndPos;
	FLinearColor Color;
};

UCLASS(Blueprintable, BlueprintType, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPVBoneComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	UPVBoneComponent();

	void InitBounds();
	void AddBone(const FVector& InStartPos, const FVector& InEndPos, const FLinearColor& InColor = FLinearColor::White);

	void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) const;

	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	void SetCollection(const FManagedArrayCollection* const InSkeletonCollection);
	int GetBoneCount();

protected:
	FBox BBox;
	TArray<FPVBoneInfo> Bones;
	const FManagedArrayCollection* SkeletonCollection;
};
