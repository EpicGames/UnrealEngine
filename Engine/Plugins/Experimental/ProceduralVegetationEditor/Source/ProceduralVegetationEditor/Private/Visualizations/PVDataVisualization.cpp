// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "PVDataVisualization.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "GeometryCollectionToDynamicMesh.h"
#include "PCGComponent.h"
#include "PCGDataVisualization.h"
#include "PlanarCut.h"
#include "PVBoneComponent.h"
#include "SPVEditorViewport.h"

#include "Algo/Count.h"
#include "Algo/Unique.h"

#include "Components/DynamicMeshComponent.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/LineBatchComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/TextRenderComponent.h"

#include "DataAssets/ProceduralVegetationPlantProfileDataAsset.h"

#include "DataTypes/PVData.h"
#include "DataTypes/PVGrafterPaletteData.h"
#include "DataTypes/PVPlantProfileData.h"
#include "DataTypes/PVTrunkTextureSetupData.h"

#include "DebugVisualization/PVDebugVisualizer.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#include "Facades/PVBranchFacade.h"
#include "Facades/PVFoliageFacade.h"
#include "Facades/PVPointFacade.h"

#include "GeometryCollection/GeometryCollectionObject.h"

#include "Helpers/PVUtilities.h"

#include "Implementations/PVFoliage.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Nodes/PVBaseSettings.h"
#include "Nodes/PVPlantProfileLoaderSettings.h"

#include "Rendering/SkeletalMeshRenderData.h"

#include "UObject/Package.h"

#include "Utils/PVAttributes.h"

#include "Visualizations/PVSkeletonVisualizerComponent.h"

#define LOCTEXT_NAMESPACE "PCGManagedArrayCollectionDataVisualization"
 
class UPVBoneComponent;
 
DEFINE_LOG_CATEGORY(LogProceduralVegetationDataVisualization);
 
FPVDataVisualization::FPVDataVisualization()
{
	RegisterVisualizations();
}
 
void FPVDataVisualization::RegisterVisualizations()
{
	RenderMap.Add(EPVRenderType::PointData, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::SkeletonRenderer));
	RenderMap.Add(EPVRenderType::Mesh, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::MeshRenderer));
	RenderMap.Add(EPVRenderType::Foliage, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::FoliageRenderer));
	RenderMap.Add(EPVRenderType::Bones, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::BonesRenderer));
	RenderMap.Add(EPVRenderType::FoliageGrid, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::FoliageGridRenderer));
	RenderMap.Add(EPVRenderType::FoliageAttachments, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::FoliageAttachmentPointsRenderer));
	RenderMap.Add(EPVRenderType::Seed, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::SeedRenderer));
	RenderMap.Add(EPVRenderType::Leaf, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::GrowerLeavesRenderer));
	
	RenderMap.Add(EPVRenderType::Textures, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::TexturesRenderer));
	RenderMap.Add(EPVRenderType::GrafterGrid, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::GrafterPaletteGridRenderer));
	RenderMap.Add(EPVRenderType::PlantProfile, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::PlantProfileRenderer));
	
	RenderMap.Add(EPVRenderType::BoundingBoxOnly, FPVRenderCallback::CreateRaw(this, &FPVDataVisualization::SetupBoundingBoxOnly));
}
 
void FPVDataVisualization::ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data,
                                                         AActor* TargetActor) const
{
	return;
}

FPCGSetupSceneFunc FPVDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return [this, WeakData=TWeakObjectPtr<const UPVData>(Cast<UPVData>(Data)), WeakSettings = TWeakObjectPtr<const UPCGSettingsInterface>(SettingsInterface)](
		FPCGSceneSetupParams& InOutParams)
	{
		const IPVRenderSettings* RenderSettings = nullptr;

		if (WeakSettings.IsValid() && WeakSettings->Implements<UPVRenderSettings>())
		{
			RenderSettings = Cast<IPVRenderSettings>(WeakSettings);
		}
	
		check(InOutParams.Scene)
			
		if (!WeakData.IsValid() || RenderSettings == nullptr)
		{
			UE_LOGF(LogProceduralVegetationDataVisualization, Error, "Failed to setup data viewport, the data was lost or invalid.");
			return;
		}
 
		InOutParams.Scene->SetFloorVisibility(true);
		InOutParams.Scene->SetFloorOffset(1.0f);
 
		InOutParams.EditorViewportClient->OverrideNearClipPlane(0.001f);
 
		// Reapply view mode settings as for some reason SetFloorVisibility changes EngineFlags
		InOutParams.EditorViewportClient->SetViewMode(InOutParams.EditorViewportClient->GetViewMode());
		InOutParams.EditorViewportClient->SetRealtime(true);

		if (const auto ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(InOutParams.EditorViewportClient->GetEditorViewportWidget()))
		{
			const ELevelViewportType PreferredViewportType = RenderSettings->GetPreferredViewportType();
			InOutParams.PreferredViewportType = PreferredViewportType != ELevelViewportType::LVT_None
				? TOptional<ELevelViewportType>(PreferredViewportType)
				: ViewportWidget->OverriddenViewportType;

			if (PreferredViewportType != ELevelViewportType::LVT_None && !ViewportWidget->OverriddenViewportType.IsSet())
			{
				ViewportWidget->OverriddenViewportType = InOutParams.EditorViewportClient->ViewportType;
			}
			else if (PreferredViewportType == ELevelViewportType::LVT_None)
			{
				ViewportWidget->OverriddenViewportType.Reset();
			}
		}
		
		if (InOutParams.Scene->IsEnvironmentEnabled())
		{
			InOutParams.Scene->HandleToggleEnvironment();
		}
		if (!InOutParams.Scene->IsPostProcessingEnabled())
		{
			InOutParams.Scene->HandleTogglePostProcessing();
		}
 
		InOutParams.Scene->GetLineBatcher()->Flush();

		FBoxSphereBounds Bounds(ForceInit);

		const bool bIsCollectionRenderingEnabled = RenderSettings->IsCollectionRenderingEnabled();
		const bool bIsVisualizationCollectionsRenderingEnabled = RenderSettings->IsVisualizationCollectionsRenderingEnabled();

		if (bIsCollectionRenderingEnabled || bIsVisualizationCollectionsRenderingEnabled)
		{
			for (const EPVRenderType& RenderType : RenderSettings->GetCurrentRenderTypes())
			{
				const FPVRenderCallback* RenderCallback = RenderMap.Find(RenderType);
				if (!ensure(RenderCallback != nullptr))
				{
					continue;
				}

				if (bIsCollectionRenderingEnabled)
				{
					RenderCallback->Execute(WeakData.Get(), WeakData->GetCollection(), InOutParams, Bounds, WeakSettings.Get());
				}

				if (bIsVisualizationCollectionsRenderingEnabled)
				{
					for (const FPVVisualizationCollection& VisualizationCollection : WeakData->GetVisualizationCollections())
					{
						RenderCallback->Execute(WeakData.Get(), VisualizationCollection.Collection, InOutParams, Bounds, WeakSettings.Get());
					}
				}
			}
		}

		for (const auto& [MeshObject, Transform] : RenderSettings->GetViewportObjects())
		{
			if (UStaticMesh* Mesh = Cast<UStaticMesh>(MeshObject))
			{
				TObjectPtr<UStaticMeshComponent> StaticMeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
				StaticMeshComponent->SetStaticMesh(Mesh);
				StaticMeshComponent->UpdateBounds();
		
				InOutParams.ManagedResources.Add(StaticMeshComponent);
				InOutParams.Scene->AddComponent(StaticMeshComponent, Transform);
			}
		}

		if (PV::Utilities::DebugModeEnabled() && RenderSettings->IsDebug())
		{
			FPVDebugVisualizer::DrawDebugVisualizations(WeakData->GetDebugSettings().VisualizationSettings, WeakData->GetCollection(), InOutParams);
			FPVDebugVisualizer::DrawDebugParams(WeakData->GetDebugSettings().ParamDebugVisualizationSettings, WeakData->GetDebugSettings().bAutoFocusLoopDebug, InOutParams);
		}

		for (const FPVVisualizationCollection& VisualizationCollection : WeakData->GetVisualizationCollections())
		{
			FPVDebugVisualizer::DrawDebugVisualizations(VisualizationCollection.VisualizationSettings, VisualizationCollection.Collection, InOutParams);
		}
 
		InOutParams.FocusBounds = FBoxSphereBounds(Bounds.Origin, Bounds.BoxExtent, Bounds.SphereRadius);
	};
}
 
void FPVDataVisualization::MeshRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
                                                  FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDataVisualization::MeshRenderer);
 
	TArray<UMaterialInterface*> Materials;
	
	if (InCollection.HasAttribute("MaterialPath", FGeometryCollection::MaterialGroup))
	{
		const TManagedArray<FString>& MaterialPaths = InCollection.GetAttribute<FString>("MaterialPath", FGeometryCollection::MaterialGroup);
		for (const FString& MaterialPath : MaterialPaths.GetConstArray())
		{
			Materials.Add(LoadObject<UMaterialInterface>(nullptr, MaterialPath, {}, LOAD_NoWarn | LOAD_Quiet));
		}
	}
 
	UE::Geometry::FGeometryCollectionToDynamicMeshes Converter;
	Converter.Init(InCollection, {});
	for (const auto& MeshInfo : Converter.Meshes)
	{
		FDynamicMesh3* DynamicMesh = MeshInfo.Mesh.Get();
		if (DynamicMesh->VertexCount())
		{
			TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = NewObject<UDynamicMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			DynamicMeshComponent->SetMesh(MoveTemp(*DynamicMesh));
			DynamicMeshComponent->SetTangentsType(EDynamicMeshComponentTangentsMode::AutoCalculated);
			DynamicMeshComponent->ConfigureMaterialSet(Materials);
			DynamicMeshComponent->UpdateBounds();
 
			InOutParams.ManagedResources.Add(DynamicMeshComponent);
			InOutParams.Scene->AddComponent(DynamicMeshComponent, FTransform::Identity);
 
			OutBounds = OutBounds + DynamicMeshComponent->CalcLocalBounds();
		}
	}
}

void FPVDataVisualization::SkeletonRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
                                                      FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDataVisualization::SkeletonRenderer);

	if (!PV::Utilities::IsValidGrowthData(InCollection))
	{
		return;
	}
 
	const TObjectPtr<UPVSkeletonVisualizerComponent> VisualizerComponent = NewObject<UPVSkeletonVisualizerComponent>(
		GetTransientPackage(), NAME_None, RF_Transient);

	{
		InOutParams.ManagedResources.Add(VisualizerComponent);
		InOutParams.Scene->AddComponent(VisualizerComponent, FTransform::Identity);
		InOutParams.ManagedResources.Add(VisualizerComponent->GetDynamicMeshComponent());
		InOutParams.Scene->AddComponent(VisualizerComponent->GetDynamicMeshComponent(), FTransform::Identity);
		InOutParams.ManagedResources.Add(VisualizerComponent->GetPointMeshInstancerComponent());
		InOutParams.Scene->AddComponent(VisualizerComponent->GetPointMeshInstancerComponent(), FTransform::Identity);
	}
	
	VisualizerComponent->SetCollection(&InCollection);
	VisualizerComponent->UpdateBounds();

	const TSharedPtr<SPVEditorViewport> ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(
		InOutParams.EditorViewportClient->GetEditorViewportWidget());
	
	ViewportWidget->SetSkeletonVisualizerComponent(VisualizerComponent);
 
	OutBounds = OutBounds + VisualizerComponent->CalcLocalBounds();
}
 
void FPVDataVisualization::FoliageRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
                                                     FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDataVisualization::FoliageRenderer);
 
	PV::Facades::FFoliageFacade FoliageFacade(InCollection);
	const int32 NumInstances = FoliageFacade.NumFoliageEntries();
	if (NumInstances <= 0)
	{
		return;
	}

	auto OnComponentCreated = [&InOutParams](UMeshComponent* InInstancedComponent)
	{
		InOutParams.ManagedResources.Add(InInstancedComponent);
		InOutParams.Scene->AddComponent(InInstancedComponent, FTransform::Identity);
	};
 
	TMap<FString, TObjectPtr<UMeshComponent>> InstancedComponentMap;
	InstancedComponentMap.Reserve(FoliageFacade.NumFoliageInfo());

	for (int32 FoliageNameIndex = 0; FoliageNameIndex < FoliageFacade.NumFoliageInfo(); ++FoliageNameIndex)
	{
		const FPVFoliageInfo FoliageInfo = FoliageFacade.GetFoliageInfo(FoliageNameIndex);
		const FString FoliageName = FoliageInfo.Mesh.ToString();
		const FSoftObjectPath MeshPath(FoliageName);

		UStaticMesh* StaticFoliageMesh = PV::Utilities::PackageExists(MeshPath.GetLongPackageName(), UStaticMesh::StaticClass()) 
			? LoadObject<UStaticMesh>(nullptr, *FoliageName, {}, LOAD_NoWarn | LOAD_Quiet) 
			: nullptr;
		USkeletalMesh* SkeletalFoliageMesh = PV::Utilities::PackageExists(MeshPath.GetLongPackageName(), USkeletalMesh::StaticClass()) 
			? LoadObject<USkeletalMesh>(nullptr, *FoliageName, {}, LOAD_NoWarn | LOAD_Quiet) 
			: nullptr;

		if (StaticFoliageMesh)
		{
			UPackage* ParentPackage = GetTransientPackage();
			const FName ObjectName = MakeUniqueObjectName(ParentPackage, UInstancedStaticMeshComponent::StaticClass(), StaticFoliageMesh->GetFName());
			UInstancedStaticMeshComponent* InstancedComponent = NewObject<UInstancedStaticMeshComponent>(ParentPackage, ObjectName);
			InstancedComponent->SetStaticMesh(StaticFoliageMesh);
			InstancedComponentMap.Add(FoliageName, InstancedComponent);
			OnComponentCreated(InstancedComponent);
		}
		else if (SkeletalFoliageMesh)
		{
			if (!SkeletalFoliageMesh->NaniteSettings.bEnabled)
			{
				SkeletalFoliageMesh->NaniteSettings.bEnabled = true;
				SkeletalFoliageMesh->Build();
				SkeletalFoliageMesh->MarkPackageDirty();
				UE_LOGF(LogProceduralVegetationDataVisualization, Warning, "Enabling Nanite for the Skeletal Mesh {%ls}, its required for InstancedSkinnedMeshComponent", *FoliageName);
			}

			UPackage* ParentPackage = GetTransientPackage();
			const FName ObjectName = MakeUniqueObjectName(ParentPackage, UInstancedSkinnedMeshComponent::StaticClass(), SkeletalFoliageMesh->GetFName());
			UInstancedSkinnedMeshComponent* InstancedComponent = NewObject<UInstancedSkinnedMeshComponent>(ParentPackage, ObjectName);
			InstancedComponent->SetSkinnedAssetAndUpdate(SkeletalFoliageMesh);
			InstancedComponentMap.Add(FoliageName, InstancedComponent);
			OnComponentCreated(InstancedComponent);
		}
	}

	TMap<FString, TArray<FTransform>> InstanceTransformsMap;
	InstanceTransformsMap.Reserve(NumInstances);

	for (int32 Id = 0; Id < NumInstances; Id++)
	{
		const PV::Facades::FFoliageEntryData FoliageEntry = FoliageFacade.GetFoliageEntry(Id);
		const FPVFoliageInfo FoliageInfo = FoliageFacade.GetFoliageInfo(FoliageEntry.NameId);
		const FString FoliageName = FoliageInfo.Mesh.ToString();
		const FTransform Transform = FoliageFacade.GetFoliageTransform(Id);

		InstanceTransformsMap.FindOrAdd(FoliageName).Add(Transform);
	}

	for (const auto& [FoliageName, Transforms] : InstanceTransformsMap)
	{
		if (Transforms.Num() <= 0)
		{
			continue;
		}

		auto* InstancedComponent = InstancedComponentMap.Find(FoliageName);
		if (!InstancedComponent)
		{
			continue;
		}

		if (TObjectPtr<UInstancedStaticMeshComponent> StaticMeshInstancedComponent = Cast<UInstancedStaticMeshComponent>(*InstancedComponent))
		{
			StaticMeshInstancedComponent->AddInstances(Transforms, false);
		}
		else if (TObjectPtr<UInstancedSkinnedMeshComponent> SkinnedInstancedComponent = Cast<UInstancedSkinnedMeshComponent>(*InstancedComponent))
		{
			TArray<int32> BankIndices;
			BankIndices.SetNum(Transforms.Num());
			SkinnedInstancedComponent->AddInstances(Transforms, BankIndices, false);
		}
	}
 
	const TSharedPtr<SPVEditorViewport> ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(
			InOutParams.EditorViewportClient->GetEditorViewportWidget());
	for (const auto& [FoliageName, Component] : InstancedComponentMap)
	{
		ViewportWidget->AddInstancedFoliageComponent(Component);
		OutBounds = OutBounds + Component->CalcLocalBounds();
	}
}
 
void FPVDataVisualization::BonesRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDataVisualization::BonesRenderer);
 
	const TObjectPtr<UPVBoneComponent> BoneComponent = NewObject<UPVBoneComponent>(
		GetTransientPackage(), NAME_None, RF_Transient);
	BoneComponent->SetCollection(&InCollection);
	BoneComponent->UpdateBounds();
 
	InOutParams.ManagedResources.Add(BoneComponent);
	InOutParams.Scene->AddComponent(BoneComponent, FTransform::Identity);
 
	FString InTextToDraw = FString::Format(TEXT("{0} Bones"),{BoneComponent->GetBoneCount()});
	const FVector3f& Pos = FVector3f::ForwardVector * 100;
 
	DrawDebugString(BoneComponent->GetWorld(), FVector::Zero(), InTextToDraw);
	
	TObjectPtr<UTextRenderComponent> Component = NewObject<UTextRenderComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	Component->SetText(FText::FromString(InTextToDraw));
	Component->SetTextRenderColor(FColor::Blue);
	Component->SetWorldSize(40);
	Component->SetGenerateOverlapEvents(false);
						
	InOutParams.ManagedResources.Add(Component);
	FTransform TextTransform = FTransform(FVector(Pos.X, Pos.Y, Pos.Z));
	FRotator TextRotator = FRotator(0,90,0);
	TextTransform.SetRotation(TextRotator.Quaternion());
	InOutParams.Scene->AddComponent(Component, TextTransform);
 
	OutBounds = OutBounds + BoneComponent->CalcLocalBounds();
}
 
TPair<int32, int32> GetMeshTrisAndVerts(const UObject* FoliageObject)
{
	int32 NumTriangles = 0;
	int32 NumVertices = 0;
	if (const UStaticMesh* StaticFoliageMesh = Cast<UStaticMesh>(FoliageObject))
	{
		NumTriangles = StaticFoliageMesh->GetNumTriangles(0);
		NumVertices = StaticFoliageMesh->GetNumVertices(0);
	}
	else if (const USkeletalMesh* SkeletalFoliageMesh = Cast<USkeletalMesh>(FoliageObject))
	{
		FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalFoliageMesh->GetResourceForRendering();
		if (SkelMeshRenderData && SkelMeshRenderData->LODRenderData.Num() > 0)
		{
			NumTriangles = SkelMeshRenderData->LODRenderData[0].GetTotalFaces();
			NumVertices = SkelMeshRenderData->LODRenderData[0].GetNumVertices();
		}
	}
	return {NumTriangles, NumVertices};
}
 
void FPVDataVisualization::FoliageGridRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	const PV::Facades::FFoliageFacade FoliageFacade(InCollection);

	const TArray<FPVFoliageInfo> FoliageInfos = FoliageFacade.GetFoliageInfos();

	FBox MaxFoliageBounds(ForceInit);
	int32 ValidFoliageEntries = 0;

	static const auto GetMeshBounds = [](auto* Mesh) -> FBox
		{
			return Mesh ? Mesh->GetBounds().GetBox() : FBox(ForceInit);
		};

	const auto AddComponentToScene = [&](USceneComponent* ComponentRef, const FTransform& Transform, const bool UpdateBounds = true)
		{
			InOutParams.ManagedResources.Add(ComponentRef);
			InOutParams.Scene->AddComponent(ComponentRef, Transform);

			if (UpdateBounds)
			{
				ComponentRef->UpdateBounds();
				OutBounds = OutBounds + ComponentRef->CalcBounds(Transform);
			}
		};

	for (const FPVFoliageInfo& Info : FoliageInfos)
	{
		if (UObject* FoliageMesh = LoadObject<UObject>(nullptr, *Info.Mesh.ToString(), {}, LOAD_NoWarn | LOAD_Quiet))
		{
			FBox Bounds = GetMeshBounds(Cast<UStaticMesh>(FoliageMesh));
			if (!Bounds.IsValid)
			{
				Bounds = GetMeshBounds(Cast<USkeletalMesh>(FoliageMesh));
			}
			MaxFoliageBounds = Bounds.IsValid && Bounds.GetVolume() > MaxFoliageBounds.GetVolume() ? Bounds : MaxFoliageBounds;
			ValidFoliageEntries++;
		}
	}

	if (ValidFoliageEntries == 0)
	{
		return;
	}

	const int32 GridSize = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(ValidFoliageEntries)));

	constexpr float CellPadding = 100.0f;
	constexpr int32 MaxCharsPerLine = 25;
	constexpr float CharAspectRatio = 0.6f;
	constexpr int32 NumAttrLines = 9;
	const float CellSize = MaxFoliageBounds.GetSize().GetMax() + CellPadding;
	const float TextWorldSize = CellSize / (MaxCharsPerLine * CharAspectRatio);
	const float TextGap = TextWorldSize * 1.0f;
	const float AttrTextHeight = TextWorldSize * NumAttrLines;
	const float MaxMeshHeight = MaxFoliageBounds.GetSize().Z;

	// Z origin for the text layout. Texts reference MaxFoliageBounds.Min/Max.Z from this base,
	// so the attribute text bottom lands at TextGap above the ground plane.
	const float CellOffsetZ = -MaxFoliageBounds.Min.Z + AttrTextHeight + 2.0f * TextGap;

	// Bottom of the mesh zone in world Z (shared by all entries, derived from the tallest mesh).
	const float MeshZoneBottomZ = AttrTextHeight + 2.0f * TextGap;

	UStaticMesh* TextBackgroundMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Plane"));
	UMaterial* BackgroundMat = LoadObject<UMaterial>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/TextBackgroundMat.TextBackgroundMat"));
	UMaterialInterface* TextMat = LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/GridVisualizationTextMaterial_Inst.GridVisualizationTextMaterial_Inst"));

	const float StartX = -(GridSize - 1) * CellSize / 2.0f;
	const float StartY = -(GridSize - 1) * CellSize / 2.0f;

	static const auto WrapText = [](const FString& InText, const int32 MaxChars) -> FString
		{
			if (InText.Len() <= MaxChars)
			{
				return InText;
			}
			FString Result;
			for (int32 Start = 0; Start < InText.Len(); Start += MaxChars)
			{
				if (!Result.IsEmpty())
				{
					Result += TEXT("\n");
				}
				Result += InText.Mid(Start, MaxChars);
			}
			return Result;
		};

	const auto AddTextLabel = [&](const FString& Text, const FVector& BaseRefPos, const bool bAboveMesh)
		{
			UTextRenderComponent* TextComponent = NewObject<UTextRenderComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			TextComponent->SetMaterial(0, TextMat);
			TextComponent->SetText(FText::FromString(Text));
			TextComponent->SetTextRenderColor(FColor::Black);
			TextComponent->SetWorldSize(TextWorldSize);
			TextComponent->SetVerticalAlignment(EVRTA_TextTop);
			TextComponent->SetGenerateOverlapEvents(false);

			FBoxSphereBounds TextBounds = TextComponent->CalcLocalBounds();
			const float TextHeight = 2.0f * TextBounds.BoxExtent.Z;

			const FVector Anchor = bAboveMesh
				? BaseRefPos + FVector(0.0f, 0.0f, TextHeight)
				: BaseRefPos;

			const FVector TextPos = Anchor + TextBounds.BoxExtent.Y * FVector::BackwardVector;
			AddComponentToScene(TextComponent, FTransform(FRotator(0, 90, 0), TextPos), false);

			TextBounds = TextBounds.TransformBy(TextComponent->GetComponentTransform());

			if (TextBackgroundMesh && BackgroundMat)
			{
				UStaticMeshComponent* BgMesh = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
				BgMesh->SetStaticMesh(TextBackgroundMesh);
				BgMesh->SetMaterial(0, BackgroundMat);
				AddComponentToScene(
					BgMesh,
					FTransform(
						FRotator(0, 0, 90),
						TextBounds.Origin + FVector::LeftVector * 0.01f,
						FVector(TextBounds.BoxExtent.X / 40.0f, TextBounds.BoxExtent.Z / 40.0f, 1)
					),
					false
				);
			}
		};

	int32 FoliageIndex = 0;
	for (const FPVFoliageInfo& Info : FoliageInfos)
	{
		UObject* FoliageObject = LoadObject<UObject>(nullptr, *Info.Mesh.ToString(), {}, LOAD_NoWarn | LOAD_Quiet);
		if (!FoliageObject)
		{
			continue;
		}

		const int32 Col = FoliageIndex % GridSize;
		const int32 Row = FoliageIndex / GridSize;

		// CellCenter.Z is used only for text positioning (references MaxFoliageBounds extents).
		const FVector CellCenter(StartX + Col * CellSize, StartY + Row * CellSize, CellOffsetZ);

		// Per-mesh: center within the mesh zone by computing how much smaller this mesh is
		// than the tallest one, then lifting it by half that difference.
		FBox IndividualBounds = GetMeshBounds(Cast<UStaticMesh>(FoliageObject));
		if (!IndividualBounds.IsValid)
		{
			IndividualBounds = GetMeshBounds(Cast<USkeletalMesh>(FoliageObject));
		}

		const float MeshHeight = IndividualBounds.GetSize().Z;
		const float CenteringOffset = (MaxMeshHeight - MeshHeight) / 2.0f;
		// Place the mesh so its bounding box bottom sits at MeshZoneBottomZ + CenteringOffset.
		const float MeshPivotZ = MeshZoneBottomZ + CenteringOffset - IndividualBounds.Min.Z;
		const FVector MeshPosition(CellCenter.X, CellCenter.Y, MeshPivotZ);

		if (UStaticMesh* StaticFoliageMesh = Cast<UStaticMesh>(FoliageObject))
		{
			UStaticMeshComponent* MeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			MeshComponent->SetStaticMesh(StaticFoliageMesh);
			AddComponentToScene(MeshComponent, FTransform(MeshPosition));
		}
		else if (USkeletalMesh* SkeletalFoliageMesh = Cast<USkeletalMesh>(FoliageObject))
		{
			USkeletalMeshComponent* MeshComponent = NewObject<USkeletalMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			MeshComponent->SetSkeletalMesh(SkeletalFoliageMesh);
			AddComponentToScene(MeshComponent, FTransform(MeshPosition));
		}

		// Name at top: text bottom sits TextGap above the mesh zone top (MaxFoliageBounds.Max.Z from CellCenter).
		const FString WrappedName = WrapText(FoliageObject->GetName(), MaxCharsPerLine);
		const FVector NameBaseRef = CellCenter + FVector(0.0f, 0.0f, MaxFoliageBounds.Max.Z + TextGap);
		AddTextLabel(WrappedName, NameBaseRef, true);

		// Attributes at bottom: text top sits TextGap below the mesh zone bottom (MaxFoliageBounds.Min.Z from CellCenter).
		const auto [Tris, Verts] = GetMeshTrisAndVerts(FoliageObject);
		const FPVDistributionConditions& Attrs = Info.Attributes;
		const FString BottomText = FString::Printf(
			TEXT("Tris: %d\nVerts: %d\nLight: %.2f\nScale: %.2f\nUpAlign: %.2f\nHealth: %.2f\nTip: %s\nHeight: %.2f\nGeneration: %.2f"),
			Tris, Verts,
			Attrs.Light, Attrs.Scale, Attrs.UpAlignment,
			Attrs.Health, Attrs.Tip ? TEXT("Yes") : TEXT("No"),
			Attrs.Height, Attrs.Generation);
		const FVector AttrBaseRef = CellCenter + FVector(0.0f, 0.0f, MaxFoliageBounds.Min.Z - TextGap);
		AddTextLabel(BottomText, AttrBaseRef, false);

		FoliageIndex++;
	}
}

void FPVDataVisualization::FoliageAttachmentPointsRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	const PV::Facades::FFoliageFacade FoliageFacade(InCollection);
 
	const TObjectPtr<UPVLineBatchComponent> LineBatchComponent = NewObject<UPVLineBatchComponent>(GetTransientPackage(), NAME_None, RF_Transient);
 
	const TObjectPtr<UInstancedStaticMeshComponent> SphereInstancer = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	SphereInstancer->SetStaticMesh(LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Sphere.Sphere")));
	SphereInstancer->SetMaterial(0, LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/FoliageAttachmentPointMaterial.FoliageAttachmentPointMaterial")));
	SphereInstancer->SetNumCustomDataFloats(3);
 
	TArray<TPair<FColor, FText>> LegendItems;
	const TArray<FString> FoliageNames = FoliageFacade.GetFoliageNames();
	for (int32 FoliageIndex = 0; FoliageIndex < FoliageNames.Num(); FoliageIndex++)
	{
		if (FoliageNames[FoliageIndex].IsEmpty())
		{
			continue;
		}
		
		const FLinearColor Color = PV::Utilities::GetRandomHueColor(static_cast<float>(FoliageIndex) / static_cast<float>(FoliageNames.Num()));
		LegendItems.Add({Color.ToFColorSRGB(), FText::FromString(FSoftObjectPath(FoliageNames[FoliageIndex]).GetAssetName())});
	}
 
	FBox Bounds;
	for (int32 PointIndex = 0; PointIndex < FoliageFacade.GetElementCount(); PointIndex++)
	{
		const PV::Facades::FFoliageEntryData FoliageData = FoliageFacade.GetFoliageEntry(PointIndex);
 
		Bounds += static_cast<FVector>(FoliageData.PivotPoint);
		const FLinearColor Color = PV::Utilities::GetRandomHueColor(static_cast<float>(FoliageData.NameId) / static_cast<float>(FoliageNames.Num()));
		LineBatchComponent->AddLine(
			static_cast<FVector>(FoliageData.PivotPoint),
			static_cast<FVector>(FoliageData.PivotPoint + FoliageData.UpVector * 5.0f),
			Color,
			SDPG_World,
			EPointDrawSettings::None
		);
 
		const int32 InstanceIndex = SphereInstancer->AddInstance(
			FTransform(
				FRotator(),
				static_cast<FVector>(FoliageData.PivotPoint),
				FVector(0.015f)
			)
		);
		SphereInstancer->SetCustomDataValue(InstanceIndex, 0, Color.R, false);
		SphereInstancer->SetCustomDataValue(InstanceIndex, 1, Color.G, false);
		SphereInstancer->SetCustomDataValue(InstanceIndex, 2, Color.B, false);
	}
 
	SphereInstancer->MarkRenderStateDirty();
 
	InOutParams.ManagedResources.Add(SphereInstancer);
	InOutParams.Scene->AddComponent(SphereInstancer, FTransform::Identity);
	
	InOutParams.ManagedResources.Add(LineBatchComponent);
	InOutParams.Scene->AddComponent(LineBatchComponent, FTransform::Identity);
 
	TSharedPtr<SPVEditorViewport> ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(InOutParams.EditorViewportClient->GetEditorViewportWidget());
	ViewportWidget->PopulateLegendOverlayText(LegendItems);
	
	OutBounds = OutBounds + Bounds;
}

void FPVDataVisualization::SeedRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	const PV::Facades::FPointFacade PointFacade(InCollection);

	TArray<TPair<FColor, FText>> LegendItems;
	LegendItems.Add({FColor::Blue,  FText::FromString("Apical Direction")});
	LegendItems.Add({FColor::Red,  FText::FromString("Axillary Direction")});
	LegendItems.Add({FColor::Green,  FText::FromString("Up Vector")});
	
	const TObjectPtr<UInstancedStaticMeshComponent> ConeInstancer = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	ConeInstancer->SetStaticMesh(LoadObject<UStaticMesh>(nullptr, TEXT("/ProceduralVegetationEditor/DefaultAssets/Visualization/SM_Cone.SM_Cone")));
	FBox Bounds;
	for (int32 PointIndex = 0; PointIndex < PointFacade.GetElementCount(); PointIndex++)
	{
		FVector3f Position = PointFacade.GetPosition(PointIndex);
		float SeedScaleRatio = 1.0;
		PointFacade.GetSeedPScaleRatio(PointIndex, SeedScaleRatio);
		SeedScaleRatio *= 0.1f;

		TArray<FVector3f> Directions = PointFacade.GetBudDirection(PointIndex);

		FQuat FullRotation = FQuat::Identity;
		
		if (Directions.Num() >= 2)
		{
			FQuat Quat = FQuat::FindBetweenVectors(FVector::UpVector, FVector(Directions[0]));
			FQuat RollQuat = FQuat::FindBetweenVectors(Quat.GetForwardVector(), FVector(Directions[1]));
			FullRotation = RollQuat * Quat;
		}
		
		Bounds += static_cast<FVector>(Position);
		
		ConeInstancer->AddInstance(
			FTransform(
				FullRotation,
				static_cast<FVector>(Position),
				FVector(SeedScaleRatio)
			)
		);
	}
 
	ConeInstancer->MarkRenderStateDirty();
 
	InOutParams.ManagedResources.Add(ConeInstancer);
	InOutParams.Scene->AddComponent(ConeInstancer, FTransform::Identity);
	
	
	TSharedPtr<SPVEditorViewport> ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(InOutParams.EditorViewportClient->GetEditorViewportWidget());
	ViewportWidget->PopulateLegendOverlayText(LegendItems);
	
	OutBounds = OutBounds + Bounds;
}

void FPVDataVisualization::TexturesRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	const UPVTrunkTextureSetupData* TextureData = Cast<UPVTrunkTextureSetupData>(Data);
	if (!ensure(TextureData))
	{
		UE_LOGF(LogProceduralVegetationDataVisualization, Warning, "Cannot visualize Textures, Texture Data is not valid.");
		return;
	}

	const TObjectPtr<UStaticMeshComponent> StaticMeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	StaticMeshComponent->SetStaticMesh(LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Plane.Plane")));

	UMaterialInterface* Material = LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/DefaultAssets/Visualization/Materials/M_TrunkTextureSetup.M_TrunkTextureSetup"));

	if (Material)
	{
		UMaterialInstanceDynamic* DynamicMaterial = UMaterialInstanceDynamic::Create(Material, nullptr);

		if (DynamicMaterial)
		{
			FString ChannelToPreview = TextureData->TrunkTextureSetupInfo.PreviewChannelName;
			if (TextureData->TrunkTextureSetupInfo.Channels.Contains(ChannelToPreview))
			{
				DynamicMaterial->SetTextureParameterValue(TEXT("PreviewTexture"), TextureData->TrunkTextureSetupInfo.Channels[ChannelToPreview]);
			}
	
			StaticMeshComponent->SetMaterial(0, DynamicMaterial);
		}
	}

	FVector Position = FVector(0,0,550);
	FBox Bounds;
	Bounds += StaticMeshComponent->GetBounds().GetBox();
	Bounds += Position;
	
	InOutParams.ManagedResources.Add(StaticMeshComponent);
	InOutParams.Scene->AddComponent(StaticMeshComponent, FTransform(
				FQuat::MakeFromEuler(FVector( 90, 0,0)),
				static_cast<FVector>(Position),
				FVector::One() * 10
			));

	const TObjectPtr<UPVLineBatchComponent> LineBatchComponent = NewObject<UPVLineBatchComponent>(GetTransientPackage(), NAME_None, RF_Transient);

	TArray<TPair<FColor, FText>> LegendItems;
	
	float Generation = 0;
	float PreviousEnd = 0;
	constexpr float Scale = 1000;
	const int NumGenerations = TextureData->TrunkTextureSetupInfo.GenerationUVs.Num();
	
	for (auto GenerationUV : TextureData->TrunkTextureSetupInfo.GenerationUVs)
	{
		float DilationFactor = (GenerationUV.OffsetXStart * Scale) - PreviousEnd;
		const FLinearColor Color = PV::Utilities::GetRandomHueColor(NumGenerations > 0 ? Generation / static_cast<float>(NumGenerations) : Generation);
		LineBatchComponent->AddLine(
			FVector(PreviousEnd - (Scale/ 2.0), 0, Scale + 100 ),
			FVector(((GenerationUV.OffsetXEnd * Scale) + DilationFactor) - (Scale/ 2.0), 0, Scale + 100),
			Color,
			SDPG_World,
			EPointDrawSettings::None
		);

		LineBatchComponent->AddLine(
			FVector(PreviousEnd - (Scale/ 2.0), 0, 0 ),
			FVector(((GenerationUV.OffsetXEnd * Scale) + DilationFactor) - (Scale/ 2.0), 0, 0),
			Color,
			SDPG_World,
			EPointDrawSettings::None
		);

		float OverflowValue = GenerationUV.GetOverflowValue();
		int Pixels = static_cast<int>( OverflowValue * static_cast<float>(TextureData->TrunkTextureSetupInfo.Resolution));
		FString LegendString = OverflowValue > 0 ? FString::Printf(TEXT("Generation %i Overflow by %i pixels."), static_cast<int>(Generation + 1), Pixels)
		: FString::Printf(TEXT("Generation %i"), static_cast<int>(Generation + 1));

		FText LegendText = FText::FromString(LegendString);
		
		LegendItems.Add({Color.ToFColor(true), LegendText });
		
		PreviousEnd = (GenerationUV.OffsetXEnd * Scale) + DilationFactor ;
		Generation++;
	}

	InOutParams.ManagedResources.Add(LineBatchComponent);
	InOutParams.Scene->AddComponent(LineBatchComponent, FTransform::Identity);
	
	TSharedPtr<SPVEditorViewport> ViewportWidget = StaticCastSharedPtr<SPVEditorViewport>(InOutParams.EditorViewportClient->GetEditorViewportWidget());
    	ViewportWidget->PopulateLegendOverlayText(LegendItems);
	
	OutBounds = OutBounds + Bounds;
}

void FPVDataVisualization::GrowerLeavesRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPVDataVisualization::GrowerLeavesRenderer);

	if (!PV::FLeafMeshPathAttribute::HasAttribute(InCollection) || !PV::FLeafPositionAttribute::HasAttribute(InCollection))
	{
		return;
	}

	const auto MeshPaths = PV::FLeafMeshPathAttribute::GetAttribute(InCollection);
	if (MeshPaths.Num() == 0 || MeshPaths[0].IsEmpty())
	{
		return;
	}

	UStaticMesh* LeafMesh = LoadObject<UStaticMesh>(nullptr, *MeshPaths[0]);
	if (!LeafMesh)
	{
		return;
	}

	const auto Positions = PV::FLeafPositionAttribute::GetAttribute(InCollection);
	const auto Rotations = PV::FLeafRotationAttribute::GetAttribute(InCollection);
	const auto Scales    = PV::FLeafScaleAttribute::GetAttribute(InCollection);

	if (Positions.Num() == 0)
	{
		return;
	}

	UInstancedStaticMeshComponent* ISMC = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	ISMC->SetStaticMesh(LeafMesh);

	FBox Bounds(EForceInit::ForceInit);
	for (int32 i = 0; i < Positions.Num(); ++i)
	{
		const FVector Pos(Positions[i]);
		const FVector4f& R = Rotations[i];
		Bounds += Pos;
		ISMC->AddInstance(FTransform(FQuat(R.X, R.Y, R.Z, R.W), Pos, FVector(Scales[i])));
	}

	InOutParams.ManagedResources.Add(ISMC);
	InOutParams.Scene->AddComponent(ISMC, FTransform::Identity);
	OutBounds = OutBounds + FBoxSphereBounds(Bounds);
}

void FPVDataVisualization::GrafterPaletteGridRenderer(const UPVData* InData, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	const UPVGrafterPaletteData* Data = Cast<UPVGrafterPaletteData>(InData);
	if (!Data || Data->GetGrowthDataElements().IsEmpty())
	{
		return;
	}

	const auto AddComponentToScene = [&](USceneComponent* ComponentRef, const FTransform& Transform, const bool bUpdateBounds = true)
		{
			InOutParams.ManagedResources.Add(ComponentRef);
			InOutParams.Scene->AddComponent(ComponentRef, Transform);
			if (bUpdateBounds)
			{
				ComponentRef->UpdateBounds();
				OutBounds = OutBounds + ComponentRef->CalcBounds(Transform);
			}
		};

	UStaticMesh* TextBackgroundMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Plane"));
	UMaterial* BackgroundMat = LoadObject<UMaterial>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/TextBackgroundMat.TextBackgroundMat"));
	UMaterialInterface* TextMat = LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/GridVisualizationTextMaterial_Inst.GridVisualizationTextMaterial_Inst"));
	
	const TArray<TObjectPtr<UPVGrowthData>>& Elements = Data->GetGrowthDataElements();
	FBox MaxSkeletonBounds(ForceInit);
	float MaxSkeletonHeight = 0.0f;

	//First Pass - Calculate bounds and horizontal and vertical padding
	for (int32 Index = 0; Index < Elements.Num(); ++Index)
	{
		const UPVGrowthData* GrowthData = Elements[Index].Get();
		if (!GrowthData)
		{
			continue;
		}

		PV::Facades::FPointFacade PointFacade(GrowthData->GetCollection());
		if (!PointFacade.IsValid())
		{
			continue;
		}

		const TManagedArray<FVector3f>& Positions = PointFacade.GetPositions();

		FBox SkeletonBox(ForceInit);
		for (const FVector3f& Pos : Positions)
		{
			SkeletonBox += FVector(Pos);
		}

		// Cell width is keyed to the widest XY footprint, ignoring height. Height is tracked
		// separately because the skeleton with the largest footprint may not be the tallest,
		// and vertical/horizontal padding are derived independently.
		if (FMath::Max(SkeletonBox.GetSize().X, SkeletonBox.GetSize().Y) > FMath::Max(MaxSkeletonBounds.GetSize().X, MaxSkeletonBounds.GetSize().Y))
		{
			MaxSkeletonBounds = SkeletonBox;
		}
		MaxSkeletonHeight = FMath::Max(MaxSkeletonHeight, SkeletonBox.GetSize().Z);
	}
	if (!MaxSkeletonBounds.IsValid)
	{
		return;
	}
	const float SkeletonsHorizontalPadding = FMath::Max(MaxSkeletonBounds.GetSize().X, MaxSkeletonBounds.GetSize().Y) + 10.0f;
	const float SkeletonsVerticalPadding = MaxSkeletonHeight + 20.0f;

	FVector GridPos = FVector(SkeletonsHorizontalPadding * Elements.Num() / -2.0f, 0.0f, 0.0f);
	// One component is shared across all skeletons. UPVLineBatchComponent stores positions in
	// absolute world space — the component transform does not move rendered lines. Grid layout
	// is applied by offsetting positions at AddLine call time, not by moving the component.
	UPVLineBatchComponent* LineBatch = NewObject<UPVLineBatchComponent>(GetTransientPackage(), NAME_None, RF_Transient);

	//Second Pass - Render plant skeleton data
	for (int32 Index = 0; Index < Elements.Num(); ++Index)
	{
		const UPVGrowthData* GrowthData = Elements[Index].Get();
		if (!GrowthData)
		{
			GridPos.X += SkeletonsHorizontalPadding;
			continue;
		}

		PV::Facades::FPointFacade PointFacade(GrowthData->GetCollection());
		PV::Facades::FBranchFacade BranchFacade(GrowthData->GetCollection());
		if (!BranchFacade.IsValid() || !PointFacade.IsValid())
		{
			GridPos.X += SkeletonsHorizontalPadding;
			continue;
		}

		const TManagedArray<FVector3f>& Positions = PointFacade.GetPositions();

		if (Positions.Num()<1)
		{
			GridPos.X += SkeletonsHorizontalPadding;
			continue;
		}

		FVector3f Centroid = FVector3f::ZeroVector;
		float MinZ = TNumericLimits<float>::Max();
		for (const FVector3f& Pos : Positions)
		{
			Centroid += Pos;
			MinZ = FMath::Min(MinZ, Pos.Z);
		}
		if (Positions.Num() > 0)
		{
			Centroid /= static_cast<float>(Positions.Num());
		}

		// Pivot on the bottom-center rather than the centroid so the skeleton sits on the floor.
		// Using centroid Z would place half the skeleton below Z=0.
		const FVector Pivot(Centroid.X, Centroid.Y, MinZ);
		const FVector Offset = GridPos - Pivot;

		for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
		{
			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
			if (BranchPoints.Num() < 2)
			{
				continue;
			}
			FVector PreviousPosition = FVector(Positions[BranchPoints[0]]) + Offset;

			for (int32 BranchPointIndex = 1; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
			{
				const int32 PointIndex = BranchPoints[BranchPointIndex];
				const FVector CurrentPosition = FVector(Positions[PointIndex]) + Offset;
				LineBatch->AddLine(PreviousPosition, CurrentPosition, FLinearColor::White, SDPG_World, EPointDrawSettings::End);
				PreviousPosition = CurrentPosition;
			}
		}

		UTextRenderComponent* TextRenderComponent = NewObject<UTextRenderComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		TextRenderComponent->SetMaterial(0, TextMat);
		TextRenderComponent->SetText(FText::FromString(FString::Printf(TEXT("Graft %d"), Index + 1)));
		TextRenderComponent->SetTextRenderColor(FColor::Black);
		TextRenderComponent->SetWorldSize(10);
		TextRenderComponent->SetVerticalAlignment(EVRTA_TextTop);
		TextRenderComponent->SetGenerateOverlapEvents(false);

		FBoxSphereBounds TextBounds = TextRenderComponent->CalcLocalBounds();
		const FVector TextPos = GridPos + TextBounds.BoxExtent.Y * FVector::BackwardVector + SkeletonsVerticalPadding * FVector::UpVector;
		AddComponentToScene(TextRenderComponent, FTransform(FRotator(0, 90, 0), TextPos), false);

		TextBounds = TextBounds.TransformBy(TextRenderComponent->GetComponentTransform());

		if (TextBackgroundMesh && BackgroundMat)
		{
			UStaticMeshComponent* TextBackgroundMeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			TextBackgroundMeshComponent->SetStaticMesh(TextBackgroundMesh);
			TextBackgroundMeshComponent->SetMaterial(0, BackgroundMat);
			AddComponentToScene(
				TextBackgroundMeshComponent,
				FTransform(
					FRotator(0, 0, 90),
					TextBounds.Origin + FVector::LeftVector * 0.01f,
					FVector(TextBounds.BoxExtent.X / 40.0, TextBounds.BoxExtent.Z / 40.0, 1)
				),
				false
			);
		}
		
		GridPos.X += SkeletonsHorizontalPadding;
	}

	// Called once after all lines are added. AddLine triggers MarkRenderStateDirty internally
	// per call, but any renderer adding many lines should call it once here to avoid a
	// render-state rebuild for every individual line addition.
	LineBatch->MarkRenderStateDirty();
	InOutParams.ManagedResources.Add(LineBatch);
	InOutParams.Scene->AddComponent(LineBatch, FTransform::Identity);
	// CalcLocalBounds, not CalcBounds(Transform), because line positions are absolute world-space.
	// Applying the component transform would produce incorrect bounds.
	OutBounds = OutBounds + LineBatch->CalcLocalBounds();

}

void FPVDataVisualization::PlantProfileRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	if (!Cast<UPVPlantProfileData>(Data))
	{
		return;
	}

	const UPVPlantProfileLoaderSettings* ProfileSettings = Cast<UPVPlantProfileLoaderSettings>(Settings);
	if (!ProfileSettings || !ProfileSettings->PlantProfileData)
	{
		return;
	}

	constexpr float DisplayRadius = 50.0f;
	constexpr float HorizontalPadding = 20.0f;
	constexpr float CellWidth = DisplayRadius * 2.0f + HorizontalPadding;

	const int32 NumProfiles = ProfileSettings->PlantProfileData->Profiles.Num();

	UMaterialInterface* TextMat = LoadObject<UMaterialInterface>(nullptr,
		TEXT("/ProceduralVegetationEditor/Materials/ScaleTextMaterial.ScaleTextMaterial"));
	UStaticMesh* TextBGMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Plane"));
	UMaterial* TextBGMat = LoadObject<UMaterial>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/TextBackgroundMat.TextBackgroundMat"));

	const auto AddComponentToScene = [&](USceneComponent* Component, const FTransform& Transform, bool bUpdateBounds = true)
		{
			InOutParams.ManagedResources.Add(Component);
			InOutParams.Scene->AddComponent(Component, Transform);
			if (bUpdateBounds)
			{
				Component->UpdateBounds();
				OutBounds = OutBounds + Component->CalcBounds(Transform);
			}
		};

	UPVLineBatchComponent* LineBatch = NewObject<UPVLineBatchComponent>(GetTransientPackage(), NAME_None, RF_Transient);

	const FLinearColor ProfileColor(0.2f, 0.8f, 1.0f);
	FVector GridPos(CellWidth * NumProfiles / -2.0f + CellWidth / 2.0f, 0.0f, DisplayRadius);

	for (int32 ProfileIndex = 0; ProfileIndex < NumProfiles; ++ProfileIndex)
	{
		const TArray<float>& Points = ProfileSettings->PlantProfileData->Profiles[ProfileIndex].Points;
		const int32 NumPoints = Points.Num();
		if (NumPoints < 2)
		{
			GridPos.X += CellWidth;
			continue;
		}

		for (int32 i = 0; i < NumPoints; ++i)
		{
			const int32 NextI = (i + 1) % NumPoints;

			const float Angle0 = (static_cast<float>(i) / static_cast<float>(NumPoints)) * TWO_PI;
			const float Angle1 = (static_cast<float>(NextI) / static_cast<float>(NumPoints)) * TWO_PI;

			const float R0 = Points[i] * DisplayRadius;
			const float R1 = Points[NextI] * DisplayRadius;

			const FVector P0 = GridPos + FVector(R0 * FMath::Cos(Angle0), 0.f, R0 * FMath::Sin(Angle0));
			const FVector P1 = GridPos + FVector(R1 * FMath::Cos(Angle1), 0.f, R1 * FMath::Sin(Angle1));

			LineBatch->AddLine(P0, P1, ProfileColor, SDPG_World, EPointDrawSettings::End);
		}

		UTextRenderComponent* Label = NewObject<UTextRenderComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		Label->SetMaterial(0, TextMat);
		Label->SetText(FText::FromString(FString::Printf(TEXT("Profile %d"), ProfileIndex + 1)));
		Label->SetTextRenderColor(FColor::Black);
		constexpr float TextWorldSize = 10.0f;
		Label->SetWorldSize(TextWorldSize);
		Label->SetVerticalAlignment(EVRTA_TextTop);
		Label->SetGenerateOverlapEvents(false);

		FBoxSphereBounds LabelBounds = Label->CalcLocalBounds();
		const FVector LabelPos = GridPos + LabelBounds.BoxExtent.Y * FVector::BackwardVector + (DisplayRadius + TextWorldSize * 2.0f) * FVector::UpVector;
		AddComponentToScene(Label, FTransform(FRotator(0, 90, 0), LabelPos), false);

		LabelBounds = LabelBounds.TransformBy(Label->GetComponentTransform());
		if (TextBGMesh && TextBGMat)
		{
			UStaticMeshComponent* LabelBG = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
			LabelBG->SetStaticMesh(TextBGMesh);
			LabelBG->SetMaterial(0, TextBGMat);
			AddComponentToScene(LabelBG, FTransform(
				FRotator(0, 0, 90),
				LabelBounds.Origin + FVector::LeftVector * 0.01f,
				FVector(LabelBounds.BoxExtent.X / 40.0, LabelBounds.BoxExtent.Z / 40.0, 1)
			), false);
		}

		GridPos.X += CellWidth;
	}

	AddComponentToScene(LineBatch, FTransform::Identity);
}

void FPVDataVisualization::SetupBoundingBoxOnly(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams,
	FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings)
{
	PV::Facades::FPointFacade PointFacade(InCollection);
	if (!PointFacade.IsValid())
	{
		return;
	}

	const TManagedArray<FVector3f>& Positions = PointFacade.GetPositions();

	FBox Bounds(EForceInit::ForceInit);
	for (const FVector3f& Position : Positions)
	{
		Bounds += FVector(Position);
	}
	
	OutBounds = OutBounds + Bounds;
}
#undef LOCTEXT_NAMESPACE
