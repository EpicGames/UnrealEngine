// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDataVisualization.h"

#include "DataTypes/PVData.h"

#include "DataVisualizations/PCGSpatialDataVisualization.h"

class AActor;
class UPCGData;
struct FPCGContext;

DECLARE_DELEGATE_FiveParams(FPVRenderCallback, const UPVData*, const FManagedArrayCollection&, FPCGSceneSetupParams&, FBoxSphereBounds&, const UPCGSettingsInterface*);

class FPVDataVisualization : public IPCGSpatialDataVisualization
{
public:
	FPVDataVisualization();
	
	// ~Begin IPCGDataVisualization interface
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const override;
	virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface

protected:
	TMap<EPVRenderType, FPVRenderCallback> RenderMap;
	
	void RegisterVisualizations();

	void MeshRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void SkeletonRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void FoliageRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void BonesRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void FoliageGridRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void FoliageAttachmentPointsRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void SeedRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void TexturesRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void GrowerLeavesRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void GrafterPaletteGridRenderer(const UPVData* InData, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void PlantProfileRenderer(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
	void SetupBoundingBoxOnly(const UPVData* Data, const FManagedArrayCollection& InCollection, FPCGSceneSetupParams& InOutParams, FBoxSphereBounds& OutBounds, const UPCGSettingsInterface* Settings);
};

DECLARE_LOG_CATEGORY_EXTERN(LogProceduralVegetationDataVisualization, Log, All);