// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVLineBatchComponent.h"
#include "MeshElementCollector.h"
#include "PrimitiveDrawInterface.h"

FPVLineSceneProxy::FPVLineSceneProxy(const UPVLineBatchComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, Lines(InComponent->Lines)
	, Points(InComponent->Points)
	, PointSize(InComponent->PointSize)
{}

FPrimitiveViewRelevance FPVLineSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance ViewRelevance;
	ViewRelevance.bDrawRelevance = IsShown(View);
	ViewRelevance.bDynamicRelevance = true;
	ViewRelevance.bSeparateTranslucency = ViewRelevance.bNormalTranslucency = true;
	return ViewRelevance;
}

void FPVLineSceneProxy::GetDynamicMeshElements(
	const TArray<const FSceneView*>& Views,
	const FSceneViewFamily& ViewFamily,
	uint32 VisibilityMap,
	FMeshElementCollector& Collector
) const
{
	const static auto DrawLines = [](FPrimitiveDrawInterface* PDI, const TArray<FPVLineInfo>& Lines, const float PointSize)
	{
		for (const FPVLineInfo& Line : Lines)
		{
			PDI->DrawLine(
				Line.StartPos,
				Line.EndPos,
				Line.Color,
				Line.DepthPriorityGroup,
				2.0f,
				0,
				true
			);

			if (Line.PointDrawSettings == EPointDrawSettings::Start || Line.PointDrawSettings == EPointDrawSettings::Both)
			{
				PDI->DrawPoint(Line.StartPos, Line.Color, PointSize, Line.DepthPriorityGroup);
			}

			if (Line.PointDrawSettings == EPointDrawSettings::End || Line.PointDrawSettings == EPointDrawSettings::Both)
			{
				PDI->DrawPoint(Line.EndPos, Line.Color, PointSize, Line.DepthPriorityGroup);
			}
		}
	};

	const static auto DrawPoints = [](FPrimitiveDrawInterface* PDI, const TArray<FPVPointInfo>& Points)
	{
		for (const FPVPointInfo& Point : Points)
		{
			PDI->DrawPoint(
				Point.PointLocation,
				Point.Color,
				Point.PointSize,
				Point.DepthPriorityGroup
			);
		}
	};

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

#if UE_ENABLE_DEBUG_DRAWING
			FPrimitiveDrawInterface* PDI = Collector.GetDebugPDI(ViewIndex);
#else	
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
#endif

			DrawLines(PDI, Lines, PointSize);
			DrawPoints(PDI, Points);
		}
	}
}

SIZE_T FPVLineSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

uint32 FPVLineSceneProxy::GetMemoryFootprint() const { return sizeof *this + GetAllocatedSize(); }

bool FPVLineSceneProxy::CanBeOccluded() const
{
	return false;
}

UPVLineBatchComponent::UPVLineBatchComponent()
{
	bAutoActivate = true;
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
	SetIgnoreStreamingManagerUpdate(true);
	
	BBox = FBox(ForceInit);
	Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector::OneVector, 1);
}

void UPVLineBatchComponent::InitBounds()
{
	BBox = FBox(ForceInit);
	MarkRenderStateDirty();
}

void UPVLineBatchComponent::AddLine(const FVector& InStartPos, const FVector& InEndPos, const FLinearColor& InColor, const ESceneDepthPriorityGroup InDepthPriorityGroup, const EPointDrawSettings InPointDrawSettings)
{
	Lines.Emplace(InStartPos, InEndPos, InColor, InDepthPriorityGroup, InPointDrawSettings);

	BBox += InStartPos;
	BBox += InEndPos;

	MarkRenderStateDirty();
}

void UPVLineBatchComponent::AddSphere(const FVector& InPos, float Radius, const FLinearColor& InColor, const ESceneDepthPriorityGroup InDepthPriorityGroup)
{
	if (Radius <= 0)
	{
		return;
	}

	const static int32 Segments = 6;

	const float AngleInc = 2.f * UE_PI / Segments;
	int32 NumSegmentsY = Segments;
	float Latitude = AngleInc;
	float SinY1 = 0.0f, CosY1 = 1.0f;

	while (NumSegmentsY--)
	{
		const float SinY2 = FMath::Sin(Latitude);
		const float CosY2 = FMath::Cos(Latitude);

		FVector Vertex1 = FVector(SinY1, 0.0f, CosY1) * Radius + InPos;
		FVector Vertex3 = FVector(SinY2, 0.0f, CosY2) * Radius + InPos;
		float Longitude = AngleInc;

		int32 NumSegmentsX = Segments;
		while (NumSegmentsX--)
		{
			const float SinX = FMath::Sin(Longitude);
			const float CosX = FMath::Cos(Longitude);

			const FVector Vertex2 = FVector((CosX * SinY1), (SinX * SinY1), CosY1) * Radius + InPos;
			const FVector Vertex4 = FVector((CosX * SinY2), (SinX * SinY2), CosY2) * Radius + InPos;

			Lines.Emplace(Vertex1, Vertex2, InColor, InDepthPriorityGroup, EPointDrawSettings::None);
			Lines.Emplace(Vertex1, Vertex3, InColor, InDepthPriorityGroup, EPointDrawSettings::None);

			BBox += Vertex1;
			BBox += Vertex2;
			BBox += Vertex3;

			Vertex1 = Vertex2;
			Vertex3 = Vertex4;
			Longitude += AngleInc;
		}
		SinY1 = SinY2;
		CosY1 = CosY2;
		Latitude += AngleInc;
	}

	MarkRenderStateDirty();
}

void UPVLineBatchComponent::AddPoint(const FVector& PointLocation, float InPointSize, const FLinearColor& InColor, const ESceneDepthPriorityGroup InDepthPriorityGroup)
{
	Points.Emplace(PointLocation, InPointSize, InColor, InDepthPriorityGroup);
	BBox += PointLocation;
	MarkRenderStateDirty();
}

void UPVLineBatchComponent::AddPoints(const TArray<FPVPointInfo>& InPoints)
{
	Points.Reserve(Points.Num() + InPoints.Num());
	for (const FPVPointInfo& Point : InPoints)
	{
		Points.Add(Point);
		BBox += Point.PointLocation;
	}

	MarkRenderStateDirty();
}

void UPVLineBatchComponent::Flush()
{
	Lines.Empty();
	Points.Empty();
	BBox = FBox(ForceInit);
	MarkRenderStateDirty();
}

FPrimitiveSceneProxy* UPVLineBatchComponent::CreateSceneProxy()
{
	FPVLineSceneProxy* Proxy = new FPVLineSceneProxy(this);
	return Proxy;
}

FBoxSphereBounds UPVLineBatchComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(BBox).TransformBy(LocalToWorld); 
}
