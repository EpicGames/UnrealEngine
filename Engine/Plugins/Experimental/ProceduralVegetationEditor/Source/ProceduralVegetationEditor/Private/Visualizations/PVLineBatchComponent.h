// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "SceneView.h"
#include "Components/PrimitiveComponent.h"
#include "PVLineBatchComponent.generated.h"

struct FManagedArrayCollection;
class UPVLineBatchComponent;

enum class EPointDrawSettings : uint8
{
	None,
	Start,
	End,
	Both
};

struct FPVLineInfo
{
	FVector StartPos;
	FVector EndPos;
	FLinearColor Color;
	ESceneDepthPriorityGroup DepthPriorityGroup = SDPG_World;
	EPointDrawSettings PointDrawSettings = EPointDrawSettings::None;
};

struct FPVPointInfo
{
	FVector PointLocation;
	float PointSize;
	FLinearColor Color;
	ESceneDepthPriorityGroup DepthPriorityGroup = SDPG_World;
};

class FPVLineSceneProxy final : public FPrimitiveSceneProxy
{
private:
	TArray<FPVLineInfo> Lines;
	TArray<FPVPointInfo> Points;
	float PointSize;

public:
	FPVLineSceneProxy(const UPVLineBatchComponent* InComponent);

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual void GetDynamicMeshElements(
		const TArray<const FSceneView*>& Views,
		const FSceneViewFamily& ViewFamily,
		uint32 VisibilityMap,
		FMeshElementCollector& Collector
	) const override;

	virtual SIZE_T GetTypeHash() const override;
	virtual uint32 GetMemoryFootprint(void) const override;
	virtual bool CanBeOccluded() const override;
};

UCLASS(Blueprintable, BlueprintType, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPVLineBatchComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

private:
	friend class FPVLineSceneProxy;

public:
	UPVLineBatchComponent();

	void InitBounds();
	
	void AddLine(const FVector& InStartPos, const FVector& InEndPos, const FLinearColor& InColor, const ESceneDepthPriorityGroup InDepthPriorityGroup, const EPointDrawSettings InPointDrawSettings = EPointDrawSettings::None);

	void AddSphere(const FVector& InPos, float Radius, const FLinearColor& InColor, const ESceneDepthPriorityGroup InDepthPriorityGroup);

	void AddPoint(const FVector& PointLocation, float PointSize, const FLinearColor& InColor, const ESceneDepthPriorityGroup InDepthPriorityGroup);

	void AddPoints(const TArray<FPVPointInfo>& InPoints);

	void Flush();
	
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

protected:
	FBox BBox;
	TArray<FPVLineInfo> Lines;
	TArray<FPVPointInfo> Points;

private:
	const float PointSize = 5.0f;
};
