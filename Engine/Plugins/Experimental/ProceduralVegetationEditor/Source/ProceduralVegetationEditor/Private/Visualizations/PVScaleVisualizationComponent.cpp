// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVScaleVisualizationComponent.h"

#include "Components/TextRenderComponent.h"

#include "Engine/World.h"

#include "GameFramework/WorldSettings.h"

#include "Materials/MaterialInterface.h"

UPVScaleVisualizationComponent::UPVScaleVisualizationComponent()
	: Bounds(ForceInit)
{
	bAutoActivate = true;
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
	SetIgnoreStreamingManagerUpdate(true);

	int32 ComponentIndex = 0;
	for (TObjectPtr<UTextRenderComponent>& Component : TextRenderComponents)
	{
		const FString ComponentName = TEXT("LengthTextRenderComponent") + FString::FromInt(ComponentIndex++);
		Component = CreateDefaultSubobject<UTextRenderComponent>(*ComponentName);
		Component->SetGenerateOverlapEvents(false);
		Component->SetupAttachment(this);
	}
}

FBoxSphereBounds& UPVScaleVisualizationComponent::GetScaleBounds()
{
	return Bounds;
}

const FBoxSphereBounds& UPVScaleVisualizationComponent::GetScaleBounds() const
{
	return Bounds;
}

void UPVScaleVisualizationComponent::SetScaleBounds(const FBoxSphereBounds& InBounds)
{
	Bounds = InBounds;
	UpdateScaleVisualizations();
}

TObjectPtr<UTextRenderComponent> UPVScaleVisualizationComponent::GetTextRenderComponent(int32 Index) const
{
	return TextRenderComponents[Index];
}

const TStaticArray<TObjectPtr<UTextRenderComponent>, 3>& UPVScaleVisualizationComponent::GetTextRenderComponents() const
{
	return TextRenderComponents;
}

void UPVScaleVisualizationComponent::OnRegister()
{
	Super::OnRegister();
	UpdateScaleVisualizations();
}

void UPVScaleVisualizationComponent::UpdateScaleVisualizations()
{
	if (!GetWorld())
	{
		return;
	}

	Flush();

	const auto WorldToMetersScale = GetWorld()->GetWorldSettings()->WorldToMeters;
	const double TextSize = (Bounds.BoxExtent.X + Bounds.BoxExtent.Y + Bounds.BoxExtent.Z) * 20.0f / 3.0f / WorldToMetersScale;
	
	{
		FVector StartPosition = Bounds.Origin;
		StartPosition.X -= Bounds.BoxExtent.X;
		StartPosition.Y -= Bounds.BoxExtent.Y;
		StartPosition.Z = 0;

		FVector EndPosition = Bounds.Origin;
		EndPosition.X += Bounds.BoxExtent.X;
		EndPosition.Y -= Bounds.BoxExtent.Y;
		EndPosition.Z = 0;

		const double Length = (Bounds.BoxExtent.X * 2.0f) / WorldToMetersScale;
		const TObjectPtr<UTextRenderComponent> TextRenderComponent = GetTextRenderComponent(0);

		TextRenderComponent->SetWorldLocation(EndPosition);
		TextRenderComponent->SetWorldRotation(FRotator(0, 90, 0));
		TextRenderComponent->SetTextRenderColor(FLinearColor::Red.ToFColorSRGB());
		TextRenderComponent->SetHorizontalAlignment(EHTA_Left);
		TextRenderComponent->SetVerticalAlignment(EVRTA_TextBottom);
		TextRenderComponent->SetWorldSize(TextSize);
		TextRenderComponent->SetMaterial(0, LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/ScaleTextMaterial.ScaleTextMaterial")));
		TextRenderComponent->SetText(FText::FromString(FString::Printf(TEXT("%.2fm"), Length)));

		AddLine(StartPosition, EndPosition, FLinearColor::Red, ESceneDepthPriorityGroup::SDPG_World);
	}

	{
		FVector StartPosition = Bounds.Origin;
		StartPosition.X -= Bounds.BoxExtent.X;
		StartPosition.Y -= Bounds.BoxExtent.Y;
		StartPosition.Z = 0;

		FVector EndPosition = Bounds.Origin;
		EndPosition.X -= Bounds.BoxExtent.X;
		EndPosition.Y += Bounds.BoxExtent.Y;
		EndPosition.Z = 0;

		const double Length = (Bounds.BoxExtent.Y * 2.0f) / WorldToMetersScale;
		const TObjectPtr<UTextRenderComponent> TextRenderComponent = GetTextRenderComponent(1);
		TextRenderComponent->SetWorldLocation(EndPosition);
		TextRenderComponent->SetTextRenderColor(FColor::Green);
		TextRenderComponent->SetHorizontalAlignment(EHTA_Right);
		TextRenderComponent->SetVerticalAlignment(EVRTA_TextBottom);
		TextRenderComponent->SetWorldSize(TextSize);
		TextRenderComponent->SetMaterial(0, LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/ScaleTextMaterial.ScaleTextMaterial")));
		TextRenderComponent->SetText(FText::FromString(FString::Printf(TEXT("%.2fm"), Length)));

		AddLine(StartPosition, EndPosition, FLinearColor::Green, ESceneDepthPriorityGroup::SDPG_World);
	}

	{
		FVector StartPosition = Bounds.Origin;
		StartPosition.X -= Bounds.BoxExtent.X;
		StartPosition.Y -= Bounds.BoxExtent.Y;
		StartPosition.Z -= Bounds.BoxExtent.Z;

		FVector EndPosition = Bounds.Origin;
		EndPosition.X -= Bounds.BoxExtent.X;
		EndPosition.Y -= Bounds.BoxExtent.Y;
		EndPosition.Z += Bounds.BoxExtent.Z;

		const double Length = (Bounds.BoxExtent.Z * 2.0f) / WorldToMetersScale;
		const TObjectPtr<UTextRenderComponent> TextRenderComponent = GetTextRenderComponent(2);
		TextRenderComponent->SetWorldLocation(EndPosition);
		TextRenderComponent->SetWorldRotation(FRotator(0, 45, 0));
		TextRenderComponent->SetTextRenderColor(FLinearColor::Blue.ToFColorSRGB());
		TextRenderComponent->SetHorizontalAlignment(EHTA_Center);
		TextRenderComponent->SetVerticalAlignment(EVRTA_TextBottom);
		TextRenderComponent->SetWorldSize(TextSize);
		TextRenderComponent->SetMaterial(0, LoadObject<UMaterialInterface>(nullptr, TEXT("/ProceduralVegetationEditor/Materials/ScaleTextMaterial.ScaleTextMaterial")));
		TextRenderComponent->SetText(FText::FromString(FString::Printf(TEXT("%.2fm"), Length)));
		
		AddLine(StartPosition, EndPosition, FLinearColor::Blue, ESceneDepthPriorityGroup::SDPG_World);
	}
}
