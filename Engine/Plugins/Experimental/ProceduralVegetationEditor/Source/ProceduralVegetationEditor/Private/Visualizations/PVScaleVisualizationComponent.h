// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PVLineBatchComponent.h"

#include "Containers/StaticArray.h"

#include "PVScaleVisualizationComponent.generated.h"

class UTextRenderComponent;

UCLASS(Blueprintable, BlueprintType, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPVScaleVisualizationComponent : public UPVLineBatchComponent
{
	GENERATED_BODY()

public:
	UPVScaleVisualizationComponent();

	FBoxSphereBounds& GetScaleBounds();
	const FBoxSphereBounds& GetScaleBounds() const;
	void SetScaleBounds(const FBoxSphereBounds& InBounds);

	TObjectPtr<UTextRenderComponent> GetTextRenderComponent(int32 Index) const;
	const TStaticArray<TObjectPtr<UTextRenderComponent>, 3>& GetTextRenderComponents() const;

	virtual void OnRegister() override;

	void UpdateScaleVisualizations();

private:
	FBoxSphereBounds Bounds;

	TStaticArray<TObjectPtr<UTextRenderComponent>, 3> TextRenderComponents = {nullptr, nullptr, nullptr};
};
