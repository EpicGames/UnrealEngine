// Copyright Epic Games, Inc. All Rights Reserved.

#include "PVSkeletonVisualizerComponent.h"

#include "PrimitiveDrawInterface.h"
#include "ProceduralVegetationEditorModule.h"
#include "PVEditorCommon.h"
#include "PVEditorSettings.h"

#include "Components/DynamicMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "Engine/StaticMesh.h"

#include "Facades/PVAttributesNames.h"
#include "Facades/PVBranchFacade.h"
#include "Facades/PVPointFacade.h"
#include "Facades/PVTreeFacade.h"

#include "Helpers/PVUtilities.h"

#include "UObject/ConstructorHelpers.h"

UPVSkeletonVisualizerComponent::UPVSkeletonVisualizerComponent()
{
	bAutoActivate = true;
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;
	bUseEditorCompositing = true;
	SetGenerateOverlapEvents(false);
	SetIgnoreStreamingManagerUpdate(true);

	bUseMeshPreview = GetDefault<UPVEditorSettings>()->bUseMeshSkeletonPreview;

	BBox = FBox(ForceInit);
	Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector::OneVector, 1);

	PointMeshInstancer = CreateDefaultSubobject<UInstancedStaticMeshComponent>(TEXT("PointSpheresISM"));
	PointMeshInstancer->SetupAttachment(this);
	PointMeshInstancer->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	PointMeshInstancer->SetNumCustomDataFloats(3);
	PointMeshInstancer->bHasPerInstanceHitProxies = true;
	
	// Sphere asset mesh-space radius = 50; instances are scaled by pscale/50 so world-space radius == pscale
	static ConstructorHelpers::FObjectFinder<UStaticMesh> SphereMeshFinder(TEXT("/Engine/BasicShapes/Sphere"));
	if (SphereMeshFinder.Succeeded())
	{
		PointMeshInstancer->SetStaticMesh(SphereMeshFinder.Object);
	}
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> PointMaterial(
			TEXT("/ProceduralVegetationEditor/Materials/PointMaterial.PointMaterial")
		);
	if (PointMaterial.Succeeded())
	{
		PointMeshInstancer->SetMaterial(0, PointMaterial.Object);
	}
	
	PreviewMeshComponent = CreateDefaultSubobject<UDynamicMeshComponent>(TEXT("PreviewMesh"));
	PreviewMeshComponent->SetupAttachment(this);
	PreviewMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	PreviewMeshComponent->SetEnableFlatShading(true);
	PreviewMeshComponent->SetNumMaterials(1);

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> EdgeMaterial(
		TEXT("/ProceduralVegetationEditor/Materials/PreviewTrunkMaterial.PreviewTrunkMaterial"));
	if (EdgeMaterial.Succeeded())
	{
		PreviewMeshComponent->SetMaterial(0, EdgeMaterial.Object);
	}
}

const FManagedArrayCollection* UPVSkeletonVisualizerComponent::GetCollection() const
{
	return SkeletonCollection;
}

void UPVSkeletonVisualizerComponent::SetCollection(const FManagedArrayCollection* const InSkeletonCollection)
{
	if (InSkeletonCollection == nullptr)
	{
		return;
	}

	SkeletonCollection = InSkeletonCollection;
	RebuildSkeleton();
}

void UPVSkeletonVisualizerComponent::RebuildSkeleton()
{
	if (!SkeletonCollection)
	{
		return;
	}
	
	const PV::Facades::FBranchFacade BranchFacade(*SkeletonCollection);
	const PV::Facades::FPointFacade PointFacade(*SkeletonCollection);
	if (!BranchFacade.IsValid() || !PointFacade.IsValid())
	{
		UE_LOGF(LogProceduralVegetationEditor, Error, "Failed to setup data viewport, Collection data is not valid for Procedural Vegetation.");
		return;
	}

	const float PointScaleBias = GetDefault<UPVEditorSettings>()->PointScaleBias;

	TArray<FLinearColor> PointColors = GetVisualizationColors();
	const FLinearColor DefaultColor = GetDefault<UPVEditorSettings>()->SkeletonDefaultColor;

	const TManagedArray<FVector3f>& PointsPositions = PointFacade.GetPositions();

	Flush();
	
	PointMeshInstancer->ClearInstances();
	InstanceToPointData.Reset(PointsPositions.Num());

	const bool bHasVisualizationMode = CurrentVisualizationMode != ESkeletonVisualizationModes::None;

	if (bUseMeshPreview)
	{
		const TManagedArray<float>& PointScales = PointFacade.GetPointScales();

		PointMeshInstancer->SetVisibility(true);
		PreviewMeshComponent->SetVisibility(true);

		UE::Geometry::FDynamicMesh3 PreviewMesh;

		PreviewMesh.EnableAttributes();
		PreviewMesh.Attributes()->EnableMaterialID();
		PreviewMesh.Attributes()->EnablePrimaryColors();
		
		EdgeMeshGenerator = MakeUnique<PV::Visualizations::FPVEdgeMeshGenerator>(&PreviewMesh);
		
		auto& BranchIndexAttribute = PV::EditorCommon::BranchIndexAttribute.GetOrAttachAttribute(PreviewMesh);
		auto& BranchPointIndexAttribute = PV::EditorCommon::BranchPointIndexAttribute.GetOrAttachAttribute(PreviewMesh);

		PV::Visualizations::FPVAppendMeshResult AppendMeshResult;
		for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
		{
			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
			bool bIsTrunk = BranchFacade.IsTrunk(BranchIndex);

			FTransform PreviousTransform;
			FLinearColor PreviousColor = FLinearColor::White;
			for (int32 BranchPointIndex = 0; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
			{
				const int32 IndexA = BranchPoints[BranchPointIndex];
				const int32 PointBudNumber = PointFacade.GetBudNumber(IndexA);
				FVector PositionA = FVector(PointsPositions[IndexA]);
				if (OnDrawPoint.IsBound())
				{
					if (!OnDrawPoint.Execute(PointBudNumber, PositionA))
					{
						break;
					}
				}

				BBox += PositionA;
				const FVector PointScaleA = FVector(FMath::Max(PointScales[IndexA], PV::EditorCommon::PointMinScale));
				const FLinearColor ColorA = bHasVisualizationMode ? PointColors[IndexA] : DefaultColor;
				FTransform TransformA(FQuat::Identity, PositionA, PointScaleA);

				if (bIsTrunk || BranchPointIndex != 0)
				[[likely]]
				{
					const int32 InstanceIndex = PointMeshInstancer->AddInstance(TransformA.GetScaled(PointScaleBias / 50.0f));
					PointMeshInstancer->SetCustomDataValue(InstanceIndex, 0, ColorA.R, false);
					PointMeshInstancer->SetCustomDataValue(InstanceIndex, 1, ColorA.G, false);
					PointMeshInstancer->SetCustomDataValue(InstanceIndex, 2, ColorA.B, false);
					InstanceToPointData.Insert({BranchIndex, BranchPointIndex}, InstanceIndex);
				}

				if (BranchPointIndex - 1 >= 0)
				[[likely]]
				{
					const FVector Dir = (PositionA - PreviousTransform.GetLocation()).GetSafeNormal();
					FQuat Rotation = FRotationMatrix::MakeFromZ(Dir).ToQuat();

					FTransform TopTransform = TransformA;
					TopTransform.SetLocation(TopTransform.GetLocation() - Dir * (PointScaleBias / 2.0f) * TopTransform.GetScale3D().X);
					TopTransform.SetRotation(Rotation);
					PreviousTransform.SetLocation(PreviousTransform.GetLocation() + Dir * (PointScaleBias / 2.0f) * PreviousTransform.GetScale3D().X);
					PreviousTransform.SetRotation(Rotation);

					AppendMeshResult = EdgeMeshGenerator->AddToMesh(PreviousTransform, TopTransform, PreviousColor, ColorA);
					for (const int32 VertexID : AppendMeshResult.VertexIDs)
					{
						BranchIndexAttribute.SetValue(VertexID, BranchIndex);
						BranchPointIndexAttribute.SetValue(VertexID, BranchPointIndex - 1);
					}
				}

				PreviousTransform = TransformA;
				PreviousColor = ColorA;
			}
		}
		
		PointMeshInstancer->MarkRenderStateDirty();
		PreviewMeshComponent->SetMesh(MoveTemp(PreviewMesh));
		if (bBuildOctree)
		{
			EdgeMeshOctree.SetMesh(PreviewMeshComponent->GetMesh());
		}
	}
	else
	{
		PreviewMeshComponent->SetVisibility(false);

		// Draw lines via UPVLineBatchComponent
		for (int32 BranchIndex = 0; BranchIndex < BranchFacade.GetElementCount(); ++BranchIndex)
		{
			const TArray<int32>& BranchPoints = BranchFacade.GetPoints(BranchIndex);
			FVector PreviousPosition = FVector(PointsPositions[BranchPoints[0]]);
			BBox += PreviousPosition;
			for (int32 BranchPointIndex = 1; BranchPointIndex < BranchPoints.Num(); ++BranchPointIndex)
			{
				const int32 PointIndex = BranchPoints[BranchPointIndex];
				const FVector CurrentPosition = FVector(PointsPositions[PointIndex]);
				const FLinearColor PointColor = bHasVisualizationMode ? PointColors[PointIndex] : DefaultColor;
				AddLine(PreviousPosition, CurrentPosition, PointColor, ESceneDepthPriorityGroup::SDPG_World, EPointDrawSettings::End);
				PreviousPosition = CurrentPosition;
				BBox += CurrentPosition;
			}
		}
	}
}

TObjectPtr<UDynamicMeshComponent> UPVSkeletonVisualizerComponent::GetDynamicMeshComponent() const
{
	return PreviewMeshComponent;
}

const UE::Geometry::FDynamicMeshAABBTree3& UPVSkeletonVisualizerComponent::GetDynamicMeshOctree() const
{
	return EdgeMeshOctree;
}

TObjectPtr<UInstancedStaticMeshComponent> UPVSkeletonVisualizerComponent::GetPointMeshInstancerComponent() const
{
	return PointMeshInstancer;
}

bool UPVSkeletonVisualizerComponent::GetPointDataFromInstanceIndex(const int32 InstanceIndex, int32& BranchIndex, int32& BranchPointIndex)
{
	if (InstanceToPointData.IsValidIndex(InstanceIndex))
	{
		BranchIndex = InstanceToPointData[InstanceIndex].Get<0>();
		BranchPointIndex = InstanceToPointData[InstanceIndex].Get<1>();
		return true;
	}
	return false;
}

ESkeletonVisualizationModes UPVSkeletonVisualizerComponent::GetVisualizationMode() const
{
	return CurrentVisualizationMode;
}

void UPVSkeletonVisualizerComponent::SetVisualizationMode(ESkeletonVisualizationModes InMode)
{
	if (CurrentVisualizationMode != InMode)
	{
		CurrentVisualizationMode = InMode;
		RebuildSkeleton();
	}
}


void UPVSkeletonVisualizerComponent::SetUseMeshPreview(bool bInUseMeshPreview)
{
	if (bUseMeshPreview != bInUseMeshPreview)
	{
		bUseMeshPreview = bInUseMeshPreview;
		RebuildSkeleton();
	}
}

bool UPVSkeletonVisualizerComponent::IsUsingMeshPreview() const
{
	return bUseMeshPreview;
}

bool UPVSkeletonVisualizerComponent::HasOctree() const
{
	return bBuildOctree && EdgeMeshOctree.GetMesh();
}

void UPVSkeletonVisualizerComponent::SetBuildOctree(const bool bInBuildOctree)
{
	if (bBuildOctree == bInBuildOctree)
	{
		return;
	}
	if (bInBuildOctree && PreviewMeshComponent && PreviewMeshComponent->GetMesh())
	{
		EdgeMeshOctree.SetMesh(PreviewMeshComponent->GetMesh(), true);
		bBuildOctree = true;
	}
	else
	{
		EdgeMeshOctree.SetMesh(nullptr, false);
		bBuildOctree = false;
	}
}

TArray<FLinearColor> UPVSkeletonVisualizerComponent::GetVisualizationColors() const
{
	const int32 NumPoints = SkeletonCollection->NumElements(PV::GroupNames::PointGroup);

	TArray<FLinearColor> PointColors;
	PointColors.Init(FLinearColor::White, NumPoints);

	TArray<float> VisualizationValues = PV::Facades::FTreeFacade::GetVisualizationValues(*SkeletonCollection, CurrentVisualizationMode);

	if (VisualizationValues.IsEmpty())
	{
		return PointColors;
	}

	const float MaxValue = *Algo::MaxElement(VisualizationValues);
	const float MinValue = *Algo::MinElement(VisualizationValues);
	const float Range = MaxValue - MinValue;

	if (Range < UE_SMALL_NUMBER)
	{
		PointColors.Init(FLinearColor::Black, NumPoints);
	}
	else
	{
		for (int32 Index = 0; Index < NumPoints; ++Index)
		{
			PointColors[Index] = PV::Utilities::GetRandomHueColor(1 - VisualizationValues[Index]);
		}
	}

	return PointColors;
}
