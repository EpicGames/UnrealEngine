// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDescriptionAdapter.h"
#include "PVLineBatchComponent.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "Facades/PVTreeFacade.h"

#include "Helpers/PVSkeletonVisualizerHelpers.h"

#include "PVSkeletonVisualizerComponent.generated.h"

struct FManagedArrayCollection;

class UInstancedStaticMeshComponent;
class UDynamicMeshComponent;

UCLASS(Blueprintable, BlueprintType, meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPVSkeletonVisualizerComponent : public UPVLineBatchComponent
{
	GENERATED_BODY()

public:
	UPVSkeletonVisualizerComponent();

	const FManagedArrayCollection* GetCollection() const;
	void SetCollection(const FManagedArrayCollection* const InSkeletonCollection);

	void RebuildSkeleton();

	TObjectPtr<UDynamicMeshComponent> GetDynamicMeshComponent() const;
	const UE::Geometry::FDynamicMeshAABBTree3& GetDynamicMeshOctree() const;
	
	TObjectPtr<UInstancedStaticMeshComponent> GetPointMeshInstancerComponent() const;
	
	bool GetPointDataFromInstanceIndex(const int32 InstanceIndex, int32& BranchIndex, int32& BranchPointIndex);

	ESkeletonVisualizationModes GetVisualizationMode() const;
	void SetVisualizationMode(ESkeletonVisualizationModes InMode);

	void SetUseMeshPreview(bool bInUseMeshPreview);
	bool IsUsingMeshPreview() const;
	
	bool HasOctree() const;
	void SetBuildOctree(bool bInBuildOctree);

	TArray<FLinearColor> GetVisualizationColors() const;

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnDrawPoint, const int32, FVector&);
	FOnDrawPoint OnDrawPoint;
	
protected:
	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> PreviewMeshComponent;
	
	UPROPERTY()
	TObjectPtr<UInstancedStaticMeshComponent> PointMeshInstancer;

	UE::Geometry::FDynamicMeshAABBTree3 EdgeMeshOctree;

	TUniquePtr<PV::Visualizations::FPVEdgeMeshGenerator> EdgeMeshGenerator;

private:
	const FManagedArrayCollection* SkeletonCollection = nullptr;
	
	TArray<TPair<int32, int32>> InstanceToPointData;
	ESkeletonVisualizationModes CurrentVisualizationMode = ESkeletonVisualizationModes::None;
	bool bUseMeshPreview = false;
	bool bBuildOctree = false;
};
