// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPVCreateDialog.h"

#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserDataModule.h"
#include "IContentBrowserSingleton.h"
#include "ProceduralVegetation.h"
#include "SPrimaryButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "SPVCreateDialog"

void SPVCreateDialog::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;

	bPressedOk = false;
	bCreateNew = false;

	TSharedRef<SVerticalBox> Container = SNew(SVerticalBox);

	FName SampleTreesVirtualPath;
	IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual("/ProceduralVegetationEditor", SampleTreesVirtualPath);

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UProceduralVegetation::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.PackagePaths.Add(SampleTreesVirtualPath);
	AssetPickerConfig.Filter.bRecursivePaths = true;

	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;

	AssetPickerConfig.bForceShowEngineContent = true;
	AssetPickerConfig.bForceShowPluginContent = true;
	AssetPickerConfig.bCanShowReadOnlyFolders = false;
	AssetPickerConfig.bCanShowFolders = false;
	AssetPickerConfig.bCanShowDevelopersFolder = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAddFilterUI = false;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bAllowRename = false;
	AssetPickerConfig.OnAssetSelected.BindLambda(
		[this](const FAssetData& AssetData)
		{
			SelectedProceduralVegetation = AssetData;
		}
	);

	Container
		->AddSlot()
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.HeaderPadding(FMargin(5.0f, 3.0f))
			.AllowAnimatedTransition(false)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("SPVCreateDialog", "SampleProceduralVegetationAreaTitle", "Sample Procedural Vegetation"))
				.TextStyle(FAppStyle::Get(), "ButtonText")
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			]
			.BodyContent()
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];

	Container
		->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(8.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(NSLOCTEXT("SPVCreateDialog", "ProceduralVegetationCreateNewButton", "Create New Procedural Vegetation"))
				.HAlign(HAlign_Center)
				.OnClicked(this, &SPVCreateDialog::OnProceduralVegetationCreateNew)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPrimaryButton)
					.Text(NSLOCTEXT("SPVCreateDialog", "ProceduralVegetationCreateButton", "Create"))
					.IsEnabled(this, &SPVCreateDialog::GetCreateButtonEnabled)
					.OnClicked(this, &SPVCreateDialog::OnProceduralVegetationCreate)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.Text(NSLOCTEXT("SPVCreateDialog", "ProceduralVegetationCancelButton", "Cancel"))
					.HAlign(HAlign_Center)
					.OnClicked(this, &SPVCreateDialog::OnProceduralVegetationCancelled)
				]
			]
		];

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		[
			SNew(SBox)
			.WidthOverride(610.0f)
			[
				Container
			]
		]
	];
}

bool SPVCreateDialog::OpenCreateModal(const FText& TitleText, TObjectPtr<UProceduralVegetation>& SampleProceduralVegetation)
{
	TSharedRef<SWindow> CreateWindow = SNew(SWindow)
	.Title(TitleText)
	.SizingRule(ESizingRule::Autosized)
	.ClientSize(FVector2D(0.f, 610.f))
	.SupportsMaximize(false)
	.SupportsMinimize(false);

	TSharedRef<SPVCreateDialog> CreateDialog = SNew(SPVCreateDialog)
		.ParentWindow(CreateWindow);

	CreateWindow->SetContent(CreateDialog);

	GEditor->EditorAddModalWindow(CreateWindow);

	if (CreateDialog->bPressedOk)
	{
		if (CreateDialog->bCreateNew)
		{
			SampleProceduralVegetation = nullptr;
			return true;
		}
		if (CreateDialog->SelectedProceduralVegetation.IsValid())
		{
			SampleProceduralVegetation = Cast<UProceduralVegetation>(CreateDialog->SelectedProceduralVegetation.GetAsset());
			return true;
		}
	}

	return false;
}

bool SPVCreateDialog::GetCreateButtonEnabled() const
{
	bool ButtonEnabled = false;
	if (SelectedProceduralVegetation.IsValid())
	{
		ButtonEnabled = true;
	}
	return ButtonEnabled;
}

FReply SPVCreateDialog::OnProceduralVegetationCreate()
{
	bPressedOk = true;
	bCreateNew = false;
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SPVCreateDialog::OnProceduralVegetationCreateNew()
{
	bPressedOk = true;
	bCreateNew = true;
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SPVCreateDialog::OnProceduralVegetationCancelled()
{
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
