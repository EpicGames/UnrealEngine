// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class UProceduralVegetation;

class SPVCreateDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPVCreateDialog)
		{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

private:
	bool GetCreateButtonEnabled() const;

	FReply OnProceduralVegetationCreate();
	FReply OnProceduralVegetationCreateNew();
	FReply OnProceduralVegetationCancelled();

public:
	void Construct(const FArguments& InArgs);

	static bool OpenCreateModal(const FText& TitleText, TObjectPtr<UProceduralVegetation>& SampleProceduralVegetation);
private:
	TWeakPtr<SWindow> WeakParentWindow;

	bool bPressedOk = false; 
	bool bCreateNew = false;

	FAssetData SelectedProceduralVegetation;
};
