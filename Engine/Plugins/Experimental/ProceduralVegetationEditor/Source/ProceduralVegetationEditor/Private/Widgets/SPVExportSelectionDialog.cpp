// Copyright Epic Games, Inc. All Rights Reserved.
#include "SPVExportSelectionDialog.h"

#include "Editor.h"
#include "IStructureDetailsView.h"
#include "ProceduralVegetation.h"
#include "PropertyEditorModule.h"
#include "SEnumCombo.h"
#include "SPrimaryButton.h"

#include "Algo/AnyOf.h"

#include "Nodes/PCGEditorGraphNodeBase.h"
#include "Nodes/PVExportSettings.h"

#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "SPVExportSelectionDialog"

class SExportEntryStatRow : public SMultiColumnTableRow<TSharedPtr<FStatType>>
{
public:
	SLATE_BEGIN_ARGS(SExportEntryStatRow)
		{}

		SLATE_ARGUMENT(TSharedPtr<FStatType>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FStatType>>::Construct(FSuperRowType::FArguments(), OwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (!Item.IsValid())
		{
			return SNew(STextBlock).Text(FText::GetEmpty());
		}

		if (ColumnName == "Name")
		{
			return SNew(STextBlock)
				.Text(Item->Key);
		}
		else if (ColumnName == "Count")
		{
			return SNew(STextBlock)
				.Justification(ETextJustify::Right)
				.Text(FText::AsNumber(Item->Value));
		}

		return SNew(STextBlock)
			.Text(FText::FromString(TEXT("Invalid Column")));
	}

private:
	TSharedPtr<FStatType> Item;
};

void SPVExportSelectionDialog::Construct(const FArguments& InArgs)
{
	ExportEntries = InArgs._ExportEntries;

	const bool bIsAnyNodeSelected = Algo::AnyOf(
		ExportEntries,
		[](const TObjectPtr<UPVExportEntry>& ExportEntry)
			{
				return ExportEntry->bIsNodeSelected;
			}
	);
	OnExportTypeChanged(
		bIsAnyNodeSelected
		? EPVExportType::Selected
		: EPVExportType::Batch
	);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	SWindow::Construct(
		SWindow::FArguments()
		.Title(InArgs._Title)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(800, 600)
		)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(2, 2, 2, 2)
			[
				SNew(SSplitter)
				+ SSplitter::Slot()
				.Resizable(true)
				.SizeRule(SSplitter::FractionOfParent)
				.Value(0.3f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Center)
					[
						SNew(SSegmentedControl<EPVExportType>)
						.Value(this, &SPVExportSelectionDialog::GetExportType)
						.OnValueChanged(this, &SPVExportSelectionDialog::OnExportTypeChanged)
						+ SSegmentedControl<EPVExportType>::Slot(EPVExportType::Batch)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTip(LOCTEXT("PVExportTypeBatchLabel", "Batch export all output nodes"))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PVBatchExportLabel", "Batch"))
						]
						+ SSegmentedControl<EPVExportType>::Slot(EPVExportType::Selected)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTip(LOCTEXT("PVExportTypeSelectedLabel", "Export only selected output nodes in the graph"))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PVSelectedExportLabel", "Selected"))
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.VAlign(VAlign_Fill)
					[
						SNew(SSplitter)
						.Orientation(EOrientation::Orient_Vertical)
						+ SSplitter::Slot()
						.Resizable(true)
						.SizeRule(SSplitter::FractionOfParent)
						.Value(0.73f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SAssignNew(ExportEntriesTableView, SListView<TObjectPtr<UPVExportEntry>>)
								.ListItemsSource(&ExportEntries)
								.OnGenerateRow(this, &SPVExportSelectionDialog::OnGenerateExportItem)
								.OnSelectionChanged(this, &SPVExportSelectionDialog::OnExportEntrySelected)
								.SelectionMode(ESelectionMode::Single)
								.ClearSelectionOnClick(false)
							]
						]
						+ SSplitter::Slot()
						.Resizable(true)
						.SizeRule(SSplitter::FractionOfParent)
						.Value(0.27f)
						[
							SAssignNew(StatTableView, SListView<TSharedPtr<FStatType>>)
							.ListItemsSource(&CurrentExportEntryStats)
							.OnGenerateRow(this, &SPVExportSelectionDialog::OnGenerateStatItem)
							.HeaderRow(
								SNew(SHeaderRow)
								+ SHeaderRow::Column("Name")
								.DefaultLabel(FText::FromString("Export Stats"))
								.FillWidth(0.7f)
								+ SHeaderRow::Column("Count")
								.DefaultLabel(FText::FromString(""))
								.FillWidth(0.3f)
							)
							.SelectionMode(ESelectionMode::None)
						]
					]
				]
				+ SSplitter::Slot()
				.Resizable(true)
				.SizeRule(SSplitter::FractionOfParent)
				.Value(0.7f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						DetailsView.ToSharedRef()
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(8.0f, 6.0f, 8.0f, 8.0f)
			[
				SNew(SUniformGridPanel)
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SAssignNew(ExportButton, SPrimaryButton)
					.Text(LOCTEXT("Export", "Export"))
					.OnClicked(this, &SPVExportSelectionDialog::OnButtonClick, EAppReturnType::Ok)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SPVExportSelectionDialog::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]
	);

	if (ExportEntriesTableView && ExportEntries.Num() > 0)
	{
		ExportEntriesTableView->SetSelection(ExportEntries[0]);
	}
}

TSharedRef<ITableRow> SPVExportSelectionDialog::OnGenerateExportItem(
	TObjectPtr<UPVExportEntry> InExportEntry,
	const TSharedRef<STableViewBase>& OwnerTable
)
{
	const FText OutputName = InExportEntry->Node->GetNodeTitle(EPCGNodeTitleType::FullTitle);
	return SNew(STableRow<TObjectPtr<UPVExportEntry>>, OwnerTable)
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([=]()
					{
						return InExportEntry->bExport
							? ECheckBoxState::Checked
							: ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([=, this](const ECheckBoxState NewState)
					{
						InExportEntry->bExport = NewState == ECheckBoxState::Checked;
						SetExportType();
					})
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			.Padding(2.0f, 0.0f)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Left)
				.Text(OutputName)
			]
		];
}

TSharedRef<ITableRow> SPVExportSelectionDialog::OnGenerateStatItem(TSharedPtr<FStatType> InStatEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SExportEntryStatRow, OwnerTable).Item(InStatEntry);
}

void SPVExportSelectionDialog::OnExportEntrySelected(TObjectPtr<UPVExportEntry> InExportEntry, ESelectInfo::Type SelectInfo)
{
	check(DetailsView);
	CurrentlySelectedEntry = InExportEntry;
	if (CurrentlySelectedEntry)
	{
		SetupCurrentExportEntryStats();
	}
	else
	{
		CurrentExportEntryStats.Empty();
	}
	if (StatTableView)
	{
		StatTableView->RebuildList();
	}
	DetailsView->SetObject(
		CurrentlySelectedEntry
		? CurrentlySelectedEntry->Settings
		: nullptr,
		true
	);
}

void SPVExportSelectionDialog::SetupCurrentExportEntryStats()
{
	check(CurrentlySelectedEntry);
	CurrentExportEntryStats.Empty();
	if (CurrentlySelectedEntry->Stats.NumPoints > -1)
	{
		CurrentExportEntryStats.Emplace(MakeShared<TPair<FText, int32>>(FText::FromString("Points"), CurrentlySelectedEntry->Stats.NumPoints));
	}
	if (CurrentlySelectedEntry->Stats.NumBranches > -1)
	{
		CurrentExportEntryStats.Emplace(MakeShared<TPair<FText, int32>>(FText::FromString("Branches"), CurrentlySelectedEntry->Stats.NumBranches));
	}
	if (CurrentlySelectedEntry->Stats.NumVertices > -1)
	{
		CurrentExportEntryStats.Emplace(MakeShared<TPair<FText, int32>>(FText::FromString("Vertices (Trunk)"),
			CurrentlySelectedEntry->Stats.NumVertices));
	}
	if (CurrentlySelectedEntry->Stats.NumTris > -1)
	{
		CurrentExportEntryStats.Emplace(
			MakeShared<TPair<FText, int32>>(FText::FromString("Triangles (Trunk)"), CurrentlySelectedEntry->Stats.NumTris));
	}
	if (CurrentlySelectedEntry->Stats.NumBones > -1)
	{
		CurrentExportEntryStats.Emplace(MakeShared<TPair<FText, int32>>(FText::FromString("Bones"), CurrentlySelectedEntry->Stats.NumBones));
	}
	if (CurrentlySelectedEntry->bExportFoliage)
	{
		if (CurrentlySelectedEntry->Stats.NumFoliage > -1)
		{
			CurrentExportEntryStats.Emplace(MakeShared<TPair<FText, int32>>(FText::FromString("Unique Foliage"),
				CurrentlySelectedEntry->Stats.NumFoliage));
		}
		if (CurrentlySelectedEntry->Stats.NumFoliageInstances > -1)
		{
			CurrentExportEntryStats.Emplace(MakeShared<TPair<FText, int32>>(FText::FromString("Num Foliage Instances"),
				CurrentlySelectedEntry->Stats.NumFoliageInstances));
		}
	}
}

FReply SPVExportSelectionDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	if (ButtonID == EAppReturnType::Cancel || ButtonID == EAppReturnType::Ok)
	{
		// Only close the window if canceling or if the ok
		RequestDestroyWindow();
	}
	else
	{
		// reset the user response in case the window is closed using 'x'.
		UserResponse = EAppReturnType::Cancel;
	}
	return FReply::Handled();
}

EPVExportType SPVExportSelectionDialog::GetExportType() const
{
	return ExportType;
}

void SPVExportSelectionDialog::OnExportTypeChanged(const EPVExportType InNewValue)
{
	ExportType = InNewValue;
	if (ExportType == EPVExportType::Batch)
	{
		for (const TObjectPtr<UPVExportEntry>& ExportEntry : ExportEntries)
		{
			ExportEntry->bExport = true;
		}
	}
	else if (ExportType == EPVExportType::Selected)
	{
		for (const TObjectPtr<UPVExportEntry>& ExportEntry : ExportEntries)
		{
			ExportEntry->bExport = ExportEntry->bIsNodeSelected;
		}
	}
}

void SPVExportSelectionDialog::SetExportType()
{
	bool bSelectedMode = true;
	for (const TObjectPtr<UPVExportEntry>& ExportEntry : ExportEntries)
	{
		bSelectedMode &= ExportEntry->bExport == ExportEntry->bIsNodeSelected;
	}
	ExportType = bSelectedMode
		? EPVExportType::Selected
		: EPVExportType::Batch;
}

void SPVExportSelectionDialog::UpdateExportButtonState()
{
	bool HasValidExport = false;
	for (const TObjectPtr<UPVExportEntry>& ExportEntry : ExportEntries)
	{
		HasValidExport |= ExportEntry->bExport;
	}
	ExportButton->SetEnabled(HasValidExport);
}

EAppReturnType::Type SPVExportSelectionDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

#undef LOCTEXT_NAMESPACE
