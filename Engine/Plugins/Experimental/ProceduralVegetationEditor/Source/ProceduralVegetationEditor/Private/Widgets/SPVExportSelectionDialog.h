// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Exporter/PVExporter.h"

#include "Input/Reply.h"

#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"

#include "SPVExportSelectionDialog.generated.h"

class IDetailsView;
class IStructureDetailsView;
class ITableRow;
class STableViewBase;
class UPCGEditorGraphNodeBase;
class UPVExportSettings;
class SScrollBox;
class UProceduralVegetationGraph;
class UPCGNode;
class SPrimaryButton;

using FStatType = TPair<FText, int32>;

UENUM()
enum class EPVExportType : int32
{
	Batch,
	Selected
};

class SPVExportSelectionDialog : public SWindow
{
	SLATE_BEGIN_ARGS(SPVExportSelectionDialog)
		{}

		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(TArray<TObjectPtr<UPVExportEntry>>, ExportEntries)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

private:
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	TSharedRef<ITableRow> OnGenerateExportItem(TObjectPtr<UPVExportEntry> InExportEntry, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGenerateStatItem(TSharedPtr<FStatType> InStatEntry, const TSharedRef<STableViewBase>& OwnerTable);

	void OnExportEntrySelected(TObjectPtr<UPVExportEntry> InExportEntry, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);
	void SetupCurrentExportEntryStats();
	
	EPVExportType GetExportType() const;
	void OnExportTypeChanged(const EPVExportType InNewValue);
	void SetExportType();

	void UpdateExportButtonState();

private:
	TSharedPtr<SScrollBox> ScrollBox;

	TSharedPtr<SPrimaryButton> ExportButton;
	TSharedPtr<SListView<TSharedPtr<FStatType>>> StatTableView;
	TSharedPtr<SListView<TObjectPtr<UPVExportEntry>>> ExportEntriesTableView;

	TArray<TObjectPtr<UPVExportEntry>> ExportEntries;
	
	TSharedPtr<IDetailsView> DetailsView;

	TObjectPtr<UPVExportEntry> CurrentlySelectedEntry = nullptr;
	
	TArray<TSharedPtr<FStatType>> CurrentExportEntryStats;

	EPVExportType ExportType = EPVExportType::Selected;

	EAppReturnType::Type UserResponse = EAppReturnType::Type::Cancel;
};
