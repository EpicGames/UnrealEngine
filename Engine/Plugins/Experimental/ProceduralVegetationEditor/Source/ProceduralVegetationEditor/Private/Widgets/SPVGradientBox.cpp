// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPVGradientBox.h"

void SPVGradientBox::Construct(const FArguments& InArgs)
{
	BoxSize = InArgs._BoxSize;

	for (int32 Index = 0; Index < 7; ++Index)
	{
		const FLinearColor DrawColor = FLinearColor(240.0f - (Index * 40.f), 1.f, 1.f).HSVToLinearRGB();
		GradientStops.Add(FSlateGradientStop(BoxSize * (static_cast<float>(Index) / 6.0f), DrawColor));
	}
}

int32 SPVGradientBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
                              FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector4f CornerRadius = FVector4f(4.0f, 4.0f, 4.0f, 4.0f);
	FSlateDrawElement::MakeGradient(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(FVector2f(BoxSize, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform()),
		GradientStops,
		Orient_Vertical,
		ESlateDrawEffect::None,
		CornerRadius
	);
	return LayerId;
}

FVector2D SPVGradientBox::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(BoxSize, 20.0f);
}
