// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"

class SPVGradientBox : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPVGradientBox) {}
		SLATE_ARGUMENT(float, BoxSize)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	                      FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle,
	                      bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	
private:
	float BoxSize = 0.0f;
	
	TArray<FSlateGradientStop> GradientStops;
};
