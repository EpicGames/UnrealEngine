// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class ProceduralVegetationEditor : ModuleRules
{
	public ProceduralVegetationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateIncludePaths.Add(Path.Combine(ModuleDirectory, "../ProceduralVegetation/Private"));

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedPreviewScene",
				"AssetDefinition",
				"AssetTools",
				"Chaos",
				"ContentBrowser",
				"ContentBrowserData",
				"Core",
				"CoreUObject",
				"CurveEditor",
				"DataflowEditor",
				"DynamicWind",
				"DynamicWindEditor",
				"EditorFramework",
				"EditorWidgets",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"GeometryScriptingCore",
				"InputCore",
				"InteractiveToolsFramework",
				"Json",
				"JsonUtilities",
				"LevelEditor",
				"MeshConversion",
				"MeshConversionEngineTypes",
				"ModelingComponents",
				"PCG",
				"PCGEditor",
				"PackagesDialog",
				"PlanarCut",
				"ProceduralVegetation",
				"Projects",
				"PropertyEditor",
				"RHI",
				"RenderCore",
				"SkeletalMeshDescription",
				"Slate",
				"SlateCore",
				"StaticMeshDescription",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd"
			}
		);
	}
}