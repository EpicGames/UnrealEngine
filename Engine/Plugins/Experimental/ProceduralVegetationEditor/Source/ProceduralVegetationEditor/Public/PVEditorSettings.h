// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PVEditorSettings.generated.h"

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta = (DisplayName = "Procedural Vegetation Editor Settings"))
class UPVEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Color used for data pins of type GrowthData */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor GrowthDataPinColor = FColor(239, 239, 239);

	/** Color used for data pins of type MeshData */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor MeshDataPinColor = FColor(3, 234, 234);

	/** Color used for data pins of type FoliageMeshData */
	UPROPERTY(EditAnywhere, config, Category = Node, meta = (HideAlphaChannel))
	FLinearColor FoliageMeshDataPinColor = FColor(0, 255, 0);

	UPROPERTY(EditAnywhere, config, Category = Viewport)
	bool bShowMannequin = false;

	UPROPERTY(EditAnywhere, config, Category = Viewport)
	float MannequinOffset = 0.0f;

	UPROPERTY(EditAnywhere, config, Category = Viewport)
	bool bShowScaleVisualization = false;

	UPROPERTY(EditAnywhere, config, Category = Export)
	bool bShowStats = true;

	UPROPERTY(EditAnywhere, config, Category = Export)
	bool bAutoFocusViewport = false;

	UPROPERTY(EditAnywhere, config, Category = Viewport)
	bool bUseMeshSkeletonPreview = false;
	
	UPROPERTY(EditAnywhere, config, Category = Viewport, meta = (UIMin=0.0f, ClampMin=0.0f))
	float PointScaleBias = 1.2f;
	
	UPROPERTY(EditAnywhere, config, Category = Viewport)
	FLinearColor SkeletonDefaultColor = FLinearColor::White;
};
