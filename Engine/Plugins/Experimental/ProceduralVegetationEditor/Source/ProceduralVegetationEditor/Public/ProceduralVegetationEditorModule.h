// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class UProceduralVegetation;
class AActor;
class FUICommandList;
class FExtender;
struct FToolMenuContext;

namespace PVEditor
{
	extern const FName MessageLogName;
};

DECLARE_LOG_CATEGORY_EXTERN(LogProceduralVegetationEditor, Log, All);

class PROCEDURALVEGETATIONEDITOR_API FProceduralVegetationEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/** IModuleInterface implementation end */

private:
	TSharedPtr<class IFabWorkflowFactory> MegaSeedImportWorkflowFactory;
	FDelegateHandle LevelEditorExtenderDelegateHandle;
	
	void RegisterMenus();
	void ExecuteOpenInPVEditor(const FToolMenuContext& InContext);
	bool CanExecuteOpenInPVEditor(const FToolMenuContext& InContext);

	void RegisterLevelEditorMenuExtension();
	void UnregisterLevelEditorMenuExtension();
	void RegisterPinColorAndIcons();
	void UnregisterPinColorAndIcons();

	static TSharedRef<FExtender> OnExtendLevelEditorActorSelectionMenu(const TSharedRef<FUICommandList> InCommandList, TArray<AActor*> InSelectedActors);
	static void ExecuteOpenInPVEditorOnSelectedActors();
	static UProceduralVegetation* GetProceduralVegetationFromAsset(UObject* InObject);
	static UProceduralVegetation* GetProceduralVegetationFromActor(AActor* InActor);
};
